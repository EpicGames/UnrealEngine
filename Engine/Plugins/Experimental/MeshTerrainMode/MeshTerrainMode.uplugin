{
	"FileVersion": 3,
	"Version": 1,
	"VersionName": "1.0",
	"FriendlyName": "Mesh Terrain Mode",
	"Description": "Mesh Terrain Mode includes a suite of interactive tools for creating and editing Mesh Partitions in the Editor",
	"Category": "Mesh Partition",
	"CreatedBy": "Epic Games, Inc.",
	"CreatedByURL": "https://epicgames.com",
	"DocsURL": "https://dev.epicgames.com/community/learning/knowledge-base/nK7J/unreal-engine-introduction-to-mesh-terrain#meshterrainmode",
	"MarketplaceURL": "",
	"SupportURL": "",
	"CanContainContent": true,
	"EnabledByDefault": false,
	"IsBetaVersion": false,
	"IsExperimentalVersion": true,
	"Installed": false,
	"Modules": [
		{
			"Name": "MeshTerrainMode",
			"Type": "Editor",
			"LoadingPhase": "Default"
		}
	],
	"Plugins": [
		{
			"Name": "MeshModelingToolset",
			"Enabled": true,
			"TargetAllowList": [
				"Editor"
			]
		},
		{
			"Name": "MeshModelingToolsetExp",
			"Enabled": true,
			"TargetAllowList": [
				"Editor"
			]
		},
		{
			"Name": "MeshLODToolset",
			"Enabled": true,
			"TargetAllowList": [
				"Editor"
			]
		},
		{
			"Name": "ToolPresets",
			"Enabled": true,
			"TargetAllowList": [
				"Editor"
			]
		},
		{
			"Name": "StylusInput",
			"Enabled": true,
			"TargetAllowList": [
				"Editor"
			]
		},
		{
			"Name": "ModelingToolsEditorMode",
			"Enabled": true,
			"TargetAllowList": [
				"Editor"
			]
		},
		{
			"Name": "MeshPartition",
			"Enabled": true
		}
	]
}