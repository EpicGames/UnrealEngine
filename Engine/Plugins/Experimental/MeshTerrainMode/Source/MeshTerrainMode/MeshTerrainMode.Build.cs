// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;

public class MeshTerrainMode : ModuleRules
{
	public MeshTerrainMode(ReadOnlyTargetRules Target) : base(Target)
	{
		PCHUsage = ModuleRules.PCHUsageMode.UseExplicitOrSharedPCHs;
		bAllowUETypesInNamespaces = true;

		PublicDependencyModuleNames.AddRange(
			new string[]
			{
				"Core"
				// ... add other public dependencies that you statically link with here ...
			}
		);

		PrivateDependencyModuleNames.AddRange(
			new string[]
			{
				"CoreUObject",
				"RHI",
				"Slate",
				"SlateCore",
				"Engine",
				"InputCore",
				"EditorFramework",
				"UnrealEd",
				"ContentBrowserData",
				"StatusBar",
				"Projects",
				"LevelEditor",
				"TypedElementRuntime",
				"InteractiveToolsFramework",
				"EditorInteractiveToolsFramework",
				"GeometryFramework",
				"GeometryCore",
				"GeometryProcessingInterfaces",
				"DynamicMesh",
				"ModelingComponents",
				"ModelingComponentsEditorOnly",
				"MeshModelingTools",
				"MeshModelingToolsExp",
				"MeshModelingToolsEditorOnly",
				"MeshModelingToolsEditorOnlyExp",
				"MeshLODToolset",
				"ToolWidgets",
				"EditorWidgets",
				"ModelingEditorUI",
				"WidgetRegistration",
				"DeveloperSettings",
				"PropertyEditor",
				"ToolMenus",
				"EditorConfig",
				"ToolPresetAsset",
				"ToolPresetEditor",
				"EditorConfig",
				"StylusInput",
				"ModelingToolsEditorMode",
				"MeshPartitionModelingToolset",
				"MeshPartitionEditor",
				"MeshPartition"
				// ... add private dependencies that you statically link with here ...	
			}
		);

		DynamicallyLoadedModuleNames.AddRange(
			new string[]
			{
				"PlacementMode"
			}
		);
		
		PrivateIncludePathModuleNames.AddRange(
			new string[]
			{
				"PlacementMode"
			}
		);

		PublicDefinitions.Add("WITH_PROXYLOD=" + (Target.Platform == UnrealTargetPlatform.Win64 ? '1' : '0'));
		PrivateDefinitions.Add("ENABLE_STYLUS_SUPPORT=1");
	}
}