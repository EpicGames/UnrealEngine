// Copyright Epic Games, Inc. All Rights Reserved.

#include "DetailsCustomizations/BakeMeshAttributeToolCustomizations.h"
#include "DetailLayoutBuilder.h"
#include "DetailWidgetRow.h"
#include "Widgets/Input/SComboButton.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"

#include "ModelingWidgets/ModelingCustomizationUtil.h"


#include "BakeMeshAttributeMapsTool.h"
#include "BakeMultiMeshAttributeMapsTool.h"
#include "BakeMeshAttributeVertexTool.h"

#define LOCTEXT_NAMESPACE "BakeMeshAttributeToolCustomizations"

namespace UE::MeshTerrain
{

//
// Bake Details locals
//
namespace BakeCustomizationLocals
{
	/** EBakeMapType UI sections */
	enum class EBakeMapTypeSections
	{
		Surface,
		Lighting,
		Material,
		Mask,
		MaxSections // Must be last
	};
	
	/** EBakeMapType UI section header names */
	static const TArray<FText> BakeMapTypeSectionNames =
		{
			LOCTEXT("BakeMapTypeSurfaceSection", "Surface"),
			LOCTEXT("BakeMapTypeLightingSection", "Lighting"),
			LOCTEXT("BakeMapTypeMaterialSection", "Material"),
			LOCTEXT("BakeMapTypeMasksSection", "IDs & Masks")
		};

	/** EBakeMapType UI section content */
	static const TArray<TArray<EBakeMapType>> BakeMapTypeSectionData =
		{
			// Surface
			{
				EBakeMapType::TangentSpaceNormal,
				EBakeMapType::ObjectSpaceNormal,
				EBakeMapType::FaceNormal,
				EBakeMapType::BentNormal,
				EBakeMapType::Position,
				EBakeMapType::Curvature
			},
			// Lighting
			{
				EBakeMapType::AmbientOcclusion
			},
			// Material
			{
				EBakeMapType::Texture,
				EBakeMapType::MultiTexture
			},
			// Mask
			{
				EBakeMapType::VertexColor,
				EBakeMapType::MaterialID,
				EBakeMapType::PolyGroupID,
				EBakeMapType::One,
				EBakeMapType::Zero
			}
		};

	/**
	 * Return an array of names from the metadata flag "ValidEnumValues".
	 * Sourced from privately defined PropertyEditorHelpers::GetValidEnumsFromPropertyOverride().
	 *
	 * @param Property Property to inspect
	 * @param InEnum UEnum class that this property represents
	 * @return array of FName for each valid enum value
	 */
	TArray<FName> GetValidEnumValues(const FProperty* Property, const UEnum* InEnum)
	{
		TArray<FName> ValidEnumValues;
		static const FName ValidEnumValuesName(TEXT("ValidEnumValues"));
		if(Property->HasMetaData(ValidEnumValuesName))
		{
			TArray<FString> ValidEnumValuesAsString;
			Property->GetMetaData(ValidEnumValuesName).ParseIntoArray(ValidEnumValuesAsString, TEXT(","));
			for(FString& Value : ValidEnumValuesAsString)
			{
				Value.TrimStartInline();
				ValidEnumValues.Add(*InEnum->GenerateFullEnumName(*Value));
			}
		}
		return ValidEnumValues;
	}

	/**
	 * Return the UEnum associated with the BitmaskEnum metadata flag.
	 * Sourced from privately defined SPropertyEditorNumeric::Construct().
	 *
	 * @param Property the property to query for the BitmaskEnum metadata
	 * @return the UEnum defined by the BitmaskEnum metadata. nullptr if undefined or cannot be found.
	 */
	UEnum* GetBitmaskEnum(const FProperty* Property)
	{
		UEnum* BitmaskEnum = nullptr;
		static const FName BitmaskEnumFlagName(TEXT("BitmaskEnum"));
		const FString& BitmaskEnumName = Property->GetMetaData(BitmaskEnumFlagName);
		if (!BitmaskEnumName.IsEmpty())
		{
			BitmaskEnum = UClass::TryFindTypeSlow<UEnum>(BitmaskEnumName);
		}
		return BitmaskEnum;
	}

	/**
	 * Utility struct that holds bitmask flag UI data.
	 */
	struct FBitmaskFlagInfo
	{
		int32 Value;
		FText DisplayName;
		FText ToolTipText;
	};

	/**
	 * Return an array of FBitmaskFlagInfo for each valid enum value in the enum
	 * defined by a property's BitmaskEnum metadata.
	 * Sourced from privately defined SPropertyEditorNumeric::Construct()
	 *
	 * @param Property the property to query
	 * @return an array of FBitmaskFlagInfo for each valid enum value
	 */
	TArray<FBitmaskFlagInfo> GetBitmaskEnumFlags(const FProperty* Property)
	{
		constexpr int32 BitmaskBitCount = sizeof(int) << 3;
		
		TArray<FBitmaskFlagInfo> Result;
		Result.Empty(BitmaskBitCount);
		const UEnum* BitmaskEnum = GetBitmaskEnum(Property);
		if (!BitmaskEnum)
		{
			return Result;
		}

		auto AddNewBitmaskFlag = [&Result, BitmaskEnum](const int32 EnumIndex, const int64& EnumValue)
		{
			Result.Emplace();
			FBitmaskFlagInfo* BitmaskFlag = &Result.Last();

			BitmaskFlag->Value = EnumValue;
			BitmaskFlag->DisplayName = BitmaskEnum->GetDisplayNameTextByIndex(EnumIndex);
			BitmaskFlag->ToolTipText = BitmaskEnum->GetToolTipTextByIndex(EnumIndex);
			if (BitmaskFlag->ToolTipText.IsEmpty())
			{
				BitmaskFlag->ToolTipText = FText::Format(LOCTEXT("BitmaskDefaultFlagToolTipText", "Toggle {0} on/off"), BitmaskFlag->DisplayName);
			}
		};

		static const FName UseEnumValuesAsMaskValuesName(TEXT("UseEnumValuesAsMaskValuesInEditor"));
		const bool bUseEnumValuesAsMaskValues = BitmaskEnum->GetBoolMetaData(UseEnumValuesAsMaskValuesName);
		const TArray<FName> AllowedPropertyEnums = GetValidEnumValues(Property, BitmaskEnum);
		// Note: This loop doesn't include (BitflagsEnum->NumEnums() - 1) in order to skip the implicit "MAX"
		// value that gets added to the enum type at compile time.
		for (int32 BitmaskEnumIndex = 0; BitmaskEnumIndex < BitmaskEnum->NumEnums() - 1; ++BitmaskEnumIndex)
		{
			const int64 EnumValue = BitmaskEnum->GetValueByIndex(BitmaskEnumIndex);
			bool bShouldBeHidden = BitmaskEnum->HasMetaData(TEXT("Hidden"), BitmaskEnumIndex);
			if (!bShouldBeHidden && AllowedPropertyEnums.Num() > 0)
			{
				bShouldBeHidden = AllowedPropertyEnums.Find(BitmaskEnum->GetNameByIndex(BitmaskEnumIndex)) == INDEX_NONE;
			}
			if (EnumValue >= 0 && !bShouldBeHidden)
			{
				if (bUseEnumValuesAsMaskValues)
				{
					if (EnumValue < MAX_int64 && FMath::IsPowerOfTwo(EnumValue))
					{
						AddNewBitmaskFlag(BitmaskEnumIndex, EnumValue);
					}
				}
				else if (EnumValue < BitmaskBitCount)
				{
					AddNewBitmaskFlag(BitmaskEnumIndex, static_cast<int64>(1) << EnumValue);
				}
			}
		}
		return Result;
	}

	/**
	 * Customize the EBakeMapTypes bitmask enum with predefined section headers.
	 *
	 * This function extends the existing default widget rather than replacing it, so
	 * the functionality is tied to its definition in SPropertyEditorNumeric. Should
	 * the widget hierarchy of SPropertyEditorNumeric change, this function should fail
	 * gracefully.
	 *
	 * @param DetailBuilder the detail layout builder
	 * @param MapTypesHandle handle to a bitmask enum property for EBakeMapTypes
	 * @param bMultiSelect if true, allow multiple enum values to be selected
	 */
	void CustomizeMapTypesEnum(IDetailLayoutBuilder& DetailBuilder, const TSharedPtr<IPropertyHandle>& MapTypesHandle, const bool bMultiSelect)
	{
		if (!ensure(MapTypesHandle->IsValidHandle()))
		{
			return;
		}
		FProperty* MapTypesProperty = MapTypesHandle->GetProperty();

		IDetailPropertyRow* DetailRow = DetailBuilder.EditDefaultProperty(MapTypesHandle);
		TSharedPtr<SWidget> NameWidget, ValueWidget;
		DetailRow->GetDefaultWidgets(NameWidget, ValueWidget);

		// Find first SComboButton in hierarchy
		TSharedPtr<SComboButton> ComboWidget;
		UE::ModelingUI::ProcessChildWidgetsByType(
			ValueWidget->AsShared(),
			TEXT("SComboButton"),
			[&ComboWidget](TSharedRef<SWidget> Widget)->bool
		{
			ComboWidget = StaticCastSharedPtr<SComboButton>(TSharedPtr<SWidget>(Widget));
			// Stop processing after first occurrence
			return false;
		});
		
		if (!ComboWidget.IsValid())
		{
			return;
		}

		// Override the ComboButton menu builder with customized sections & entries
		ComboWidget->SetOnGetMenuContent(FOnGetContent::CreateLambda([MapTypesProperty, DetailRow, bMultiSelect]()
		{
			FMenuBuilder MenuBuilder(!bMultiSelect, nullptr);

			TArray<FBitmaskFlagInfo> BitmaskFlags = GetBitmaskEnumFlags(MapTypesProperty);
			TMap<int64, FBitmaskFlagInfo> BitmaskFlagMap;
			const int NumFlags = BitmaskFlags.Num();
			for (int Idx = 0; Idx < NumFlags; ++Idx)
			{
				BitmaskFlagMap.Emplace(BitmaskFlags[Idx].Value, BitmaskFlags[Idx]);
			}

			auto AddMenuEntry = [DetailRow, &MenuBuilder, bMultiSelect](const FBitmaskFlagInfo& BitmaskFlag)
			{
				MenuBuilder.AddMenuEntry(
					BitmaskFlag.DisplayName,
					BitmaskFlag.ToolTipText,
					FSlateIcon(),
					FUIAction
					(
						FExecuteAction::CreateLambda([DetailRow, BitmaskFlag, bMultiSelect]()
						{
							const TSharedPtr<IPropertyHandle> Property = DetailRow->GetPropertyHandle();
							if (Property.IsValid())
							{
								int32 Value;
								if (Property->GetValue(Value) == FPropertyAccess::Success)
								{
									bMultiSelect ? Property->SetValue(Value ^ BitmaskFlag.Value) : Property->SetValue(BitmaskFlag.Value);
								}
							}
						}),
						FCanExecuteAction(),
						FIsActionChecked::CreateLambda([DetailRow, BitmaskFlag]() -> bool
						{
							const TSharedPtr<IPropertyHandle> Property = DetailRow->GetPropertyHandle();
							if (Property.IsValid())
							{
								int32 Value;
								if (Property->GetValue(Value) == FPropertyAccess::Success)
								{
									return (Value & BitmaskFlag.Value) != 0;
								}
							}
							return false;
						})
					),
					NAME_None,
					EUserInterfaceActionType::Check);
			};

			// Explicitly handle EBakeMapTypes::None to be at the head of the list if valid.
			if (const FBitmaskFlagInfo* BitmaskFlag = BitmaskFlagMap.Find(static_cast<int64>(EBakeMapType::None)))
			{
				AddMenuEntry(*BitmaskFlag);
			}

			constexpr int NumSections = static_cast<int>(EBakeMapTypeSections::MaxSections);
			for (int SectionIdx = 0; SectionIdx < NumSections; ++SectionIdx)
			{
				MenuBuilder.BeginSection(NAME_None, BakeMapTypeSectionNames[SectionIdx]);
				const int NumSectionData = BakeMapTypeSectionData[SectionIdx].Num();
				for (int DataIdx = 0; DataIdx < NumSectionData; ++DataIdx)
				{
					const EBakeMapType DataType = BakeMapTypeSectionData[SectionIdx][DataIdx];
					if (const FBitmaskFlagInfo* BitmaskFlag = BitmaskFlagMap.Find(static_cast<int64>(DataType)))
					{
						AddMenuEntry(*BitmaskFlag);
					}
				}
				MenuBuilder.EndSection();
			}
			return MenuBuilder.MakeWidget();
		}));

		// Assign default name and modified value widget to respective content widgets.
		DetailRow->CustomWidget()
			.NameContent()
			[
				NameWidget->AsShared()
			]
			.ValueContent()
			[
				// Fix the size of the SComboButton by setting a MaxWidth HBox.
				// This addresses the pop-up menu flickering when the parent SComboWidget
				// is resized while the menu remains visible after selection.
				SNew(SHorizontalBox)
				+ SHorizontalBox::Slot()
				.VAlign(VAlign_Center)
				.Padding(FMargin(0))
				.MaxWidth(125.0f)
				[
					ValueWidget->AsShared()
				]
			];
	}
};


//
// Mesh Map Bake
//


TSharedRef<IDetailCustomization> FBakeMeshAttributeMapsToolDetails::MakeInstance()
{
	return MakeShareable(new FBakeMeshAttributeMapsToolDetails);
}


void FBakeMeshAttributeMapsToolDetails::CustomizeDetails(IDetailLayoutBuilder& DetailBuilder)
{
	const TSharedPtr<IPropertyHandle> MapTypesHandle = DetailBuilder.GetProperty(GET_MEMBER_NAME_CHECKED(UBakeMeshAttributeMapsToolProperties, MapTypes), UBakeMeshAttributeMapsToolProperties::StaticClass());
	BakeCustomizationLocals::CustomizeMapTypesEnum(DetailBuilder, MapTypesHandle, true);
}


//
// MultiMesh Map Bake
//


TSharedRef<IDetailCustomization> FBakeMultiMeshAttributeMapsToolDetails::MakeInstance()
{
	return MakeShareable(new FBakeMultiMeshAttributeMapsToolDetails);
}


void FBakeMultiMeshAttributeMapsToolDetails::CustomizeDetails(IDetailLayoutBuilder& DetailBuilder)
{
	const TSharedPtr<IPropertyHandle> MapTypesHandle = DetailBuilder.GetProperty(GET_MEMBER_NAME_CHECKED(UBakeMultiMeshAttributeMapsToolProperties, MapTypes), UBakeMultiMeshAttributeMapsToolProperties::StaticClass());
	BakeCustomizationLocals::CustomizeMapTypesEnum(DetailBuilder, MapTypesHandle, true);
}


//
// Mesh Vertex Bake
//


TSharedRef<IDetailCustomization> FBakeMeshAttributeVertexToolDetails::MakeInstance()
{
	return MakeShareable(new FBakeMeshAttributeVertexToolDetails);
}


void FBakeMeshAttributeVertexToolDetails::CustomizeDetails(IDetailLayoutBuilder& DetailBuilder)
{
	const TSharedPtr<IPropertyHandle> OutputTypeHandle = DetailBuilder.GetProperty(GET_MEMBER_NAME_CHECKED(UBakeMeshAttributeVertexToolProperties, OutputType), UBakeMeshAttributeVertexToolProperties::StaticClass());
	BakeCustomizationLocals::CustomizeMapTypesEnum(DetailBuilder, OutputTypeHandle, false);

	const TSharedPtr<IPropertyHandle> OutputTypeRHandle = DetailBuilder.GetProperty(GET_MEMBER_NAME_CHECKED(UBakeMeshAttributeVertexToolProperties, OutputTypeR), UBakeMeshAttributeVertexToolProperties::StaticClass());
	const TSharedPtr<IPropertyHandle> OutputTypeGHandle = DetailBuilder.GetProperty(GET_MEMBER_NAME_CHECKED(UBakeMeshAttributeVertexToolProperties, OutputTypeG), UBakeMeshAttributeVertexToolProperties::StaticClass());
	const TSharedPtr<IPropertyHandle> OutputTypeBHandle = DetailBuilder.GetProperty(GET_MEMBER_NAME_CHECKED(UBakeMeshAttributeVertexToolProperties, OutputTypeB), UBakeMeshAttributeVertexToolProperties::StaticClass());
	const TSharedPtr<IPropertyHandle> OutputTypeAHandle = DetailBuilder.GetProperty(GET_MEMBER_NAME_CHECKED(UBakeMeshAttributeVertexToolProperties, OutputTypeA), UBakeMeshAttributeVertexToolProperties::StaticClass());
	BakeCustomizationLocals::CustomizeMapTypesEnum(DetailBuilder, OutputTypeRHandle, false);
	BakeCustomizationLocals::CustomizeMapTypesEnum(DetailBuilder, OutputTypeGHandle, false);
	BakeCustomizationLocals::CustomizeMapTypesEnum(DetailBuilder, OutputTypeBHandle, false);
	BakeCustomizationLocals::CustomizeMapTypesEnum(DetailBuilder, OutputTypeAHandle, false);
}

}

#undef LOCTEXT_NAMESPACE

