// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "IDetailCustomization.h"

namespace UE::MeshTerrain
{

// Customization for UBakeMeshAttributeMapsToolProperties
class FBakeMeshAttributeMapsToolDetails : public IDetailCustomization
{
public:
	static TSharedRef<IDetailCustomization> MakeInstance();
	virtual void CustomizeDetails(IDetailLayoutBuilder& DetailBuilder) override;
};

// Customization for UBakeMultiMeshAttributeMapsToolProperties
class FBakeMultiMeshAttributeMapsToolDetails : public IDetailCustomization
{
public:
	static TSharedRef<IDetailCustomization> MakeInstance();
	virtual void CustomizeDetails(IDetailLayoutBuilder& DetailBuilder) override;
};

// Customization for UBakeMeshAttributeVertexToolProperties
class FBakeMeshAttributeVertexToolDetails : public IDetailCustomization
{
public:
	static TSharedRef<IDetailCustomization> MakeInstance();
	virtual void CustomizeDetails(IDetailLayoutBuilder& DetailBuilder) override;
};

}