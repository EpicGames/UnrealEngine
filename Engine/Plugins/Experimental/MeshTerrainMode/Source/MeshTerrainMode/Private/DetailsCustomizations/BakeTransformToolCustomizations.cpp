// Copyright Epic Games, Inc. All Rights Reserved.

#include "DetailsCustomizations/BakeTransformToolCustomizations.h"
#include "DetailLayoutBuilder.h"
#include "PropertyHandle.h"
#include "PropertyRestriction.h"

#include "BakeTransformTool.h"


#define LOCTEXT_NAMESPACE "BakeTransformToolCustomizations"

namespace UE::MeshTerrain
{

TSharedRef<IDetailCustomization> FBakeTransformToolDetails::MakeInstance()
{
	return MakeShareable(new FBakeTransformToolDetails);
}

void FBakeTransformToolDetails::CustomizeDetails(IDetailLayoutBuilder& DetailBuilder)
{
	TSharedRef<IPropertyHandle> AllowNoScale = DetailBuilder.GetProperty(GET_MEMBER_NAME_CHECKED(UBakeTransformToolProperties, bAllowNoScale), UBakeTransformToolProperties::StaticClass());
	TSharedRef<IPropertyHandle> BakeScaleHandle = DetailBuilder.GetProperty(GET_MEMBER_NAME_CHECKED(UBakeTransformToolProperties, BakeScale), UBakeTransformToolProperties::StaticClass());
	AllowNoScale->MarkHiddenByCustomization();

	// Note: In practice, currently the tool computes AllowNoScale on Setup and the value does not change, so we don't need to monitor for property changes
	// If AllowNoScale could change while the tool is open, we'd need to add a delegate like this:
	//HasNonUniformScaleHandle->SetOnPropertyValueChanged(FSimpleDelegate::CreateRaw(this, &FBakeTransformToolDetails::EvaluateBakeScaleMethodRestrictions, AllowNoScale, BakeScaleHandle));

	static FText RestrictReason = LOCTEXT("BakeScaleMethodRestrictionText", "cannot only bake rotation: selection includes meshes with both non-uniform scale and rotation");
	BakeScaleMethodEnumRestriction = MakeShareable(new FPropertyRestriction(RestrictReason));
	BakeScaleHandle->AddRestriction(BakeScaleMethodEnumRestriction.ToSharedRef());

	EvaluateBakeScaleMethodRestrictions(AllowNoScale, BakeScaleHandle);
}

void FBakeTransformToolDetails::EvaluateBakeScaleMethodRestrictions(TSharedRef<IPropertyHandle> AllowNoScale, TSharedRef<IPropertyHandle> BakeScaleHandle)
{
	bool bAllowNoScale = false;
	AllowNoScale->GetValue(bAllowNoScale);

	UEnum* BakeScaleEnum = StaticEnum<EBakeScaleMethod>();

	BakeScaleMethodEnumRestriction->RemoveAll();
	if (!bAllowNoScale)
	{
		int32 EnumValue = BakeScaleEnum->GetValueByName("EBakeScaleMethod::DoNotBakeScale");
		BakeScaleMethodEnumRestriction->AddDisabledValue("DoNotBakeScale");

		// if the enum is set to DoNotBakeScale and we don't allow it, switch it over to BakeNonuniformScale instead
		int32 Value = 0;
		BakeScaleHandle->GetValue(Value);
		if (Value == EnumValue)
		{
			Value = BakeScaleEnum->GetValueByName("EBakeScaleMethod::BakeNonuniformScale");
			BakeScaleHandle->SetValue(Value, EPropertyValueSetFlags::NotTransactable | EPropertyValueSetFlags::InteractiveChange);
		}
	}


}

}

#undef LOCTEXT_NAMESPACE
