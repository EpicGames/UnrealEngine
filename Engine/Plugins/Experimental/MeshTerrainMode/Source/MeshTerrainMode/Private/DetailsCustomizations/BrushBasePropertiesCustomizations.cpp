// Copyright Epic Games, Inc. All Rights Reserved.

#include "DetailsCustomizations/BrushBasePropertiesCustomizations.h"

#include "BaseTools/BaseBrushTool.h"
#include "DetailLayoutBuilder.h"
#include "DetailWidgetRow.h"
#include "MeshPartitionAttributePaintTool.h"
#include "ModelingToolsEditorModeStyle.h"
#include "PropertyHandle.h"
#include "Widgets/MeshTerrainModeUtil.h"
#include "Widgets/Input/SSpinBox.h"
#include "Widgets/Images/SImage.h"
#include "Widgets/Input/SCheckBox.h"

#define LOCTEXT_NAMESPACE "BrushBasePropertiesDetails"

using namespace UE::MeshTerrain;

TSharedRef<IDetailCustomization> FBrushBasePropertiesDetails::MakeInstance()
{
	return MakeShareable(new FBrushBasePropertiesDetails);
}

void FBrushBasePropertiesDetails::CustomizeDetails(IDetailLayoutBuilder& DetailBuilder)
{
	TArray<TWeakObjectPtr<UObject>> ObjectsBeingCustomized;
	DetailBuilder.GetObjectsBeingCustomized((ObjectsBeingCustomized));
	check(ObjectsBeingCustomized.Num() > 0)
	UBrushBaseProperties* BrushProperties = Cast<UBrushBaseProperties>(ObjectsBeingCustomized[0]);

	// Per-instance slider range override for BrushRadius. SetInstanceMetaData mutates the property
	// handle (FPropertyNode), not the FProperty itself, so other UBrushBaseProperties instances
	// keep the default 1.0..1000.0 range declared on UBrushBaseProperties::BrushRadius.
	{
		const UInteractiveTool* OwningTool = BrushProperties
			? Cast<UInteractiveTool>(BrushProperties->GetOuter())
			: nullptr;

		FString UIMinOverride, UIMaxOverride, ClampMinOverride, ClampMaxOverride;
		if (UE::MeshTerrain::GetBrushRadiusRangeOverride(OwningTool, UIMinOverride, UIMaxOverride, ClampMinOverride, ClampMaxOverride))
		{
			TSharedPtr<IPropertyHandle> BrushRadiusHandle = DetailBuilder.GetProperty(
				GET_MEMBER_NAME_CHECKED(UBrushBaseProperties, BrushRadius),
				UBrushBaseProperties::StaticClass());
			if (BrushRadiusHandle.IsValid() && BrushRadiusHandle->IsValidHandle())
			{
				if (!UIMinOverride.IsEmpty())    { BrushRadiusHandle->SetInstanceMetaData(TEXT("UIMin"),    UIMinOverride);    }
				if (!UIMaxOverride.IsEmpty())    { BrushRadiusHandle->SetInstanceMetaData(TEXT("UIMax"),    UIMaxOverride);    }
				if (!ClampMinOverride.IsEmpty()) { BrushRadiusHandle->SetInstanceMetaData(TEXT("ClampMin"), ClampMinOverride); }
				if (!ClampMaxOverride.IsEmpty()) { BrushRadiusHandle->SetInstanceMetaData(TEXT("ClampMax"), ClampMaxOverride); }
			}
		}
	}

	TSharedPtr<IPropertyHandle> BrushSizeHandle = DetailBuilder.GetProperty(GET_MEMBER_NAME_CHECKED(UBrushBaseProperties, BrushSize), UBrushBaseProperties::StaticClass());
	ensure (BrushSizeHandle->IsValidHandle());
	
	TSharedPtr<IPropertyHandle> PressureSensitivityHandle = DetailBuilder.GetProperty(GET_MEMBER_NAME_CHECKED(UBrushBaseProperties, bEnablePressureSensitivity), UBrushBaseProperties::StaticClass());
	ensure (PressureSensitivityHandle->IsValidHandle());
	PressureSensitivityHandle->MarkHiddenByCustomization();
	
	// build the widget
	IDetailPropertyRow* DetailRow = DetailBuilder.EditDefaultProperty(BrushSizeHandle);
	TSharedPtr<SWidget> NameWidget, ValueWidget;
	DetailRow->GetDefaultWidgets(NameWidget, ValueWidget);
	
	TSharedPtr<SHorizontalBox> ValueContent;

	DetailRow->CustomWidget()
	.NameContent() // property text
	[
		NameWidget->AsShared()
	]
	.ValueContent() // normal property widget and pressure sensitivity toggle
	[
		SAssignNew(ValueContent, SHorizontalBox)
	];

	// always add usual widget
	ValueContent->AddSlot()
	[
		ValueWidget->AsShared()
	];

	// add pressure sensitivity toggle
	ValueContent->AddSlot()
	.AutoWidth()
	[
		SNew(SCheckBox)
		.Style(FAppStyle::Get(), "DetailsView.SectionButton")
		.Padding(FMargin(4, 2))
		.ToolTipText(PressureSensitivityHandle->GetToolTipText())
		.HAlign(HAlign_Center)
		.OnCheckStateChanged_Lambda([BrushProperties](const ECheckBoxState NewState)
		{
			BrushProperties->bEnablePressureSensitivity = NewState == ECheckBoxState::Checked;
		})
		.IsChecked_Lambda([BrushProperties]() -> ECheckBoxState
		{
			return BrushProperties->bEnablePressureSensitivity ? ECheckBoxState::Checked : ECheckBoxState::Unchecked;
		})
		[
			SNew(SHorizontalBox)
			+ SHorizontalBox::Slot()
			.VAlign(VAlign_Center)
			.Padding(FMargin(0))
			.AutoWidth()
			[
				SNew(SImage)
				.Image(FModelingToolsEditorModeStyle::Get()->GetBrush("BrushIcons.PressureSensitivity.Small"))
			]
		]
	];
}

#undef LOCTEXT_NAMESPACE