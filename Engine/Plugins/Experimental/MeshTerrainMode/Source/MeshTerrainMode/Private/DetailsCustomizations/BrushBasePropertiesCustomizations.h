// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "IDetailCustomization.h"

namespace UE::MeshTerrain
{
	class FBrushBasePropertiesDetails : public IDetailCustomization
	{
	public:
		static TSharedRef<IDetailCustomization> MakeInstance();
		void CustomizeDetails(IDetailLayoutBuilder& DetailBuilder) override;
	};
}