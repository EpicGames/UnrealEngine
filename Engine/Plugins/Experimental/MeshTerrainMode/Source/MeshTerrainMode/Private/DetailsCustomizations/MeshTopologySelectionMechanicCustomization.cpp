// Copyright Epic Games, Inc. All Rights Reserved.

#include "DetailsCustomizations/MeshTopologySelectionMechanicCustomization.h"

#include "DetailCategoryBuilder.h"
#include "DetailLayoutBuilder.h"
#include "DetailWidgetRow.h"
#include "EditorViewportCommands.h"
#include "Framework/MultiBox/MultiBoxBuilder.h" // FSlimHorizontalToolBarBuilder
#include "MeshTerrainModeStyle.h"
#include "ModelingWidgets/ModelingCustomizationUtil.h"
#include "Selection/MeshTopologySelectionMechanic.h"

#define LOCTEXT_NAMESPACE "MeshTopologySelectionMechanicCustomizations"

using namespace UE::ModelingUI;

namespace UE::MeshTerrain
{

TSharedRef<IDetailCustomization> FMeshTopologySelectionMechanicPropertiesDetails::MakeInstance()
{
	return MakeShareable(new FMeshTopologySelectionMechanicPropertiesDetails);
}

void FMeshTopologySelectionMechanicPropertiesDetails::CustomizeDetails(IDetailLayoutBuilder& DetailBuilder)
{
	// Get the property object that we're customizing
	TArray<TWeakObjectPtr<UObject>> ObjectsBeingCustomized;
	DetailBuilder.GetObjectsBeingCustomized(ObjectsBeingCustomized);
	if (!ensure(ObjectsBeingCustomized.Num() > 0))
	{
		return;
	}
	UMeshTopologySelectionMechanicProperties* Properties = CastChecked<UMeshTopologySelectionMechanicProperties>(ObjectsBeingCustomized[0]);
	
	// Create a toolbar for the selection filter
	TSharedPtr<FUICommandList> CommandList = MakeShared<FUICommandList>();
	FSlimHorizontalToolBarBuilder ToolbarBuilder(CommandList, FMultiBoxCustomization::None);

	FName ToolBarStyle = "PolyEd.SelectionToolbar";
	ToolbarBuilder.SetStyle(FMeshTerrainModeStyle::Get().Get(), ToolBarStyle);
	ToolbarBuilder.SetLabelVisibility(EVisibility::Collapsed);

	// Given bool property and the icon name, adds the button to the builder
	auto AddToggleButtonForBool = [&ToolbarBuilder, Properties](TSharedPtr<IPropertyHandle> BoolPropertyHandle, FName IconName)
	{
		ToolbarBuilder.AddToolBarButton(FUIAction(
			FExecuteAction::CreateWeakLambda(Properties, [BoolPropertyHandle]()
			{
				bool bCurrentValue;
				BoolPropertyHandle->GetValue(bCurrentValue);
				BoolPropertyHandle->SetValue(!bCurrentValue);
			}),
			FCanExecuteAction::CreateWeakLambda(Properties, [BoolPropertyHandle]() { 
				return BoolPropertyHandle->IsEditable();
			}),
			FIsActionChecked::CreateWeakLambda(Properties, [BoolPropertyHandle]()
			{
				bool bValue;
				BoolPropertyHandle->GetValue(bValue);
				return bValue;
			})),
			NAME_None, // Extension hook
			TAttribute<FText>(BoolPropertyHandle->GetPropertyDisplayName()), // Label
			TAttribute<FText>(BoolPropertyHandle->GetToolTipText()), // Tooltip
			TAttribute<FSlateIcon>(FSlateIcon(FMeshTerrainModeStyle::Get()->GetStyleSetName(), IconName)), 
			EUserInterfaceActionType::ToggleButton);
	};

	ToolbarBuilder.BeginSection("SelectionFilter");
	ToolbarBuilder.BeginBlockGroup();

	// This first bool we're not going to hide. Instead we're going to end up replacing it with the selection filter below
	TSharedPtr<IPropertyHandle> FirstHandle = nullptr;

	// Hide or add a toggle button for the given property based on bCanSelect
	auto ProcessButtonForBool = [&FirstHandle, AddToggleButtonForBool](TSharedPtr<IPropertyHandle> PropertyHandle, bool bCanSelect, FName IconName, bool& bCurrentlySelecting)
	{
		if (bCanSelect)
		{
			if (FirstHandle == nullptr)
			{
				FirstHandle = PropertyHandle;
			}
			else
			{
				PropertyHandle->MarkHiddenByCustomization();
			}
			AddToggleButtonForBool(PropertyHandle, IconName);
		}
		else
		{
			bCurrentlySelecting = false;
			PropertyHandle->MarkHiddenByCustomization();
		}
	};

	ProcessButtonForBool(DetailBuilder.GetProperty(GET_MEMBER_NAME_CHECKED(UMeshTopologySelectionMechanicProperties, bSelectVertices), UMeshTopologySelectionMechanicProperties::StaticClass()),
		Properties->bCanSelectVertices,
		"PolyEd.SelectCorners",
		Properties->bSelectVertices);

	ProcessButtonForBool(DetailBuilder.GetProperty(GET_MEMBER_NAME_CHECKED(UMeshTopologySelectionMechanicProperties, bSelectEdges), UMeshTopologySelectionMechanicProperties::StaticClass()),
		Properties->bCanSelectEdges,
		"PolyEd.SelectEdges",
		Properties->bSelectEdges);

	ProcessButtonForBool(DetailBuilder.GetProperty(GET_MEMBER_NAME_CHECKED(UMeshTopologySelectionMechanicProperties, bSelectFaces), UMeshTopologySelectionMechanicProperties::StaticClass()),
		Properties->bCanSelectFaces,
		"PolyEd.SelectFaces",
		Properties->bSelectFaces);


	ToolbarBuilder.EndBlockGroup();
	ToolbarBuilder.EndSection();

	TSharedPtr<IPropertyHandle> SelectEdgeLoopsHandle = DetailBuilder.GetProperty(GET_MEMBER_NAME_CHECKED(UMeshTopologySelectionMechanicProperties, bSelectEdgeLoops),
		UMeshTopologySelectionMechanicProperties::StaticClass());
	SelectEdgeLoopsHandle->MarkHiddenByCustomization();
	
	TSharedPtr<IPropertyHandle> SelectEdgeRingsHandle = DetailBuilder.GetProperty(GET_MEMBER_NAME_CHECKED(UMeshTopologySelectionMechanicProperties, bSelectEdgeRings),
		UMeshTopologySelectionMechanicProperties::StaticClass());
	SelectEdgeRingsHandle->MarkHiddenByCustomization();

	if (Properties->bDisplayPolygroupReliantControls)
	{
		ToolbarBuilder.BeginSection("ExtraSelectionModes");
		ToolbarBuilder.BeginBlockGroup();

		AddToggleButtonForBool(SelectEdgeLoopsHandle, "PolyEd.SelectEdgeLoops");
		AddToggleButtonForBool(SelectEdgeRingsHandle, "PolyEd.SelectEdgeRings");

		ToolbarBuilder.EndBlockGroup();
		ToolbarBuilder.EndSection();
	}

	// Now we're going to take one of the old checkboxes and replace it with our seleciton filter. Doing this instead
	// of creating a custom row allows us to use the original splitter, which will then align with the other splitters.
	if (FirstHandle)
	{
		IDetailPropertyRow* SelectionFilterRow = DetailBuilder.EditDefaultProperty(FirstHandle);

		const FText SelectionFilterLabel = LOCTEXT("SelectionFilterLabel", "Selection Filter");

		SelectionFilterRow->CustomWidget()
			.NameContent()
			[
				FirstHandle->CreatePropertyNameWidget(SelectionFilterLabel, SelectionFilterLabel)
			]
			.ValueContent()
			[
				SNew(SBox)
				.Padding(FMargin(0, ModelingUIConstants::DetailRowVertPadding))
				[
					ToolbarBuilder.MakeWidget()
				]
			];
	}
}

}

#undef LOCTEXT_NAMESPACE