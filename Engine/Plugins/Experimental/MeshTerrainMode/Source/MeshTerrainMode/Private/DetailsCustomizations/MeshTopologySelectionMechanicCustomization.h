// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "IDetailCustomization.h"
#include "Templates/SharedPointer.h"

class IDetailLayoutBuilder;

namespace UE::MeshTerrain
{

// Customization for UMeshTopologySelectionMechanicProperties
class FMeshTopologySelectionMechanicPropertiesDetails : public IDetailCustomization
{
public:
	static TSharedRef<IDetailCustomization> MakeInstance();
	void CustomizeDetails(IDetailLayoutBuilder& DetailBuilder) override;
};

}