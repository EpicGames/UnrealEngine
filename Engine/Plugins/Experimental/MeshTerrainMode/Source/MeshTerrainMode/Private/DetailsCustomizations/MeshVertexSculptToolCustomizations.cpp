// Copyright Epic Games, Inc. All Rights Reserved.

#include "DetailsCustomizations/MeshVertexSculptToolCustomizations.h"
#include "DetailLayoutBuilder.h"
#include "DetailWidgetRow.h"
#include "Engine/Texture2D.h"
#include "Widgets/Input/SCheckBox.h"

#include "CollectionManagerModule.h"
#include "ICollectionManager.h"

#include "ModelingWidgets/SComboPanel.h"
#include "ModelingWidgets/SToolInputAssetComboPanel.h"
#include "ModelingWidgets/ModelingCustomizationUtil.h"
#include "DetailsCustomizations/ModelingToolsBrushSizeCustomization.h"

#include "MeshTerrainModeStyle.h"
#include "MeshTerrainModeSettings.h"

#include "MeshVertexSculptTool.h"
#include "Widgets/Layout/SBox.h"
#include "Widgets/SBoxPanel.h"

using namespace UE::ModelingUI;

#define LOCTEXT_NAMESPACE "MeshVertexSculptToolCustomizations"

namespace UE::MeshTerrain
{

TSharedRef<IDetailCustomization> FSculptBrushPropertiesDetails::MakeInstance()
{
	return MakeShareable(new FSculptBrushPropertiesDetails);
}


void FSculptBrushPropertiesDetails::CustomizeDetails(IDetailLayoutBuilder& DetailBuilder)
{
	TArray<TWeakObjectPtr<UObject>> ObjectsBeingCustomized;
	DetailBuilder.GetObjectsBeingCustomized(ObjectsBeingCustomized);
	check(ObjectsBeingCustomized.Num() > 0);
	USculptBrushProperties* BrushProperties = Cast<USculptBrushProperties>(ObjectsBeingCustomized[0]);

	TSharedPtr<IPropertyHandle> FlowRateHandle = DetailBuilder.GetProperty(GET_MEMBER_NAME_CHECKED(USculptBrushProperties, FlowRate), USculptBrushProperties::StaticClass());
	ensure(FlowRateHandle->IsValidHandle());

	TSharedPtr<IPropertyHandle> SpacingHandle = DetailBuilder.GetProperty(GET_MEMBER_NAME_CHECKED(USculptBrushProperties, Spacing), USculptBrushProperties::StaticClass());
	ensure(SpacingHandle->IsValidHandle());
	SpacingHandle->MarkHiddenByCustomization();

	TSharedPtr<IPropertyHandle> FalloffHandle = DetailBuilder.GetProperty(GET_MEMBER_NAME_CHECKED(USculptBrushProperties, BrushFalloffAmount), USculptBrushProperties::StaticClass());
	ensure(FalloffHandle->IsValidHandle());
	FalloffHandle->MarkHiddenByCustomization();

	TSharedPtr<SDynamicNumericEntry::FDataSource> FlowRateSource = SDynamicNumericEntry::MakeSimpleDataSource(
		FlowRateHandle, TInterval<float>(0.0f, 1.0f), TInterval<float>(0.0f, 1.0f));

	if ( BrushProperties->bShowSpacing )
	{
		TSharedPtr<SDynamicNumericEntry::FDataSource> SpacingSource = SDynamicNumericEntry::MakeSimpleDataSource(
			SpacingHandle, TInterval<float>(0.0f, 1000.0f), TInterval<float>(0.0f, 4.0f));

		DetailBuilder.EditDefaultProperty(FlowRateHandle)->CustomWidget()
			.OverrideResetToDefault(FResetToDefaultOverride::Hide())
			.WholeRowContent()
		[
			MakeTwoWidgetDetailRowHBox(
				MakeFixedWidthLabelSliderHBox(FlowRateHandle, FlowRateSource, FSculptToolsUIConstants::SculptShortLabelWidth),
				MakeFixedWidthLabelSliderHBox(SpacingHandle, SpacingSource, FSculptToolsUIConstants::SculptShortLabelWidth)
			)
		];
	}
	else   // if bShowFalloff
	{
		TSharedPtr<SDynamicNumericEntry::FDataSource> FalloffSource = SDynamicNumericEntry::MakeSimpleDataSource(
			FalloffHandle, TInterval<float>(0.0f, 1.0f), TInterval<float>(0.0f, 1.0f));

		DetailBuilder.EditDefaultProperty(FlowRateHandle)->CustomWidget()
			.OverrideResetToDefault(FResetToDefaultOverride::Hide())
			.WholeRowContent()
			[
				MakeTwoWidgetDetailRowHBox(
					MakeFixedWidthLabelSliderHBox(FalloffHandle, FalloffSource, FSculptToolsUIConstants::SculptShortLabelWidth),
					MakeFixedWidthLabelSliderHBox(FlowRateHandle, FlowRateSource, FSculptToolsUIConstants::SculptShortLabelWidth)
				)
			];
	}

	TSharedPtr<IPropertyHandle> LazynessHandle = DetailBuilder.GetProperty(GET_MEMBER_NAME_CHECKED(USculptBrushProperties, Lazyness), USculptBrushProperties::StaticClass());
	ensure(LazynessHandle->IsValidHandle());

	TSharedPtr<IPropertyHandle> HitBackFacesHandle = DetailBuilder.GetProperty(GET_MEMBER_NAME_CHECKED(USculptBrushProperties, bHitBackFaces), USculptBrushProperties::StaticClass());
	ensure(HitBackFacesHandle->IsValidHandle());
	HitBackFacesHandle->MarkHiddenByCustomization();

	// todo: 0-100 mapping
	TSharedPtr<SDynamicNumericEntry::FDataSource> LazynessSource = SDynamicNumericEntry::MakeSimpleDataSource(
		LazynessHandle, TInterval<float>(0.0f, 1.0f), TInterval<float>(0.0f, 1.0f));

	DetailBuilder.EditDefaultProperty(LazynessHandle)->CustomWidget()
		.OverrideResetToDefault(FResetToDefaultOverride::Hide())
		.WholeRowContent()
	[
		MakeTwoWidgetDetailRowHBox(
			MakeFixedWidthLabelSliderHBox(LazynessHandle, LazynessSource, FSculptToolsUIConstants::SculptShortLabelWidth),
			MakeBoolToggleButton(HitBackFacesHandle, LOCTEXT("HitBackFacesText", "Hit Back Faces") )
		)
	];

}


TSharedRef<IDetailCustomization> FVertexBrushSculptPropertiesDetails::MakeInstance()
{
	return MakeShareable(new FVertexBrushSculptPropertiesDetails);
}

FVertexBrushSculptPropertiesDetails::~FVertexBrushSculptPropertiesDetails()
{
	if (TargetTool.IsValid())
	{
		TargetTool.Get()->OnDetailsPanelRequestRebuild.Remove(FalloffTypeUpdateHandle);
	}
}

void FVertexBrushSculptPropertiesDetails::CustomizeDetails(IDetailLayoutBuilder& DetailBuilder)
{
	TSharedPtr<IPropertyHandle> BrushTypeHandle = DetailBuilder.GetProperty(GET_MEMBER_NAME_CHECKED(UVertexBrushSculptProperties, PrimaryBrushID), UVertexBrushSculptProperties::StaticClass());
	ensure(BrushTypeHandle->IsValidHandle());

	DetailBuilder.GetObjectsBeingCustomized(ObjectsBeingCustomized);
	check(ObjectsBeingCustomized.Num() > 0);
	UVertexBrushSculptProperties* BrushProperties = CastChecked<UVertexBrushSculptProperties>(ObjectsBeingCustomized[0]);
	UMeshVertexSculptTool* Tool = BrushProperties->Tool.Get();
	TargetTool = Tool;

	const TSet<UMeshSculptToolBase::FBrushTypeInfo> BrushTypeInfos = Tool->GetAvailableBrushTypes();
	int32 CurrentBrushType = BrushProperties->PrimaryBrushID;
	int32 CurrentBrushTypeIndex = 0;

	TArray<TSharedPtr<SComboPanel::FComboPanelItem>> BrushTypeItems;
	for (const UMeshSculptToolBase::FBrushTypeInfo& BrushTypeInfo : BrushTypeInfos)
	{
		TSharedPtr<SComboPanel::FComboPanelItem> NewBrushTypeItem = MakeShared<SComboPanel::FComboPanelItem>();
		NewBrushTypeItem->Name = BrushTypeInfo.Name;
		NewBrushTypeItem->Identifier = BrushTypeInfo.Identifier;
		const FString* SourceBrushName = FTextInspector::GetSourceString(BrushTypeInfo.Name);
		NewBrushTypeItem->Icon = FMeshTerrainModeStyle::Get()->GetBrush( FName(TEXT("BrushTypeIcons.") + (*SourceBrushName) ));
		if (NewBrushTypeItem->Identifier == CurrentBrushType)
		{
			CurrentBrushTypeIndex = BrushTypeItems.Num();
		}
		BrushTypeItems.Add(NewBrushTypeItem);
	}

	float ComboIconSize = 60;
	float FlyoutIconSize = 100;
	float FlyoutWidth = 840;

	TSharedPtr<SComboPanel> BrushTypeCombo = SNew(SComboPanel)
		.ToolTipText(BrushTypeHandle->GetToolTipText())
		.ComboButtonTileSize(FVector2D(ComboIconSize, ComboIconSize))
		.FlyoutTileSize(FVector2D(FlyoutIconSize, FlyoutIconSize))
		.FlyoutSize(FVector2D(FlyoutWidth, 1))
		.ListItems(BrushTypeItems)
		.OnSelectionChanged_Lambda([this](TSharedPtr<SComboPanel::FComboPanelItem> NewSelectedItem) {
			if ( TargetTool.IsValid() )
			{
				TargetTool.Get()->SetActiveBrushType(NewSelectedItem->Identifier);
			}
		})
		.FlyoutHeaderText(LOCTEXT("BrushesHeader", "Brush Types"))
		.InitialSelectionIndex(CurrentBrushTypeIndex);


	TSharedPtr<IPropertyHandle> FalloffTypeHandle = DetailBuilder.GetProperty(GET_MEMBER_NAME_CHECKED(UVertexBrushSculptProperties, PrimaryFalloffType), UVertexBrushSculptProperties::StaticClass());
	ensure(FalloffTypeHandle->IsValidHandle());

	const TArray<UMeshSculptToolBase::FFalloffTypeInfo>& FalloffTypeInfos = Tool->GetRegisteredPrimaryFalloffTypes();
	int32 CurrentFalloffType = (int32)BrushProperties->PrimaryFalloffType;
	int32 CurrentFalloffTypeIndex = 0;

	TArray<TSharedPtr<SComboPanel::FComboPanelItem>> FalloffTypeItems;
	for (const UMeshSculptToolBase::FFalloffTypeInfo& FalloffTypeInfo : FalloffTypeInfos)
	{
		TSharedPtr<SComboPanel::FComboPanelItem> NewFalloffTypeItem = MakeShared<SComboPanel::FComboPanelItem>();
		NewFalloffTypeItem->Name = FalloffTypeInfo.Name;
		NewFalloffTypeItem->Identifier = FalloffTypeInfo.Identifier;
		NewFalloffTypeItem->Icon = FMeshTerrainModeStyle::Get()->GetBrush( FName(TEXT("BrushFalloffIcons.") + FalloffTypeInfo.StringIdentifier) );
		if (NewFalloffTypeItem->Identifier == CurrentFalloffType)
		{
			CurrentFalloffTypeIndex = FalloffTypeItems.Num();
		}
		FalloffTypeItems.Add(NewFalloffTypeItem);
	}

	FalloffTypeCombo = SNew(SComboPanel)		
		.ToolTipText(FalloffTypeHandle->GetToolTipText())
		.ComboButtonTileSize(FVector2D(18, 18))
		.FlyoutTileSize(FVector2D(FlyoutIconSize, FlyoutIconSize))
		.FlyoutSize(FVector2D(FlyoutWidth, 1))
		.ListItems(FalloffTypeItems)
		.ComboDisplayType(SComboPanel::EComboDisplayType::IconAndLabel)
		.OnSelectionChanged_Lambda([this](TSharedPtr<SComboPanel::FComboPanelItem> NewSelectedFalloffTypeItem) {
			if ( TargetTool.IsValid() )
			{
				TargetTool.Get()->SetActiveFalloffType(NewSelectedFalloffTypeItem->Identifier);
			}
		})
		.FlyoutHeaderText(LOCTEXT("FalloffsHeader", "Falloff Types"))
		.InitialSelectionIndex(CurrentFalloffTypeIndex);

	FalloffTypeUpdateHandle = TargetTool.Get()->OnDetailsPanelRequestRebuild.AddLambda([this]() {
			UVertexBrushSculptProperties* BrushProperties = CastChecked<UVertexBrushSculptProperties>(ObjectsBeingCustomized[0]);
			const TArray<UMeshSculptToolBase::FFalloffTypeInfo>& FalloffTypeInfos = TargetTool->GetRegisteredPrimaryFalloffTypes();
			int32 CurrentFalloffType = (int32)BrushProperties->PrimaryFalloffType;
			FalloffTypeCombo->SetSelectionIndex(CurrentFalloffType);
	});

	//DetailBuilder.HideProperty(FalloffTypeHandle);
	FalloffTypeHandle->MarkHiddenByCustomization();


	TSharedPtr<IPropertyHandle> BrushFilterHandle = DetailBuilder.GetProperty(GET_MEMBER_NAME_CHECKED(UVertexBrushSculptProperties, BrushFilter), UVertexBrushSculptProperties::StaticClass());
	BrushFilterHandle->MarkHiddenByCustomization();

	TSharedPtr<IPropertyHandle> FreezeTargetHandle = DetailBuilder.GetProperty(GET_MEMBER_NAME_CHECKED(UVertexBrushSculptProperties, bFreezeTarget), UVertexBrushSculptProperties::StaticClass());
	FreezeTargetHandle->MarkHiddenByCustomization();


	DetailBuilder.EditDefaultProperty(BrushTypeHandle)->CustomWidget()
		.OverrideResetToDefault(FResetToDefaultOverride::Hide())
		.WholeRowContent()
		[
			SNew(SHorizontalBox)
			+ SHorizontalBox::Slot()
			.Padding(FMargin(0, ModelingUIConstants::DetailRowVertPadding))
			.AutoWidth()
			[
				SNew(SBox)
				.HeightOverride(ComboIconSize+14)
				[
					BrushTypeCombo->AsShared()
				]
			]
			+ SHorizontalBox::Slot()
			.Padding(FMargin(ModelingUIConstants::MultiWidgetRowHorzPadding,ModelingUIConstants::DetailRowVertPadding,0,ModelingUIConstants::MultiWidgetRowHorzPadding))
			.FillWidth(1.0f)
			[
				SNew(SVerticalBox)
					+ SVerticalBox::Slot()
					.Padding(FMargin(0))
					.AutoHeight()
					[
						SNew(SHorizontalBox)
							+ SHorizontalBox::Slot()
							.AutoWidth()
							[
								WrapInFixedWidthBox(FalloffTypeHandle->CreatePropertyNameWidget(), FSculptToolsUIConstants::SculptShortLabelWidth)
							]
							+ SHorizontalBox::Slot()
							.FillWidth(1.0f)
							[
								FalloffTypeCombo->AsShared()
							]
					]

					+ SVerticalBox::Slot()
					.Padding(FMargin(0, ModelingUIConstants::DetailRowVertPadding, 0, 0))
					.AutoHeight()
					[
						SNew(SHorizontalBox)
						+ SHorizontalBox::Slot()
						.AutoWidth()
						[
							WrapInFixedWidthBox(BrushFilterHandle->CreatePropertyNameWidget(), FSculptToolsUIConstants::SculptShortLabelWidth)
						]
						+ SHorizontalBox::Slot()
						.FillWidth(1.0f)
						[
							MakeRegionFilterWidget()->AsShared()
						]
					]

					+ SVerticalBox::Slot()
					.Padding(FMargin(0, ModelingUIConstants::DetailRowVertPadding, 0, 0))
					.AutoHeight()
					[
						MakeFreezeTargetWidget()->AsShared()
					]
			]
		];

}


TSharedPtr<SWidget> FVertexBrushSculptPropertiesDetails::MakeRegionFilterWidget()
{
	static const FText RegionFilterLabels[] = {
		LOCTEXT("RegionFilterNone", "Vol"),  
		LOCTEXT("RegionFilterComponent", "Cmp"),  
		LOCTEXT("RegionFilterPolyGroup", "Grp")
	};
	static const FText RegionFilterTooltips[] = {
		LOCTEXT("RegionFilterNoneTooltip", "Do not filter brush area, include all triangles in brush sphere"),
		LOCTEXT("RegionFilterComponentTooltip", "Only apply brush to triangles in the same connected mesh component/island"),
		LOCTEXT("RegionFilterPolygroupTooltip", "Only apply brush to triangles with the same PolyGroup"),
	};

	auto MakeRegionFilterButton = [this](EMeshVertexSculptBrushFilterType FilterType)
	{
		return SNew(SCheckBox)
			.Style(FAppStyle::Get(), "DetailsView.SectionButton")
			.Padding(FMargin(2, 2))
			.HAlign(HAlign_Center)
			.ToolTipText( RegionFilterTooltips[(int32)FilterType] )
			.OnCheckStateChanged_Lambda([this, FilterType](ECheckBoxState State) {
				if (TargetTool.IsValid() && State == ECheckBoxState::Checked)
				{
					TargetTool.Get()->SetRegionFilterType( static_cast<int32>(FilterType) );
				}
			})
			.IsChecked_Lambda([this, FilterType]() {
				return (TargetTool.IsValid() && TargetTool.Get()->SculptProperties->BrushFilter == FilterType) ? ECheckBoxState::Checked : ECheckBoxState::Unchecked;
			})
			[
				SNew(SHorizontalBox)
				+ SHorizontalBox::Slot()
				.VAlign(VAlign_Center)
				.Padding(FMargin(0))
				.AutoWidth()
				[
					SNew(STextBlock)
					.Justification(ETextJustify::Center)
					.TextStyle(FAppStyle::Get(), "DetailsView.CategoryTextStyle")
					.Text( RegionFilterLabels[(int32)FilterType] )
				]
			];
	};


	return SNew(SHorizontalBox)
		+ SHorizontalBox::Slot()
		.FillWidth(1.0f)
		[
			MakeRegionFilterButton(EMeshVertexSculptBrushFilterType::None)
		]
		+ SHorizontalBox::Slot()
		.FillWidth(1.0f)
		[
			MakeRegionFilterButton(EMeshVertexSculptBrushFilterType::Component)
		]
		+ SHorizontalBox::Slot()
		.FillWidth(1.0f)
		[
			MakeRegionFilterButton(EMeshVertexSculptBrushFilterType::PolyGroup)
		];
}


TSharedPtr<SWidget> FVertexBrushSculptPropertiesDetails::MakeFreezeTargetWidget()
{
	return SNew(SCheckBox)
		.Style(FAppStyle::Get(), "DetailsView.SectionButton")
		.Padding(FMargin(0, 4))
		.ToolTipText(LOCTEXT("FreezeTargetTooltip", "When Freeze Target is toggled on, the Brush Target Surface will be Frozen in its current state, until toggled off. Brush strokes will be applied relative to the Target Surface, for applicable Brushes"))
		.HAlign(HAlign_Center)
		.OnCheckStateChanged(this, &FVertexBrushSculptPropertiesDetails::OnSetFreezeTarget)
		.IsChecked_Lambda([this] { return IsFreezeTargetEnabled() ? ECheckBoxState::Checked : ECheckBoxState::Unchecked; } )
		[
			SNew(SHorizontalBox)
			+ SHorizontalBox::Slot()
			.VAlign(VAlign_Center)
			.Padding(FMargin(0))
			.AutoWidth()
			[
				SNew(STextBlock)
				.Justification(ETextJustify::Center)
				.TextStyle(FAppStyle::Get(), "DetailsView.CategoryTextStyle")
				.Text_Lambda( [this] { return IsFreezeTargetEnabled() ? LOCTEXT("UnFreezeTarget", "UnFreeze Target") : LOCTEXT("FreezeTarget", "Freeze Target"); } )
			]
		];
}




FReply FVertexBrushSculptPropertiesDetails::OnToggledFreezeTarget()
{
	if ( TargetTool.IsValid() )
	{
		TargetTool.Get()->SculptProperties->bFreezeTarget = !TargetTool.Get()->SculptProperties->bFreezeTarget;
	}
	return FReply::Handled();
}

void FVertexBrushSculptPropertiesDetails::OnSetFreezeTarget(ECheckBoxState State)
{
	if ( TargetTool.IsValid() )
	{
		TargetTool.Get()->SculptProperties->bFreezeTarget = (State == ECheckBoxState::Checked);
	}
}

bool FVertexBrushSculptPropertiesDetails::IsFreezeTargetEnabled()
{
	return (TargetTool.IsValid()) ? TargetTool.Get()->SculptProperties->bFreezeTarget : false;
}









class FRecentAlphasProvider : public SToolInputAssetComboPanel::IRecentAssetsProvider
{
public:
	TArray<FAssetData> RecentAssets;

	virtual TArray<FAssetData> GetRecentAssetsList() override
	{
		return RecentAssets;
	}
	virtual void NotifyNewAsset(const FAssetData& NewAsset) override
	{
		if (NewAsset.GetAsset() == nullptr)
		{
			return;
		}
		for (int32 k = 0; k < RecentAssets.Num(); ++k)
		{
			if (RecentAssets[k] == NewAsset)
			{
				if (k == 0)
				{
					return;
				}
				RecentAssets.RemoveAt(k, EAllowShrinking::No);
			}
		}
		RecentAssets.Insert(NewAsset, 0);
		
		if (RecentAssets.Num() > 10)
		{
			RecentAssets.SetNum(10, EAllowShrinking::No);
		}
	}
};



TSharedRef<IDetailCustomization> FVertexBrushAlphaPropertiesDetails::MakeInstance()
{
	return MakeShareable(new FVertexBrushAlphaPropertiesDetails);
}

FVertexBrushAlphaPropertiesDetails::~FVertexBrushAlphaPropertiesDetails()
{
	if (TargetTool.IsValid())
	{
		TargetTool.Get()->OnDetailsPanelRequestRebuild.Remove(AlphaTextureUpdateHandle);
	}
}

void FVertexBrushAlphaPropertiesDetails::CustomizeDetails(IDetailLayoutBuilder& DetailBuilder)
{
	// TODO: move this to a subsystem or UObject CDO
	struct FRecentAlphasContainer
	{
		TSharedPtr<FRecentAlphasProvider> RecentAlphas;
	};
	static FRecentAlphasContainer RecentAlphasStatic;
	if (RecentAlphasStatic.RecentAlphas.IsValid() == false)
	{
		RecentAlphasStatic.RecentAlphas = MakeShared<FRecentAlphasProvider>();
	}
	RecentAlphasProvider = RecentAlphasStatic.RecentAlphas;

	DetailBuilder.GetObjectsBeingCustomized(ObjectsBeingCustomized);
	check(ObjectsBeingCustomized.Num() > 0);
	UVertexBrushAlphaProperties* AlphaProperties = CastChecked<UVertexBrushAlphaProperties>(ObjectsBeingCustomized[0]);
	UMeshVertexSculptTool* Tool = AlphaProperties->Tool.Get();
	TargetTool = Tool;


	TSharedPtr<IPropertyHandle> AlphaHandle = DetailBuilder.GetProperty(GET_MEMBER_NAME_CHECKED(UVertexBrushAlphaProperties, Alpha), UVertexBrushAlphaProperties::StaticClass());
	ensure(AlphaHandle->IsValidHandle());

	TSharedPtr<IPropertyHandle> RotationAngleHandle = DetailBuilder.GetProperty(GET_MEMBER_NAME_CHECKED(UVertexBrushAlphaProperties, RotationAngle), UVertexBrushAlphaProperties::StaticClass());
	ensure(RotationAngleHandle->IsValidHandle());
	RotationAngleHandle->MarkHiddenByCustomization();

	TSharedPtr<IPropertyHandle> bRandomizeHandle = DetailBuilder.GetProperty(GET_MEMBER_NAME_CHECKED(UVertexBrushAlphaProperties, bRandomize), UVertexBrushAlphaProperties::StaticClass());
	ensure(bRandomizeHandle->IsValidHandle());
	bRandomizeHandle->MarkHiddenByCustomization();

	TSharedPtr<IPropertyHandle> RandomRangeHandle = DetailBuilder.GetProperty(GET_MEMBER_NAME_CHECKED(UVertexBrushAlphaProperties, RandomRange), UVertexBrushAlphaProperties::StaticClass());
	ensure(RandomRangeHandle->IsValidHandle());
	RandomRangeHandle->MarkHiddenByCustomization();

	TSharedPtr<SDynamicNumericEntry::FDataSource> RotationAngleSource = SDynamicNumericEntry::MakeSimpleDataSource(
		RotationAngleHandle, TInterval<float>(-180.0f, 180.0f), TInterval<float>(-180.0f, 180.0f));
	TSharedPtr<SDynamicNumericEntry::FDataSource> RandomRangeSource = SDynamicNumericEntry::MakeSimpleDataSource(
		RandomRangeHandle, TInterval<float>(0.0f, 180.0f), TInterval<float>(0.0f, 180.0f));

	float ComboIconSize = 60;

	UMeshTerrainModeCustomizationSettings* UISettings = GetMutableDefault<UMeshTerrainModeCustomizationSettings>();
	TArray<SToolInputAssetComboPanel::FNamedCollectionList> BrushAlphasLists;
	for (const FMeshTerrainModeAssetCollectionSet& AlphasCollectionSet : UISettings->BrushAlphaSets)
	{
		SToolInputAssetComboPanel::FNamedCollectionList CollectionSet;
		CollectionSet.Name = AlphasCollectionSet.Name;
		for (FCollectionReference CollectionRef : AlphasCollectionSet.Collections)
		{
			CollectionSet.Collections.Emplace(
				FCollectionManagerModule::GetModule().Get().GetProjectCollectionContainer(),
				CollectionRef.CollectionName,
				ECollectionShareType::CST_Local);
		}
		BrushAlphasLists.Add(CollectionSet);
	}

	AlphaAssetPicker = SNew(SToolInputAssetComboPanel)
		.AssetClassType(UTexture2D::StaticClass())		// can infer from property...
		.Property(AlphaHandle)
		.ComboButtonTileSize(FVector2D(ComboIconSize, ComboIconSize))
		.FlyoutTileSize(FVector2D(80, 80))
		.FlyoutSize(FVector2D(1000, 600))
		.RecentAssetsProvider(RecentAlphasProvider)
		.CollectionSets(BrushAlphasLists);

	AlphaTextureUpdateHandle = TargetTool.Get()->OnDetailsPanelRequestRebuild.AddLambda([this]() {
		AlphaAssetPicker->RefreshThumbnailFromProperty();
	});

	DetailBuilder.EditDefaultProperty(AlphaHandle)->CustomWidget()
		.OverrideResetToDefault(FResetToDefaultOverride::Hide())
		.WholeRowContent()
		[
			SNew(SHorizontalBox)
			+ SHorizontalBox::Slot()
			.Padding(FMargin(0, ModelingUIConstants::DetailRowVertPadding))
			.AutoWidth()
			[
				SNew(SBox)
				.HeightOverride(ComboIconSize+14)
				[
					AlphaAssetPicker->AsShared()
				]
			]
			+ SHorizontalBox::Slot()
			.Padding(FMargin(ModelingUIConstants::MultiWidgetRowHorzPadding,ModelingUIConstants::DetailRowVertPadding,0,ModelingUIConstants::MultiWidgetRowHorzPadding))
			.FillWidth(1.0f)
			[
				SNew(SVerticalBox)
					+ SVerticalBox::Slot()
					.Padding(FMargin(0))
					.AutoHeight()
					[
						MakeFixedWidthLabelSliderHBox(RotationAngleHandle, RotationAngleSource, FSculptToolsUIConstants::SculptShortLabelWidth)
					]

					+ SVerticalBox::Slot()
					.Padding(FMargin(0, ModelingUIConstants::DetailRowVertPadding))
					.AutoHeight()
					[
						MakeToggleSliderHBox(bRandomizeHandle, LOCTEXT("RandomizeLabel", "Rand"), RandomRangeSource, FSculptToolsUIConstants::SculptShortLabelWidth)
					]
			]
		];
	


}

}





#undef LOCTEXT_NAMESPACE
