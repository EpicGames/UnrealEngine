// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "IDetailCustomization.h"
#include "UObject/WeakObjectPtr.h"

class FReply;
class SButton;
class SWidget;
enum class ECheckBoxState : uint8;

class IDetailLayoutBuilder;
class IPropertyHandle;
class IDetailChildrenBuilder;
class SComboButton;
class SComboPanel;
class SBox;
class SToolInputAssetComboPanel;
class UMeshVertexSculptTool;

namespace UE::MeshTerrain
{
class FRecentAlphasProvider;


// customization for USculptBrushProperties, creates two-column layout
// for secondary brush properties like lazy/etc
class FSculptBrushPropertiesDetails : public IDetailCustomization
{
public:
	static TSharedRef<IDetailCustomization> MakeInstance();
	void CustomizeDetails(IDetailLayoutBuilder& DetailBuilder) override;
};


// customization for vertexsculpt properties, creates combopanel for brush type,
// small-style combopanel for falloff type, and stacks controls to the right
class FVertexBrushSculptPropertiesDetails : public IDetailCustomization
{
public:
	virtual ~FVertexBrushSculptPropertiesDetails();

	static TSharedRef<IDetailCustomization> MakeInstance();
	void CustomizeDetails(IDetailLayoutBuilder& DetailBuilder) override;

protected:
	TWeakObjectPtr<UMeshVertexSculptTool> TargetTool;

	TSharedPtr<SWidget> MakeRegionFilterWidget();
	TSharedPtr<SWidget> MakeFreezeTargetWidget();
	
	TSharedPtr<SButton> FreezeTargetButton;
	FReply OnToggledFreezeTarget();
	void OnSetFreezeTarget(ECheckBoxState State);
	bool IsFreezeTargetEnabled();

	TSharedPtr<SComboPanel> FalloffTypeCombo;
	TArray<TWeakObjectPtr<UObject>> ObjectsBeingCustomized;
	FDelegateHandle FalloffTypeUpdateHandle;
};



// customization for UVertexBrushAlphaProperties, creates custom asset picker
// tileview-combopanel for brush alphas and stacks controls to the right
class FVertexBrushAlphaPropertiesDetails : public IDetailCustomization
{
public:
	virtual ~FVertexBrushAlphaPropertiesDetails();

	static TSharedRef<IDetailCustomization> MakeInstance();
	void CustomizeDetails(IDetailLayoutBuilder& DetailBuilder) override;

	TSharedPtr<FRecentAlphasProvider> RecentAlphasProvider;

protected:
	TWeakObjectPtr<UMeshVertexSculptTool> TargetTool;
	TArray<TWeakObjectPtr<UObject>> ObjectsBeingCustomized;
	TSharedPtr<SToolInputAssetComboPanel> AlphaAssetPicker;
	FDelegateHandle AlphaTextureUpdateHandle;
};

}
