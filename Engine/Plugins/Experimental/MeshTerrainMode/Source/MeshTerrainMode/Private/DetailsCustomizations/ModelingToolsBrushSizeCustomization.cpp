// Copyright Epic Games, Inc. All Rights Reserved.

#include "DetailsCustomizations/ModelingToolsBrushSizeCustomization.h"

#include "DetailWidgetRow.h"
#include "InteractiveTool.h"
#include "PropertyHandle.h"
#include "Widgets/Input/SCheckBox.h"
#include "Widgets/Layout/SBox.h"
#include "Widgets/MeshTerrainModeUtil.h"
#include "ModelingWidgets/ModelingCustomizationUtil.h"

#include "Sculpting/MeshSculptToolBase.h"
#include "Widgets/SBoxPanel.h"

using namespace UE::ModelingUI;

#define LOCTEXT_NAMESPACE "ModelingToolsBrushSizeCustomization"

namespace UE::MeshTerrain
{


TSharedRef<IPropertyTypeCustomization> FModelingToolsBrushSizeCustomization::MakeInstance()
{
	return MakeShareable(new FModelingToolsBrushSizeCustomization);
}

void FModelingToolsBrushSizeCustomization::CustomizeHeader(TSharedRef<IPropertyHandle> StructPropertyHandle, FDetailWidgetRow& HeaderRow, IPropertyTypeCustomizationUtils& StructCustomizationUtils)
{
	TSharedPtr<IPropertyHandle> SizeType = StructPropertyHandle->GetChildHandle(GET_MEMBER_NAME_CHECKED(FBrushToolRadius, SizeType));
	SizeType->MarkHiddenByCustomization();

	TSharedPtr<IPropertyHandle> AdaptiveSize = StructPropertyHandle->GetChildHandle(GET_MEMBER_NAME_CHECKED(FBrushToolRadius, AdaptiveSize));
	AdaptiveSize->MarkHiddenByCustomization();

	TSharedPtr<IPropertyHandle> WorldRadius = StructPropertyHandle->GetChildHandle(GET_MEMBER_NAME_CHECKED(FBrushToolRadius, WorldRadius));
	WorldRadius->MarkHiddenByCustomization();

	// Identify the tool that owns this BrushSize struct so the slider range can be tool-specific.
	// StructPropertyHandle's outer object is the property set (e.g. USculptBrushProperties); its
	// outer is the tool.
	TArray<UObject*> OuterObjects;
	StructPropertyHandle->GetOuterObjects(OuterObjects);
	const UInteractiveTool* OwningTool = (OuterObjects.Num() > 0 && OuterObjects[0])
		? Cast<UInteractiveTool>(OuterObjects[0]->GetOuter())
		: nullptr;

	auto GetCurrentSizeType = [SizeType]() -> EBrushToolSizeType
	{
		uint8 SizeTypeInt = 0;
		SizeType->GetValue(SizeTypeInt);
		return (SizeTypeInt == (uint8)EBrushToolSizeType::World) ? EBrushToolSizeType::World : EBrushToolSizeType::Adaptive;
	};
	auto GetSizeHandle = [SizeType, AdaptiveSize, WorldRadius]() -> TSharedPtr<IPropertyHandle>
	{
		uint8 SizeTypeInt = 0;
		SizeType->GetValue(SizeTypeInt);
		return (SizeTypeInt == (uint8)EBrushToolSizeType::World) ? WorldRadius : AdaptiveSize;
	};

	TSharedPtr<SDynamicNumericEntry::FDataSource> NumericSource = MakeShared<SDynamicNumericEntry::FDataSource>();
	NumericSource->SetValue = [GetSizeHandle](float NewSize, EPropertyValueSetFlags::Type Flags)
	{
		GetSizeHandle()->SetValue(NewSize, Flags);
	};
	NumericSource->GetValue = [GetSizeHandle]() -> float
	{
		float Size;
		GetSizeHandle()->GetValue(Size);
		return Size;
	};
	NumericSource->GetValueRange = [GetCurrentSizeType, OwningTool]() -> TInterval<float>
	{
		TInterval<float> UI, Value;
		UE::MeshTerrain::GetSculptBrushSizeRange(OwningTool, GetCurrentSizeType(), UI, Value);
		return Value;
	};
	NumericSource->GetUIRange = [GetCurrentSizeType, OwningTool]() -> TInterval<float>
	{
		TInterval<float> UI, Value;
		UE::MeshTerrain::GetSculptBrushSizeRange(OwningTool, GetCurrentSizeType(), UI, Value);
		return UI;
	};

	HeaderRow
		.OverrideResetToDefault(FResetToDefaultOverride::Hide())
		.WholeRowContent()
	[
		SNew(SHorizontalBox)

		// name
		+ SHorizontalBox::Slot()
		.AutoWidth()
		.Padding(FMargin(0,4))
		[
			WrapInFixedWidthBox(AdaptiveSize->CreatePropertyNameWidget(), FSculptToolsUIConstants::SculptShortLabelWidth)
		]

		// slider
		+ SHorizontalBox::Slot()
		.Padding(ModelingUIConstants::LabelWidgetMinPadding, 0,0,0)
		.FillWidth(10.0f)
		[
			SNew(SDynamicNumericEntry)
			.Source(NumericSource)
		]

		// toggle
		+ SHorizontalBox::Slot()
		.AutoWidth()
		//.FillWidth(0.5f)
		.Padding(FMargin(ModelingUIConstants::MultiWidgetRowHorzPadding, ModelingUIConstants::DetailRowVertPadding, 0, ModelingUIConstants::DetailRowVertPadding))
		.HAlign(HAlign_Right)
		[
			SNew(SBox)
			.WidthOverride(50)
			[
				SNew(SCheckBox)
					.Style(FAppStyle::Get(), "DetailsView.SectionButton")
					.Padding(FMargin(4, 2))
					.HAlign(HAlign_Center)
					.ToolTipText( LOCTEXT("WorldToggleTooltip", "Specify Brush Size in World Units") )
					.OnCheckStateChanged_Lambda([this, SizeType](ECheckBoxState State) {
						SizeType->SetValue( static_cast<uint8>( (State == ECheckBoxState::Checked) ? EBrushToolSizeType::World : EBrushToolSizeType::Adaptive ) );
					})
					.IsChecked_Lambda([this, SizeType]() {
						uint8 CurValueInt;
						SizeType->GetValue(CurValueInt);
						return (CurValueInt == static_cast<uint8>(EBrushToolSizeType::World)) ? ECheckBoxState::Checked : ECheckBoxState::Unchecked;
					})
					[
						SNew(SHorizontalBox)
						+ SHorizontalBox::Slot()
						.VAlign(VAlign_Center)
						.Padding(FMargin(0))
						.AutoWidth()
						[
							SNew(STextBlock)
							.Justification(ETextJustify::Center)
							.TextStyle(FAppStyle::Get(), "DetailsView.CategoryTextStyle")
							.Text( LOCTEXT("World", "World") )
						]
					]
			]
		]
	];

}


void FModelingToolsBrushSizeCustomization::CustomizeChildren(TSharedRef<IPropertyHandle> StructPropertyHandle, class IDetailChildrenBuilder& ChildBuilder, IPropertyTypeCustomizationUtils& StructCustomizationUtils)
{
}

}

#undef LOCTEXT_NAMESPACE
