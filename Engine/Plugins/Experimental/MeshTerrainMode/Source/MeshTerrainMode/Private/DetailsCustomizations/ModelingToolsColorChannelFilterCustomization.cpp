// Copyright Epic Games, Inc. All Rights Reserved.

#include "DetailsCustomizations/ModelingToolPropertyCustomizations.h"

#include "DetailWidgetRow.h"

#include "PropertySets/ColorChannelFilterPropertyType.h"
#include "Widgets/SBoxPanel.h"


#define LOCTEXT_NAMESPACE "ModelingToolsColorChannelFilterCustomization"

namespace UE::MeshTerrain
{

TSharedRef<IPropertyTypeCustomization> FModelingToolsColorChannelFilterCustomization::MakeInstance()
{
	return MakeShareable(new FModelingToolsColorChannelFilterCustomization);
}

void FModelingToolsColorChannelFilterCustomization::CustomizeHeader(TSharedRef<IPropertyHandle> StructPropertyHandle, FDetailWidgetRow& HeaderRow, IPropertyTypeCustomizationUtils& StructCustomizationUtils)
{
	const float XYZPadding = 10.f;

	TSharedPtr<IPropertyHandle> bColorChannelR = StructPropertyHandle->GetChildHandle(GET_MEMBER_NAME_CHECKED(FModelingToolsColorChannelFilter, bRed));
	bColorChannelR->MarkHiddenByCustomization();

	TSharedPtr<IPropertyHandle> bColorChannelG = StructPropertyHandle->GetChildHandle(GET_MEMBER_NAME_CHECKED(FModelingToolsColorChannelFilter, bGreen));
	bColorChannelG->MarkHiddenByCustomization();

	TSharedPtr<IPropertyHandle> bColorChannelB = StructPropertyHandle->GetChildHandle(GET_MEMBER_NAME_CHECKED(FModelingToolsColorChannelFilter, bBlue));
	bColorChannelB->MarkHiddenByCustomization();

	TSharedPtr<IPropertyHandle> bColorChannelA = StructPropertyHandle->GetChildHandle(GET_MEMBER_NAME_CHECKED(FModelingToolsColorChannelFilter, bAlpha));
	bColorChannelA->MarkHiddenByCustomization();

	HeaderRow.NameContent()
		[
			StructPropertyHandle->CreatePropertyNameWidget()
		]
	.ValueContent()
		.MinDesiredWidth(125.f)
		[
			SNew(SHorizontalBox)
			+ SHorizontalBox::Slot()
			.Padding(0.f, 0.f, XYZPadding, 0.f)
			.AutoWidth()
			[
				SNew(SHorizontalBox)
				+ SHorizontalBox::Slot()
				.AutoWidth()
				[
					bColorChannelR->CreatePropertyNameWidget()
				]
				+ SHorizontalBox::Slot()
				.AutoWidth()
				[
					bColorChannelR->CreatePropertyValueWidget()
				]
			]

			+ SHorizontalBox::Slot()
			.Padding(0.f, 0.f, XYZPadding, 0.f)
			.AutoWidth()
			[
				SNew(SHorizontalBox)
				+ SHorizontalBox::Slot()
				.AutoWidth()
				[
					bColorChannelG->CreatePropertyNameWidget()
				]
				+ SHorizontalBox::Slot()
				.AutoWidth()
				[
					bColorChannelG->CreatePropertyValueWidget()
				]
			]

			+ SHorizontalBox::Slot()
			.Padding(0.f, 0.f, XYZPadding, 0.f)
			.AutoWidth()
			[
				SNew(SHorizontalBox)
				+ SHorizontalBox::Slot()
				.AutoWidth()
				[
					bColorChannelB->CreatePropertyNameWidget()
				]
				+ SHorizontalBox::Slot()
				.AutoWidth()
				[
					bColorChannelB->CreatePropertyValueWidget()
				]
			]

			+ SHorizontalBox::Slot()
			.Padding(0.f, 0.f, XYZPadding, 0.f)
			.AutoWidth()
			[
				SNew(SHorizontalBox)
				+ SHorizontalBox::Slot()
				.AutoWidth()
				[
					bColorChannelA->CreatePropertyNameWidget()
				]
				+ SHorizontalBox::Slot()
				.AutoWidth()
				[
					bColorChannelA->CreatePropertyValueWidget()
				]
			]
		];
}

void FModelingToolsColorChannelFilterCustomization::CustomizeChildren(TSharedRef<IPropertyHandle> StructPropertyHandle, class IDetailChildrenBuilder& ChildBuilder, IPropertyTypeCustomizationUtils& StructCustomizationUtils)
{
}

}

#undef LOCTEXT_NAMESPACE
