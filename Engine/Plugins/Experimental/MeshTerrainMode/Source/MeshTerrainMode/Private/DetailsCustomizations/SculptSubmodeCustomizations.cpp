// Copyright Epic Games, Inc. All Rights Reserved.

#include "SculptSubmodeCustomizations.h"

#include "MeshTerrainModeStyle.h"
#include "Widgets/Layout/SBox.h"
#include "ModelingWidgets/SDynamicNumericEntry.h"
#include "Sculpting/MeshSculptToolBase.h"
#include "Widgets/SWidget.h"

#define LOCTEXT_NAMESPACE "SculptSubmodeCustomizations"

using namespace UE::MeshTerrain;

TSharedRef<IMeshTerrainPropertyCustomization> FSculptBrushPropertiesCustomizations::MakeInstance()
{
	return MakeShareable(new FSculptBrushPropertiesCustomizations);
}

TMap<FName, FMeshTerrainCustomizationData> FSculptBrushPropertiesCustomizations::GetCustomizationData()
{
	TMap<FName, FMeshTerrainCustomizationData> Customizations;

	const FMeshTerrainCustomizationData CustomizationData =
			FMeshTerrainCustomizationData(FGetMeshTerrainDetailCustomization::CreateStatic(&FSculptBrushPropertiesCustomizations::MakeBrushSizeWidget));
	Customizations.Add("BrushSize", CustomizationData);
	
	return Customizations;
}

TMap<FName, FMeshTerrainEditConditionData> FSculptBrushPropertiesCustomizations::GetEditConditionData()
{
	TMap<FName, FMeshTerrainEditConditionData> EditConditions;
	
	const FMeshTerrainEditConditionData BrushFalloffAmountEditCondition =
		FMeshTerrainEditConditionData(true, FGetMeshTerrainEditCondition::CreateLambda([](FProperty* Property, UObject* PropListOwner)
			{
				const USculptBrushProperties* PropertySet = Cast<USculptBrushProperties>(PropListOwner);
				return PropertySet->bShowFalloff;
			}));
	EditConditions.Add("BrushFalloffAmount", BrushFalloffAmountEditCondition);
		
	const FMeshTerrainEditConditionData DepthEditCondition =
		FMeshTerrainEditConditionData(true, FGetMeshTerrainEditCondition::CreateLambda([](FProperty* Property, UObject* PropListOwner)
			{
				const USculptBrushProperties* PropertySet = Cast<USculptBrushProperties>(PropListOwner);
				return PropertySet->bShowPerBrushProps;
			}));
	EditConditions.Add("Depth", DepthEditCondition);
		
	const FMeshTerrainEditConditionData FlowRateEditCondition =
		FMeshTerrainEditConditionData(true, FGetMeshTerrainEditCondition::CreateLambda([](FProperty* Property, UObject* PropListOwner)
			{
				const USculptBrushProperties* PropertySet = Cast<USculptBrushProperties>(PropListOwner);
				return PropertySet->bShowFlowRate && PropertySet->StrokeType == EMeshSculptStrokeType::Airbrush;
			}));
	EditConditions.Add("FlowRate", FlowRateEditCondition);
		
	const FMeshTerrainEditConditionData SpacingEditCondition =
		FMeshTerrainEditConditionData(true, FGetMeshTerrainEditCondition::CreateLambda([](FProperty* Property, UObject* PropListOwner)
			{
				const USculptBrushProperties* PropertySet = Cast<USculptBrushProperties>(PropListOwner);
				return PropertySet->bShowSpacing && PropertySet->StrokeType == EMeshSculptStrokeType::Spacing;
			}));
	EditConditions.Add("Spacing", SpacingEditCondition);
		
	const FMeshTerrainEditConditionData LazynessEditCondition =
		FMeshTerrainEditConditionData(true, FGetMeshTerrainEditCondition::CreateLambda([](FProperty* Property, UObject* PropListOwner)
			{
				const USculptBrushProperties* PropertySet = Cast<USculptBrushProperties>(PropListOwner);
				return PropertySet->bShowLazyness;
			}));
	EditConditions.Add("Lazyness", LazynessEditCondition);
	
	return EditConditions;
}

TSharedRef<SWidget> FSculptBrushPropertiesCustomizations::MakeBrushSizeWidget(FProperty* BrushSizeProperty, UObject* PropListOwner)
{
	// retrieves the current BrushSize property (represented as FBrushToolRadius
	TFunction<FBrushToolRadius()> GetCurBrushSizeProp = [BrushSizeProperty, PropListOwner]()
	{
		FBrushToolRadius PropValue;
		BrushSizeProperty->GetValue_InContainer(PropListOwner, &PropValue);
		return PropValue;
	};

	// retrieves the current size type (adaptive or world) of the BrushSize property
	TFunction<EBrushToolSizeType()> GetCurrentSizeType = [GetCurBrushSizeProp]()
	{
		const FBrushToolRadius PropValue = GetCurBrushSizeProp();
		return (PropValue.SizeType == EBrushToolSizeType::World) ? EBrushToolSizeType::World : EBrushToolSizeType::Adaptive;
	};

	// toggles the size type of the BrushSize property between Adaptive and World
	TFunction<void(bool)> ToggleCurrentSizeType = [BrushSizeProperty, PropListOwner, GetCurBrushSizeProp] (bool bIsChecked)
	{
		const EBrushToolSizeType NewType = bIsChecked ? EBrushToolSizeType::World : EBrushToolSizeType::Adaptive;
		FBrushToolRadius PropValue = GetCurBrushSizeProp();
		PropValue.SizeType = NewType;
		BrushSizeProperty->SetValue_InContainer(PropListOwner, &PropValue);
		FPropertyChangedEvent PropertyChangedEvent(BrushSizeProperty, EPropertyChangeType::ValueSet);
		PropListOwner->PostEditChangeProperty(PropertyChangedEvent);
	};

	// retrieves the current size value of the BrushSize property
	const TFunction<float()> SliderValueFunc = [GetCurrentSizeType, GetCurBrushSizeProp]()
	{
		const FBrushToolRadius PropValue = GetCurBrushSizeProp();
		float Value = -1;
		if (GetCurrentSizeType() == EBrushToolSizeType::Adaptive)
		{
			Value = PropValue.AdaptiveSize;
		}
		else if (GetCurrentSizeType() == EBrushToolSizeType::World)
		{
			Value = PropValue.WorldRadius;
		}
		return Value;
	};

	// sets a new size value for the Brush Size property
	const TFunction<void(float, unsigned)> SetSliderValueFunc =
		[GetCurrentSizeType, BrushSizeProperty, PropListOwner, GetCurBrushSizeProp](const float NewValue, EPropertyValueSetFlags::Type Flags)
	{
		FBrushToolRadius PropValue = GetCurBrushSizeProp();
		
		if (GetCurrentSizeType() == EBrushToolSizeType::Adaptive)
		{
			PropValue.AdaptiveSize = NewValue;
		}
		else if (GetCurrentSizeType() == EBrushToolSizeType::World)
		{
			PropValue.WorldRadius = NewValue;
		}
		BrushSizeProperty->SetValue_InContainer(PropListOwner, &PropValue);
		FPropertyChangedEvent PropertyChangedEvent(BrushSizeProperty, EPropertyChangeType::ValueSet);
		PropListOwner->PostEditChangeProperty(PropertyChangedEvent);
	};

	// used as slider value can represent AdaptiveSize or WorldSize depending on condition
	TSharedPtr<SDynamicNumericEntry::FDataSource> NumericSource = MakeShared<SDynamicNumericEntry::FDataSource>();
	NumericSource->SetValue = SetSliderValueFunc;
	NumericSource->GetValue = SliderValueFunc;
	NumericSource->GetValueRange = [GetCurrentSizeType]() -> TInterval<float>
	{
		return (GetCurrentSizeType() == EBrushToolSizeType::Adaptive) ? TInterval<float>(0.0f, 10.0f) : TInterval<float>(0.01f, 50000.0f);
	};
	NumericSource->GetUIRange = [GetCurrentSizeType]() -> TInterval<float>
	{
		return (GetCurrentSizeType() == EBrushToolSizeType::Adaptive) ? TInterval<float>(0.0f, 1.0f) : TInterval<float>(1.0f, 1000.0f);
	};

	// construct the widget representation of the brush size property
	SHorizontalBox::FArguments BrushSizePropArgs;
	
	
	BrushSizePropArgs
	// slider
	+ SHorizontalBox::Slot()
	.AutoWidth()
	[
		SNew(SBox)
		.WidthOverride(75.f)
		[
			SNew(SDynamicNumericEntry)
			.Source(NumericSource)
		]
	]

	// adaptive/world toggle
	+ SHorizontalBox::Slot()
	.AutoWidth()
	.HAlign(HAlign_Left)
	[
		SNew(SBox)
		[
			SNew(SCheckBox)
			.Style(FMeshTerrainModeStyle::Get().Get(), "QuickProperties.RoundButton")
			.Padding(FMargin(4, 2))
			.HAlign(HAlign_Center)
			.ToolTipText( LOCTEXT("WorldToggleTooltip", "Specify Brush Size in World Units") )
			.OnCheckStateChanged_Lambda([ToggleCurrentSizeType](ECheckBoxState State)
			{
				ToggleCurrentSizeType( State == ECheckBoxState::Checked );
			})
			.IsChecked_Lambda([GetCurrentSizeType]()
			{
				return (GetCurrentSizeType() == (EBrushToolSizeType::World)) ? ECheckBoxState::Checked : ECheckBoxState::Unchecked;
			})
			[
				SNew(SHorizontalBox)
				+ SHorizontalBox::Slot()
				.VAlign(VAlign_Center)
				.Padding(FMargin(0))
				.AutoWidth()
				[
					SNew(SImage)
					.Image(FAppStyle::GetBrush("EditorViewport.RelativeCoordinateSystem_World"))
				]
			]
		]
	];
	
	return SArgumentNew(BrushSizePropArgs, SHorizontalBox);
}

//----------------------------------------------------------------------------------------------------------------

TSharedRef<IMeshTerrainPropertyCustomization> FMeshEditingViewPropertiesCustomizations::MakeInstance()
{
	return MakeShareable(new FMeshEditingViewPropertiesCustomizations);
}

TMap<FName, FMeshTerrainEditConditionData> FMeshEditingViewPropertiesCustomizations::GetEditConditionData()
{
	TMap<FName, FMeshTerrainEditConditionData> EditConditions;

	const FMeshTerrainEditConditionData FlatShadingEditCondition =
		FMeshTerrainEditConditionData(true, FGetMeshTerrainEditCondition::CreateLambda([](FProperty* Property, UObject* PropListOwner)
			{
				const UMeshEditingViewProperties* PropertySet = Cast<UMeshEditingViewProperties>(PropListOwner);
				const EMeshEditingMaterialModes MaterialMode = PropertySet->MaterialMode;

				return MaterialMode != EMeshEditingMaterialModes::ExistingMaterial
				&& MaterialMode != EMeshEditingMaterialModes::Transparent
				&& MaterialMode != EMeshEditingMaterialModes::Custom;
			}));
	EditConditions.Add("bFlatShading", FlatShadingEditCondition);

	const FMeshTerrainEditConditionData ColorEditCondition =
		FMeshTerrainEditConditionData(true, FGetMeshTerrainEditCondition::CreateLambda([](FProperty* Property, UObject* PropListOwner)
			{
				const UMeshEditingViewProperties* PropertySet = Cast<UMeshEditingViewProperties>(PropListOwner);
				const EMeshEditingMaterialModes MaterialMode = PropertySet->MaterialMode;

				return MaterialMode == EMeshEditingMaterialModes::Diffuse;
			}));
	EditConditions.Add("Color", ColorEditCondition);

	const FMeshTerrainEditConditionData ImageEditCondition =
		FMeshTerrainEditConditionData(true, FGetMeshTerrainEditCondition::CreateLambda([](FProperty* Property, UObject* PropListOwner)
			{
				const UMeshEditingViewProperties* PropertySet = Cast<UMeshEditingViewProperties>(PropListOwner);
				const EMeshEditingMaterialModes MaterialMode = PropertySet->MaterialMode;

				return MaterialMode == EMeshEditingMaterialModes::CustomImage;
			}));
	EditConditions.Add("Image", ImageEditCondition);

	FGetMeshTerrainEditCondition TransparentMaterialModeLambda =
		FGetMeshTerrainEditCondition::CreateLambda([](FProperty* Property, UObject* PropListOwner)
			{
				const UMeshEditingViewProperties* PropertySet = Cast<UMeshEditingViewProperties>(PropListOwner);
				const EMeshEditingMaterialModes MaterialMode = PropertySet->MaterialMode;

				return MaterialMode == EMeshEditingMaterialModes::Transparent;
			});
	
	const FMeshTerrainEditConditionData OpacityEditCondition =
		FMeshTerrainEditConditionData(true, TransparentMaterialModeLambda);
	EditConditions.Add("Opacity", OpacityEditCondition);
	
	const FMeshTerrainEditConditionData TransparentMaterialColorEditCondition =
		FMeshTerrainEditConditionData(true, TransparentMaterialModeLambda);
	EditConditions.Add("TransparentMaterialColor", TransparentMaterialColorEditCondition);
	
	const FMeshTerrainEditConditionData TwoSidedEditCondition =
		FMeshTerrainEditConditionData(true, TransparentMaterialModeLambda);
	EditConditions.Add("bTwoSided", TwoSidedEditCondition);
	
	const FMeshTerrainEditConditionData CustomMaterialEditCondition =
		FMeshTerrainEditConditionData(true, FGetMeshTerrainEditCondition::CreateLambda([](FProperty* Property, UObject* PropListOwner)
			{
				const UMeshEditingViewProperties* PropertySet = Cast<UMeshEditingViewProperties>(PropListOwner);
				const EMeshEditingMaterialModes MaterialMode = PropertySet->MaterialMode;

				return MaterialMode == EMeshEditingMaterialModes::Custom;
			}));
	EditConditions.Add("CustomMaterial", CustomMaterialEditCondition);
	
	return EditConditions;
}

#undef LOCTEXT_NAMESPACE
