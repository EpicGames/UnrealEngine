// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "MeshTerrainDetailCustomizations.h"

class SWidget;

namespace UE::MeshTerrain
{
	class FSculptBrushPropertiesCustomizations : public IMeshTerrainPropertyCustomization
	{
	public:
		static TSharedRef<IMeshTerrainPropertyCustomization> MakeInstance();
		virtual TMap<FName, FMeshTerrainCustomizationData> GetCustomizationData() override;
		virtual TMap<FName, FMeshTerrainEditConditionData> GetEditConditionData() override;
	private:
		static TSharedRef<SWidget> MakeBrushSizeWidget(FProperty* BrushSizeProperty, UObject* PropListOwner);
	};

	class FMeshEditingViewPropertiesCustomizations : public IMeshTerrainPropertyCustomization
	{
	public:
		static TSharedRef<IMeshTerrainPropertyCustomization> MakeInstance();
		virtual TMap<FName, FMeshTerrainEditConditionData> GetEditConditionData() override;
	};
}