// Copyright Epic Games, Inc. All Rights Reserved.

#include "MeshTerrainDetailCustomizations.h"

#include "MeshTerrainModeStyle.h"
#include "ObjectEditorUtils.h"

#define LOCTEXT_NAMESPACE "MeshTerrainDetailCustomizations"

using namespace UE::MeshTerrain;

FMeshTerrainDetailCustomizations* FMeshTerrainDetailCustomizations::Instance = nullptr;
TMap<FName, TMap<FName, FMeshTerrainCustomizationData>> FMeshTerrainDetailCustomizations::Customizations;
TMap<FName, TMap<FName, FMeshTerrainEditConditionData>> FMeshTerrainDetailCustomizations::EditConditions;

FMeshTerrainDetailCustomizations* FMeshTerrainDetailCustomizations::Get()
{
	if (!Instance)
	{
		Instance = new FMeshTerrainDetailCustomizations();
	}
	return Instance;
}

void FMeshTerrainDetailCustomizations::RegisterCustomization(const FName& SectionName,  TSharedRef<IMeshTerrainPropertyCustomization> InDelegate)
{
	Customizations.Add(SectionName, InDelegate->GetCustomizationData());
	EditConditions.Add(SectionName, InDelegate->GetEditConditionData());
}

const TMap<FName, FMeshTerrainCustomizationData>* FMeshTerrainDetailCustomizations::GetCustomizedWidgets(const FName& SectionName)
{
	if (const TMap<FName, FMeshTerrainCustomizationData>* WidgetCustomizationsMap = Customizations.Find( FName(SectionName.GetPlainNameString())))
	{
		return WidgetCustomizationsMap;
	}
	return nullptr;
}

const FMeshTerrainCustomizationData* FMeshTerrainDetailCustomizations::GetCustomizationData(const UObject* PropOwner, const FProperty* Property)
{
	const TMap<FName, FMeshTerrainCustomizationData>* WidgetCustomizationsMap = GetCustomizedWidgets(PropOwner->GetFName());

	if (!WidgetCustomizationsMap)
	{
		return nullptr;
	}
	if (const FMeshTerrainCustomizationData* WidgetData = WidgetCustomizationsMap->Find(Property->GetFName()))
	{
		return WidgetData;
	}
	return nullptr;
}

const TMap<FName, FMeshTerrainEditConditionData>* FMeshTerrainDetailCustomizations::GetEditConditionData(const FName& SectionName)
{
	if (const TMap<FName, FMeshTerrainEditConditionData>* EditConditionMap = EditConditions.Find(FName(SectionName.GetPlainNameString())))
	{
		return EditConditionMap;
	}
	return nullptr;
}

const FMeshTerrainEditConditionData* FMeshTerrainDetailCustomizations::GetEditCondition(const UObject* PropOwner, const FProperty* Property)
{
	const TMap<FName, FMeshTerrainEditConditionData>* EditConditionMap = GetEditConditionData(PropOwner->GetFName());
	if (!EditConditionMap)
	{
		return nullptr;
	}
	if (const FMeshTerrainEditConditionData* EditCondition = EditConditionMap->Find(Property->GetFName()))
	{
		return EditCondition;
	}
	return nullptr;
}

#undef LOCTEXT_NAMESPACE
