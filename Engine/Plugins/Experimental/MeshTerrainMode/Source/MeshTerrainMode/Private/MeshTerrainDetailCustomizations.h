// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

class SWidget;

DECLARE_DELEGATE_RetVal_TwoParams(TSharedRef<SWidget>, FGetMeshTerrainDetailCustomization, FProperty*, UObject*);
DECLARE_DELEGATE_RetVal_TwoParams(bool, FGetMeshTerrainEditCondition, FProperty*, UObject*);

namespace UE::MeshTerrain
{

	enum class EMeshTerrainCustomizationType
	{
		None,
		WidgetOnly,
		WholeRow
	};

	struct FMeshTerrainCustomizationData
	{
		FGetMeshTerrainDetailCustomization DetailCustomization;
		EMeshTerrainCustomizationType Type;

		FMeshTerrainCustomizationData() : DetailCustomization(nullptr), Type(EMeshTerrainCustomizationType::None) {}
		FMeshTerrainCustomizationData(FGetMeshTerrainDetailCustomization InDetailsCustomization)
		: DetailCustomization(InDetailsCustomization), Type(EMeshTerrainCustomizationType::WidgetOnly) {}
		FMeshTerrainCustomizationData(const FGetMeshTerrainDetailCustomization& InDetailsCustomization, EMeshTerrainCustomizationType InType)
		: DetailCustomization(InDetailsCustomization), Type(InType) {}
	};

	struct FMeshTerrainEditConditionData
	{
		bool bEditConditionHides;
		FGetMeshTerrainEditCondition EditCondition;
	};
	
	class IMeshTerrainPropertyCustomization : public TSharedFromThis<IMeshTerrainPropertyCustomization>
	{
	public:
		virtual ~IMeshTerrainPropertyCustomization() {}
		
		virtual TMap<FName, FMeshTerrainCustomizationData> GetCustomizationData() { return TMap<FName, FMeshTerrainCustomizationData>(); };
		virtual TMap<FName, FMeshTerrainEditConditionData> GetEditConditionData() { return TMap<FName, FMeshTerrainEditConditionData>(); };
	};
	

	class FMeshTerrainDetailCustomizations
	{
	public:
		static FMeshTerrainDetailCustomizations* Get();

		static void RegisterCustomization(const FName& SectionName,  TSharedRef<IMeshTerrainPropertyCustomization> InDelegate);

		/** Retrieve the map of all PropertyNames to their CustomizationData which fall in the given section */
		static const TMap<FName, FMeshTerrainCustomizationData>* GetCustomizedWidgets(const FName& SectionName);
		
		/** Retrieve the customization data matching the provided PropertyOwner and Property. Return null if none exists*/
		static const FMeshTerrainCustomizationData* GetCustomizationData(const UObject* PropOwner, const FProperty* Property);

		/** Retrieve the map of all PropertyNames to their EditCondition data which fall in the given section */
		static const TMap<FName, FMeshTerrainEditConditionData>* GetEditConditionData(const FName& SectionName);

		/** Retrieve the edit condition data matching the provided PropertyOwner and Property. Return null if none exists*/
		static const FMeshTerrainEditConditionData* GetEditCondition(const UObject* PropOwner, const FProperty* Property);
		
	private:
		FMeshTerrainDetailCustomizations() { };
		~FMeshTerrainDetailCustomizations() { };
		
		static FMeshTerrainDetailCustomizations* Instance;
		static TMap<FName, TMap<FName, FMeshTerrainCustomizationData>> Customizations;
		static TMap<FName, TMap<FName, FMeshTerrainEditConditionData>> EditConditions;
	};
}