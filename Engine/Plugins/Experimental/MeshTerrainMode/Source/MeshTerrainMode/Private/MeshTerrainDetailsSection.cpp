// Copyright Epic Games, Inc. All Rights Reserved.

#include "MeshTerrainDetailsSections.h"

FMeshTerrainDetailsSections* FMeshTerrainDetailsSections::Instance = nullptr;
TMap<FName, TMap<FName, FName>> FMeshTerrainDetailsSections::AllSectionMappings;

FMeshTerrainDetailsSections* FMeshTerrainDetailsSections::Get()
{
	return Instance;
}

void FMeshTerrainDetailsSections::Initialize()
{
	if (!Instance)
	{
		Instance = new FMeshTerrainDetailsSections();
	}
}

TMap<FName, FName> FMeshTerrainDetailsSections::GetSectionMappings(const FName& SectionName)
{
	if (TMap<FName, FName>* CategoryToSectionMap = AllSectionMappings.Find(SectionName))
	{
		return *CategoryToSectionMap;
	}
	return TMap<FName, FName>();
}

FMeshTerrainDetailsSections::FMeshTerrainDetailsSections()
{
	RegisterSculptSubmodeTabs();
	RegisterShapeSubmodeTabs();
	RegisterEditSubmodeTabs();
	RegisterCreateSubmodeTabs();
	RegisterPaintSubmodeTabs();
}

void FMeshTerrainDetailsSections::RegisterSculptSubmodeTabs()
{
	{
		TMap<FName, FName> SculptToolBrushTab;
		SculptToolBrushTab.Add("Stroke", "Brush");
		SculptToolBrushTab.Add("Brush", "Brush");
		AllSectionMappings.Add(FName("SculptBrushProperties"), SculptToolBrushTab);

		TMap<FName, FName> SculptToolSculptBrushTab;
		SculptToolSculptBrushTab.Add("SculptBrush", "Brush");
		AllSectionMappings.Add(FName("StandardSculptBrushOpProps"), SculptToolSculptBrushTab);

		TMap<FName, FName> SculptToolMoveBrushTab;
		SculptToolMoveBrushTab.Add("MoveBrush", "Brush");
		AllSectionMappings.Add(FName("MoveBrushOpProps"), SculptToolMoveBrushTab);
		
		TMap<FName, FName> SculptToolSmoothBrushTab;
		SculptToolSmoothBrushTab.Add("SmoothBrush", "Brush");
		AllSectionMappings.Add(FName("SmoothBrushOpProps"), SculptToolSmoothBrushTab);
		
		TMap<FName, FName> SculptToolPinchBrushTab;
		SculptToolPinchBrushTab.Add("PinchBrush", "Brush");
		AllSectionMappings.Add(FName("PinchBrushOpProps"), SculptToolPinchBrushTab);
		
		TMap<FName, FName> SculptToolFlattenBrushTab;
		SculptToolFlattenBrushTab.Add("FlattenBrush", "Brush");
		AllSectionMappings.Add(FName("FlattenBrushOpProps"), SculptToolFlattenBrushTab);

		TMap<FName, FName> SculptToolEraseSculptLayerBrushTab;
		SculptToolEraseSculptLayerBrushTab.Add("EraseSculptLayerBrush", "Brush");
		AllSectionMappings.Add(FName("EraseSculptLayerBrushOpProps"), SculptToolEraseSculptLayerBrushTab);

		TMap<FName, FName> SculptToolBrushOpBrushTab;
		SculptToolBrushOpBrushTab.Add("BrushOp", "Brush");
		AllSectionMappings.Add(FName("MeshSculptBrushOpProps"), SculptToolBrushOpBrushTab);

		TMap<FName, FName> SculptToolSculptingBrushTab;
		SculptToolSculptingBrushTab.Add("Sculpting", "Brush");
		AllSectionMappings.Add(FName("VertexBrushSculptProperties"), SculptToolSculptingBrushTab);

		TMap<FName, FName> SculptToolAlphaBrushTab;
		SculptToolAlphaBrushTab.Add("Alpha", "Brush");
		AllSectionMappings.Add(FName("VertexBrushAlphaProperties"), SculptToolAlphaBrushTab);

		TMap<FName, FName> SculptToolSecondarySmoothBrushTab;
		SculptToolSecondarySmoothBrushTab.Add("ShiftSmoothBrush", "Brush");
		AllSectionMappings.Add(FName("SecondarySmoothBrushOpProps"), SculptToolSecondarySmoothBrushTab);
		
		TMap<FName, FName> SculptToolRenderingTab;
		SculptToolRenderingTab.Add("Rendering", "Rendering");
		AllSectionMappings.Add(FName("MeshEditingViewProperties"), SculptToolRenderingTab);

		TMap<FName, FName> SculptToolMeshLayersTab;
		SculptToolMeshLayersTab.Add("MeshLayers", "Sculpting");
		AllSectionMappings.Add(FName("MeshSculptLayerProperties"), SculptToolMeshLayersTab);
		
		TMap<FName, FName> SculptToolSymmetry;
		SculptToolSymmetry.Add("Symmetry", "Brush");
		AllSectionMappings.Add(FName("MeshSymmetryProperties"), SculptToolSymmetry);
	}

	{
		TMap<FName, FName> HSculptToolHeightSculptBrushTab;
		HSculptToolHeightSculptBrushTab.Add("HeightSculptOptions", "Brush");
		AllSectionMappings.Add(FName("HeightSculptToolProperties"), HSculptToolHeightSculptBrushTab);

		TMap<FName, FName> HSculptToolHeightFlattenBrushTab;
		HSculptToolHeightFlattenBrushTab.Add("HeightBrush", "Brush");
		AllSectionMappings.Add(FName("MeshHeightSculptFlattenBrushOpProps"), HSculptToolHeightFlattenBrushTab);

		TMap<FName, FName> HSculptToolHeightEraseBrushTab;
		HSculptToolHeightEraseBrushTab.Add("HeightBrush", "Brush");
		AllSectionMappings.Add(FName("MeshHeightSculptEraseBrushOpProps"), HSculptToolHeightEraseBrushTab);

		TMap<FName, FName> HSculptToolSlopeErodeBrushTab;
		HSculptToolSlopeErodeBrushTab.Add("HeightBrush", "Brush");
		AllSectionMappings.Add(FName("MeshHeightSculptSlopeErodeBrushOpProps"), HSculptToolSlopeErodeBrushTab);
		
		TMap<FName, FName> HSculptToolHeightBrushTab;
		HSculptToolHeightBrushTab.Add("HeightBrush", "Brush");
		AllSectionMappings.Add(FName("MeshHeightSculptBrushOpProps"), HSculptToolHeightBrushTab);

		TMap<FName, FName> HSculptToolTargetPlaneBrushTab;
		HSculptToolTargetPlaneBrushTab.Add("TargetPlane", "Brush");
		AllSectionMappings.Add(FName("WorkPlaneProperties"), HSculptToolTargetPlaneBrushTab);
		AllSectionMappings.Add(FName("HeightSculptWorkPlaneProperties"), HSculptToolTargetPlaneBrushTab);
	}
}

void FMeshTerrainDetailsSections::RegisterShapeSubmodeTabs()
{
		{
		TMap<FName, FName> BoxToolOutputShapeTab;
		BoxToolOutputShapeTab.Add("OutputType", "Shape");
		AllSectionMappings.Add(FName("CreateMeshObjectTypeProperties"), BoxToolOutputShapeTab);
		
		TMap<FName, FName> ShapeToolShapeTab;
		ShapeToolShapeTab.Add("Shape", "Shape");
		ShapeToolShapeTab.Add("Positioning", "Position");
		AllSectionMappings.Add(FName("ShapeSettings"), ShapeToolShapeTab);

		TMap<FName, FName> BoxToolShapeTab;
		BoxToolShapeTab.Add("Shape", "Shape");
		AllSectionMappings.Add(FName("ProceduralBoxToolProperties"), BoxToolShapeTab);
		
		TMap<FName, FName> RectangleToolShapeTab;
		RectangleToolShapeTab.Add("Shape", "Shape");
		AllSectionMappings.Add(FName("ProceduralRectangleToolProperties"), RectangleToolShapeTab);

		TMap<FName, FName> DiscToolShapeTab;
		DiscToolShapeTab.Add("Shape", "Shape");
		AllSectionMappings.Add(FName("ProceduralDiscToolProperties"), DiscToolShapeTab);
		
		TMap<FName, FName> TorusToolShapeTab;
		TorusToolShapeTab.Add("Shape", "Shape");
		AllSectionMappings.Add(FName("ProceduralTorusToolProperties"), TorusToolShapeTab);

		TMap<FName, FName> CylinderToolShapeTab;
		CylinderToolShapeTab.Add("Shape", "Shape");
		AllSectionMappings.Add(FName("ProceduralCylinderToolProperties"), CylinderToolShapeTab);
		
		TMap<FName, FName> ConeToolShapeTab;
		ConeToolShapeTab.Add("Shape", "Shape");
		AllSectionMappings.Add(FName("ProceduralConeToolProperties"), ConeToolShapeTab);

		TMap<FName, FName> ArrowToolShapeTab;
		ArrowToolShapeTab.Add("Shape", "Shape");
		AllSectionMappings.Add(FName("ProceduralArrowToolProperties"), ArrowToolShapeTab);

		TMap<FName, FName> SphereToolShapeTab;
		SphereToolShapeTab.Add("Shape", "Shape");
		AllSectionMappings.Add(FName("ProceduralSphereToolProperties"), SphereToolShapeTab);
		
		TMap<FName, FName> CapsuleToolShapeTab;
		CapsuleToolShapeTab.Add("Shape", "Shape");
		AllSectionMappings.Add(FName("ProceduralCapsuleToolProperties"), CapsuleToolShapeTab);
		
		TMap<FName, FName> StairsToolShapeTab;
		StairsToolShapeTab.Add("Shape", "Shape");
		AllSectionMappings.Add(FName("ProceduralStairsToolProperties"), StairsToolShapeTab);
	}

	{
		TMap<FName, FName> AddPrimitiveToolMaterialTab;
		AddPrimitiveToolMaterialTab.Add("Material", "Materials");
		AllSectionMappings.Add(FName("NewMeshMaterialProperties"), AddPrimitiveToolMaterialTab);
	}
}

void FMeshTerrainDetailsSections::RegisterEditSubmodeTabs()
{
	{
		TMap<FName, FName> ConvertToolSplit;
		ConvertToolSplit.Add("Split", "GeneralOptions");
		AllSectionMappings.Add(FName("SplitProperties"), ConvertToolSplit);

		TMap<FName, FName> ConvertToolAdd;
		ConvertToolAdd.Add("Add", "GeneralOptions");
		AllSectionMappings.Add(FName("CreateProperties"), ConvertToolAdd);
	}

	{
		TMap<FName, FName> ExpandToolTabs;
		ExpandToolTabs.Add("Settings", "GeneralOptions");
		ExpandToolTabs.Add("Mirror", "GeneralOptions");
		ExpandToolTabs.Add("Visualization", "GeneralOptions");
		ExpandToolTabs.Add("Extrude", "Extrude");
		AllSectionMappings.Add(FName("ExpandToolProperties"), ExpandToolTabs);
	}

	{
		TMap<FName, FName> SplitToolBrushTab;
		SplitToolBrushTab.Add("Brush", "Brush");
		AllSectionMappings.Add(FName("Brush"), SplitToolBrushTab);
	
		TMap<FName, FName> SplitToolVisualizationTab;
		SplitToolVisualizationTab.Add("MeshElementVisualization", "Visualization");
		AllSectionMappings.Add(FName("MeshElementsVisualizerProperties"), SplitToolVisualizationTab);
	}

	{
		TMap<FName, FName> StitchToolActions;
		StitchToolActions.Add("None", "GeneralOptions");
		AllSectionMappings.Add(FName("StitchToolActions"), StitchToolActions);
	}

	{
		TMap<FName, FName> MergeToolTab;
		MergeToolTab.Add("Merge", "GeneralOptions");
		AllSectionMappings.Add(FName("MergeToolProperties"), MergeToolTab);
	}
	
	{
		TMap<FName, FName> ResectionToolTab;
		ResectionToolTab.Add("Merge", "GeneralOptions");
		ResectionToolTab.Add("Sections", "GeneralOptions");
		AllSectionMappings.Add(FName("MeshPartitionResectionToolProperties"), ResectionToolTab);
	}

	{
		TMap<FName, FName> DuplicateToolOutputObj;
		DuplicateToolOutputObj.Add("OutputObject", "GeneralOptions");
		AllSectionMappings.Add(FName("CombineMeshesToolProperties"), DuplicateToolOutputObj);

		TMap<FName, FName> DuplicateToolOutputType;
		DuplicateToolOutputType.Add("OutputType", "GeneralOptions");
		AllSectionMappings.Add(FName("CreateMeshObjectTypeProperties"), DuplicateToolOutputType);

		TMap<FName, FName> DuplicateToolOnToolAccept;
		DuplicateToolOnToolAccept.Add("OnToolAccept", "GeneralOptions");
		AllSectionMappings.Add(FName("OnAcceptHandleSourcesProperties"), DuplicateToolOnToolAccept);

		TMap<FName, FName> DuplicateToolOnToolAcceptSingle;
		DuplicateToolOnToolAcceptSingle.Add("OnToolAccept", "GeneralOptions");
		AllSectionMappings.Add(FName("OnAcceptHandleSourcesPropertiesSingle"), DuplicateToolOnToolAcceptSingle);
	}
	{
		TMap<FName, FName> EditPivotToolSnappingTab;
		EditPivotToolSnappingTab.Add("Options", "Snapping");
		EditPivotToolSnappingTab.Add("SnapDragging", "Snapping");
		AllSectionMappings.Add(FName("EditPivotToolProperties"), EditPivotToolSnappingTab);
		
		TMap<FName, FName> EditPivotToolBoxPosnTab;
		EditPivotToolBoxPosnTab.Add("BoxPositions", "Position");
		AllSectionMappings.Add(FName("EditPivotToolActionPropertySet"), EditPivotToolBoxPosnTab);
	}
	{
		TMap<FName, FName> BakeTransformToolTab;
		BakeTransformToolTab.Add("Options", "GeneralOptions");
		AllSectionMappings.Add(FName("BakeTransformToolProperties"), BakeTransformToolTab);
	}
	{
		TMap<FName, FName> EditAttribToolNormalsTab;
		EditAttribToolNormalsTab.Add("None", "Normals");
		AllSectionMappings.Add(FName("AttributeEditorNormalsActions"), EditAttribToolNormalsTab);
		
		TMap<FName, FName> EditAttribToolUVsTab;
		EditAttribToolUVsTab.Add("UVs", "UVs");
		AllSectionMappings.Add(FName("AttributeEditorUVActions"), EditAttribToolUVsTab);
		
		TMap<FName, FName> EditAttribToolLightmapUVsTab;
		EditAttribToolLightmapUVsTab.Add("LightmapUVs", "UVs");
		AllSectionMappings.Add(FName("AttributeEditorLightmapUVActions"), EditAttribToolLightmapUVsTab);
		
		TMap<FName, FName> EditAttribToolNewAttributesTab;
		EditAttribToolNewAttributesTab.Add("NewAttribute", "Attributes");
		AllSectionMappings.Add(FName("AttributeEditorNewAttributeActions"), EditAttribToolNewAttributesTab);
		
		TMap<FName, FName> EditAttribToolModifyAttributesTab;
		EditAttribToolModifyAttributesTab.Add("ModifyAttribute", "Attributes");
		AllSectionMappings.Add(FName("AttributeEditorModifyAttributeActions"), EditAttribToolModifyAttributesTab);
		
		TMap<FName, FName> EditAttribToolCopyAttributesTab;
		EditAttribToolCopyAttributesTab.Add("CopyAttribute", "Attributes");
		AllSectionMappings.Add(FName("AttributeEditorCopyAttributeActions"), EditAttribToolCopyAttributesTab);
		
		TMap<FName, FName> EditAttribToolAttribInspectorTab;
		EditAttribToolAttribInspectorTab.Add("AttributesInspector", "AttributeInspector");
		AllSectionMappings.Add(FName("AttributeEditorAttribProperties"), EditAttribToolAttribInspectorTab);
	}
	{
		TMap<FName, FName> InspectToolOptionsTab;
		InspectToolOptionsTab.Add("Options", "GeneralOptions");
		AllSectionMappings.Add(FName("MeshInspectorProperties"), InspectToolOptionsTab);

		TMap<FName, FName> InspectToolPolyGroupLayer;
		InspectToolPolyGroupLayer.Add("PolyGroup Layer", "Polygroups");
		AllSectionMappings.Add(FName("PolygroupLayersProperties"), InspectToolPolyGroupLayer);
		
		TMap<FName, FName> InspectToolPolyGroupPrevMaterial;
		InspectToolPolyGroupPrevMaterial.Add("PreviewMaterial", "Polygroups");
		AllSectionMappings.Add(FName("MeshInspectorMaterialProperties"), InspectToolPolyGroupPrevMaterial);

		TMap<FName, FName> InspectToolStatistics;
		InspectToolStatistics.Add("MeshStatistics", "Inspect");
		AllSectionMappings.Add(FName("MeshStatisticsProperties"), InspectToolStatistics);

		TMap<FName, FName> InspectToolAnalysis;
		InspectToolAnalysis.Add("MeshAnalysis", "Inspect");
		AllSectionMappings.Add(FName("MeshAnalysisProperties"), InspectToolAnalysis);
	}
}

void FMeshTerrainDetailsSections::RegisterCreateSubmodeTabs()
{
	{
		TMap<FName, FName> MMRectangleToolMesh;
		MMRectangleToolMesh.Add("Mesh", "Shape");
		MMRectangleToolMesh.Add("Sections", "Shape");
		AllSectionMappings.Add(FName("CreateRectangleToolProperties"), MMRectangleToolMesh);
		
		TMap<FName, FName> MMRectangleToolPlacement;
		MMRectangleToolPlacement.Add("Positioning", "Position");
		MMRectangleToolPlacement.Add("Snapping", "Position");
		AllSectionMappings.Add(FName("PlacementProperties"), MMRectangleToolPlacement);

		TMap<FName, FName> MMCreateProps;
		MMCreateProps.Add("Add", "Shape");
		AllSectionMappings.Add(FName("CreateProperties"), MMCreateProps);
	}
	{
		TMap<FName, FName> ImportHeightmapTool;
		ImportHeightmapTool.Add("Heightmap", "Import");
		ImportHeightmapTool.Add("Mesh", "Import");
		ImportHeightmapTool.Add("Sections", "Import");
		ImportHeightmapTool.Add("LocationVolumes", "Import");
		AllSectionMappings.Add(FName("HeightmapImportPropertySet"), ImportHeightmapTool);
	}
	{
		TMap<FName, FName> PatternToolGeneralTab;
		PatternToolGeneralTab.Add("General", "GeneralOptions");
		PatternToolGeneralTab.Add("Shape", "Pattern");
		AllSectionMappings.Add(FName("PatternToolSettings"), PatternToolGeneralTab);
		
		TMap<FName, FName> PatternToolLinear;
		PatternToolLinear.Add("LinearPattern", "Pattern");
		AllSectionMappings.Add(FName("PatternTool_LinearSettings"), PatternToolLinear);

		TMap<FName, FName> PatternToolRotation;
		PatternToolRotation.Add("Rotation", "Position");
		AllSectionMappings.Add(FName("PatternTool_RotationSettings"), PatternToolRotation);

		TMap<FName, FName> PatternToolTranslation;
		PatternToolTranslation.Add("Translation", "Position");
		AllSectionMappings.Add(FName("PatternTool_TranslationSettings"), PatternToolTranslation);

		TMap<FName, FName> PatternToolScale;
		PatternToolScale.Add("Scale", "Position");
		AllSectionMappings.Add(FName("PatternTool_ScaleSettings"), PatternToolScale);

		TMap<FName, FName> PatternToolOutput;
		PatternToolOutput.Add("Output", "Output");
		AllSectionMappings.Add(FName("PatternTool_OutputSettings"), PatternToolOutput);
	}
	{
		TMap<FName, FName> DrawSplineTool;
		DrawSplineTool.Add("Spline", "Spline");
		DrawSplineTool.Add("RaycastTargets", "Raycast");
		AllSectionMappings.Add(FName("DrawSplineToolProperties"), DrawSplineTool);
	}
}

void FMeshTerrainDetailsSections::RegisterPaintSubmodeTabs()
{
		{
		TMap<FName, FName> GroupPaintToolFilterProps;
		GroupPaintToolFilterProps.Add("ActionType", "Polygroups");
		GroupPaintToolFilterProps.Add("Filters", "Filters");
		GroupPaintToolFilterProps.Add("Visualization", "Filters");
		AllSectionMappings.Add(FName("GroupPaintBrushFilterProperties"), GroupPaintToolFilterProps);

		TMap<FName, FName> GroupPaintToolOperations;
		GroupPaintToolOperations.Add("None", "Operations");
		AllSectionMappings.Add(FName("MeshGroupPaintToolFreezeActions"), GroupPaintToolOperations);
	}
	{
		TMap<FName, FName> VertexColorsToolBrushTab;
		VertexColorsToolBrushTab.Add("Settings", "Brush");
		AllSectionMappings.Add(FName("VertexPaintBasicProperties"), VertexColorsToolBrushTab);

		TMap<FName, FName> VertexColorsToolFiltersTab;
		VertexColorsToolFiltersTab.Add("Filters", "Filters");
		VertexColorsToolFiltersTab.Add("Visualization", "Filters");
		AllSectionMappings.Add(FName("VertexPaintBrushFilterProperties"), VertexColorsToolFiltersTab);

		TMap<FName, FName> VertexColorsQuickActions;
		VertexColorsQuickActions.Add("None", "Operations");
		AllSectionMappings.Add(FName("MeshVertexPaintToolQuickActions"), VertexColorsQuickActions);
		
		TMap<FName, FName> VertexColorsUtilityOperations;
		VertexColorsUtilityOperations.Add("None", "Operations");
		AllSectionMappings.Add(FName("MeshVertexPaintToolUtilityActions"), VertexColorsUtilityOperations);
	}
	{
		TMap<FName, FName> PaintMapsToolAttributeOps;
		PaintMapsToolAttributeOps.Add("Attribute", "GeneralOptions");
		AllSectionMappings.Add(FName("MeshAttributePaintBrushOperationProperties"), PaintMapsToolAttributeOps);

		TMap<FName, FName> PaintMapsToolMeshAttributes;
		PaintMapsToolMeshAttributes.Add("Attribute", "GeneralOptions");
		AllSectionMappings.Add(FName("MeshAttributePaintToolProperties"), PaintMapsToolMeshAttributes);

		TMap<FName, FName> PaintMapsToolMMAttributes;
		PaintMapsToolMMAttributes.Add("None", "Brush");
		PaintMapsToolMMAttributes.Add("Mega Mesh Channels", "Operations");
			PaintMapsToolMMAttributes.Add("Name", "Operations");
			PaintMapsToolMMAttributes.Add("WeightChannelName", "Operations");
		AllSectionMappings.Add(FName("AttributePaintToolAddChannelProperties"), PaintMapsToolMMAttributes);
			
		TMap<FName, FName> PaintMapsToolVisualization;
		PaintMapsToolVisualization.Add("Rendering", "Rendering");
		AllSectionMappings.Add(FName("MeshAttributePaintToolVisualizationProperties"), PaintMapsToolVisualization);
	}
}
