// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

class FMeshTerrainDetailsSections
{
public:
	static FMeshTerrainDetailsSections* Get();
	static void Initialize();

	static TMap<FName, FName> GetSectionMappings(const FName& SectionName);
private:
	FMeshTerrainDetailsSections();
	~FMeshTerrainDetailsSections() { };

	static void RegisterSculptSubmodeTabs();
	static void RegisterShapeSubmodeTabs();
	static void RegisterEditSubmodeTabs();
	static void RegisterCreateSubmodeTabs();
	static void RegisterPaintSubmodeTabs();
		
	static FMeshTerrainDetailsSections* Instance;
	static TMap<FName, TMap<FName, FName>> AllSectionMappings;
};