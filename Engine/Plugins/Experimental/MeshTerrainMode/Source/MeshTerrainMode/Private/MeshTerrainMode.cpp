// Copyright Epic Games, Inc. All Rights Reserved.

#include "MeshTerrainMode.h"
#include "Components/StaticMeshComponent.h"
#include "IGeometryProcessingInterfacesModule.h"
#include "Kismet2/ComponentEditorUtils.h"
#include "Editor/UnrealEdEngine.h"
#include "Tools/EdModeInteractiveToolsContext.h"
#include "InteractiveToolManager.h"
#include "GeometryProcessingInterfaces/IUVEditorModularFeature.h"
#include "IAnalyticsProviderET.h"
#include "MeshSculptLayersManagerAPI.h"
#include "MeshTerrainModeToolkit.h"
#include "ILevelEditor.h"
#include "MeshTerrainModeSettings.h"
#include "MeshTerrainModeHostCustomizationAPI.h"
#include "ToolTargetManager.h"
#include "ContextObjectStore.h"
#include "InputRouter.h"
#include "ToolTargets/StaticMeshComponentToolTarget.h"
#include "Selection.h"
#include "ToolTargets/VolumeComponentToolTarget.h"
#include "ToolTargets/DynamicMeshComponentToolTarget.h"
#include "ToolTargets/SkeletalMeshComponentToolTarget.h"
#include "EngineAnalytics.h"
#include "BaseGizmos/TransformGizmoUtil.h"
#include "Snapping/ModelingSceneSnappingManager.h"
#include "Scene/LevelObjectsObserver.h"
#include "UnrealEdGlobals.h" // GUnrealEd
#include "Algo/RemoveIf.h"
#include "Widgets/Notifications/SNotificationList.h"
#include "Framework/Notifications/NotificationManager.h"

#include "Features/IModularFeatures.h"
#include "MeshTerrainModeToolExtensions.h"

#include "DynamicMeshSculptTool.h"
#include "MeshVertexSculptTool.h"
#include "EditMeshPolygonsTool.h"
#include "DeformMeshPolygonsTool.h"
#include "SubdividePolyTool.h"
#include "ConvertToPolygonsTool.h"
#include "AddPrimitiveTool.h"
#include "AddPatchTool.h"
#include "CubeGridTool.h"
#include "RevolveBoundaryTool.h"
#include "SmoothMeshTool.h"
#include "OffsetMeshTool.h"
#include "SimplifyMeshTool.h"
#include "MeshInspectorTool.h"
#include "WeldMeshEdgesTool.h"
#include "DrawPolygonTool.h"
#include "DrawPolyPathTool.h"
#include "DrawAndRevolveTool.h"
#include "DrawSplineTool.h"
#include "RevolveSplineTool.h"
#include "ShapeSprayTool.h"
#include "VoxelSolidifyMeshesTool.h"
#include "VoxelBlendMeshesTool.h"
#include "VoxelMorphologyMeshesTool.h"
#include "PlaneCutTool.h"
#include "MirrorTool.h"
#include "SelfUnionMeshesTool.h"
#include "CSGMeshesTool.h"
#include "CutMeshWithMeshTool.h"
#include "BspConversionTool.h"
#include "MeshToVolumeTool.h"
#include "VolumeToMeshTool.h"
#include "HoleFillTool.h"
#include "PolygonOnMeshTool.h"
#include "DisplaceMeshTool.h"
#include "MeshSpaceDeformerTool.h"
#include "EditNormalsTool.h"
#include "RemoveOccludedTrianglesTool.h"
#include "AttributeEditorTool.h"
#include "TransformMeshesTool.h"
#include "UVProjectionTool.h"
#include "UVLayoutTool.h"
#include "EditMeshMaterialsTool.h"
#include "AddPivotActorTool.h"
#include "EditPivotTool.h"
#include "BakeTransformTool.h"
#include "CombineMeshesTool.h"
#include "AlignObjectsTool.h"
#include "EditUVIslandsTool.h"
#include "BakeMeshAttributeMapsTool.h"
#include "BakeMultiMeshAttributeMapsTool.h"
#include "BakeMeshAttributeVertexTool.h"
#include "BakeRenderCaptureTool.h"
#include "MeshPartitionAttributePaintTool.h"
#include "ParameterizeMeshTool.h"
#include "RecomputeUVsTool.h"
#include "MeshTangentsTool.h"
#include "ProjectToTargetTool.h"
#include "LatticeDeformerTool.h"
#include "SeamSculptTool.h"
#include "MeshGroupPaintTool.h"
#include "MeshVertexPaintTool.h"
#include "TransferMeshTool.h"
#include "ConvertMeshesTool.h"
#include "SplitMeshesTool.h"
#include "PatternTool.h"
#include "HarvestInstancesTool.h"
#include "TriangulateSplinesTool.h"
#include "BaseTools/BaseBrushTool.h"

#include "Polymodeling/ExtrudeMeshSelectionTool.h"
#include "Polymodeling/OffsetMeshSelectionTool.h"

#include "Physics/PhysicsInspectorTool.h"
#include "Physics/SetCollisionGeometryTool.h"
#include "Physics/ExtractCollisionGeometryTool.h"
#include "Physics/SimpleCollisionEditorTool.h"

// asset tools
#include "Tools/GenerateStaticMeshLODAssetTool.h"
#include "Tools/LODManagerTool.h"
#include "ISMEditorTool.h"

// commands
#include "Commands/DeleteGeometrySelectionCommand.h"
#include "Commands/DisconnectGeometrySelectionCommand.h"
#include "Commands/RetriangulateGeometrySelectionCommand.h"
#include "Commands/ModifyGeometrySelectionCommand.h"


#include "EditorModeManager.h"
#include "UnrealWidget.h"


#include "LevelEditor.h"
#include "SLevelViewport.h"
#include "Application/ThrottleManager.h"

#include "MeshTerrainModeActions.h"
#include "MeshTerrainModeManagerActions.h"
#include "MeshTerrainModeAssetUtils.h"
#include "EditorModelingObjectsCreationAPI.h"

#include "Selection/GeometrySelectionManager.h"
#include "Selection/VolumeSelector.h"
#include "Selection/StaticMeshSelector.h"
#include "MeshTerrainModeSelectionInteraction.h"
#include "DynamicMeshActor.h"
#include "MeshTerrainModeSettings.h"
#include "MeshTerrainModeStyle.h"
#include "Components/BrushComponent.h"
#include "Engine/StaticMeshActor.h"
#include "Engine/World.h" // FWorldDelegates
#include "ViewportToolbar/UnrealEdViewportToolbar.h"
#include "ToolMenus.h"
#include "Framework/Docking/LayoutExtender.h"
#include "Framework/Application/SlateApplication.h"
#include "Widgets/MeshTerrainModeUtil.h"

// MegaMesh
#include "MeshPartitionChannel.h"
#include "MeshPartitionModifierToolTarget.h"
#include "ModelingToolTargetUtil.h"
#include "MeshPartitionMultiSectionToolTarget.h"
#include "MeshPartitionSectionToolTarget.h"
#include "MeshPartitionToolTarget.h"
#include "Modifiers/MeshPartitionMeshProvider.h"
#include "MeshPartitionMultiSectionToolTarget.h"
#include "MeshPartitionSectionToolTarget.h"
#include "MeshPartitionToolTarget.h"

#include "MeshPartitionConvertTool.h"
#include "MeshPartitionCreateMeshTool.h"
#include "MeshPartitionExpandTool.h"
#include "MeshPartitionHeightmapImportTool.h"
#include "MeshPartitionHeightSculptTool.h"
#include "MeshPartitionMergeTool.h"
#include "MeshPartitionResectionTool.h"
#include "MeshPartitionPlaceModifierTool.h"
#include "MeshPartitionSplitTool.h"
#include "MeshPartitionStitchTool.h"
#include "MeshTerrainModeStylusInputHandler.h"
#include "MeshPartition.h"
#include "MeshPartitionModifierActor.h"
#include "MeshPartitionModifierComponent.h"
#include "Modifiers/MeshPartitionEditableModifierBase.h"
#include "SingleSelectionTool.h"
#include "Submodes/Submode.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(MeshTerrainMode)

#if WITH_PROXYLOD
#include "MergeMeshesTool.h"
#include "VoxelCSGMeshesTool.h"
#endif

#define LOCTEXT_NAMESPACE "UMeshTerrainMode"

namespace UE::MeshTerrain
{

const FEditorModeID UMeshTerrainMode::EM_MeshTerrainEditorModeId = TEXT("EM_MeshTerrainMode");

FDateTime UMeshTerrainMode::LastModeStartTimestamp;
FDateTime UMeshTerrainMode::LastToolStartTimestamp;

FDelegateHandle UMeshTerrainMode::GlobalModelingWorldTeardownEventHandle;

namespace MeshTerrainModeLocals
{
const FName MeshTerrainModeOwnerName = "MeshTerrainModeViewportToolbar";
	
FString GetToolName(const UInteractiveTool& Tool)
{
	const FString* ToolName = FTextInspector::GetSourceString(Tool.GetToolInfo().ToolDisplayName);
	return ToolName ? *ToolName : FString(TEXT("<Invalid ToolName>"));
}

bool bAllowNonEditableTargets = false;
static FAutoConsoleVariableRef CVarAllowNonEditableComponents(
	TEXT("MeshTerrainMode.AllowNonEditableTargets"),
	bAllowNonEditableTargets,
	TEXT("Determines whether Mesh Terrain Mode allows tools to be run on internal components that do not show up in the component outliner."));
}

UMeshTerrainMode::UMeshTerrainMode()
{
	Info = FEditorModeInfo(
		EM_MeshTerrainEditorModeId,
		LOCTEXT("MeshTerrainModeName", "Mesh Terrain"),
		FSlateIcon("MeshTerrainModeStyle", "LevelEditor.MeshTerrainMode", "LevelEditor.MeshTerrainMode.Small"),
		true,
		5020);
}

UMeshTerrainMode::UMeshTerrainMode(FVTableHelper& Helper)
	: UBaseLegacyWidgetEdMode(Helper)
{
}

UMeshTerrainMode::~UMeshTerrainMode()
{
}

bool UMeshTerrainMode::ProcessEditDelete()
{
	if (GetSelectionManager() && GetSelectionManager()->HasSelection())
	{
		if ( UDeleteGeometrySelectionCommand* DeleteCommand = Cast<UDeleteGeometrySelectionCommand>(ModelingModeCommands[0]) )
		{
			GetSelectionManager()->ExecuteSelectionCommand(DeleteCommand);
			return true;
		}
	}

	if (UEdMode::ProcessEditDelete())
	{
		return true;
	}

	// for now we disable deleting in an Accept-style tool because it can result in crashes if we are deleting target object
	if ( GetToolManager()->HasAnyActiveTool() && GetToolManager()->GetActiveTool(EToolSide::Mouse)->HasAccept() )
	{
		GetToolManager()->DisplayMessage(
			LOCTEXT("CannotDeleteWarning", "Cannot delete objects while this Tool is active"), EToolMessageLevel::UserWarning);
		return true;
	}

	return false;
}


bool UMeshTerrainMode::ProcessEditCut()
{
	// for now we disable deleting in an Accept-style tool because it can result in crashes if we are deleting target object
	if (GetToolManager()->HasAnyActiveTool() && GetToolManager()->GetActiveTool(EToolSide::Mouse)->HasAccept())
	{
		GetToolManager()->DisplayMessage(
			LOCTEXT("CannotCutWarning", "Cannot cut objects while this Tool is active"), EToolMessageLevel::UserWarning);
		return true;
	}

	return false;
}


EEditAction::Type UMeshTerrainMode::GetActionEditDuplicate()
{
	// Block duplication of the tool's target actors while a tool is active
	if (!GetToolManager()->HasAnyActiveTool() || ActiveToolTargetActors.IsEmpty())
	{
		return EEditAction::Skip;
	}

	if (USelection* const SelectedActors = GEditor->GetSelectedActors())
	{
		TArray<AActor*> Selected;
		SelectedActors->GetSelectedObjects<AActor>(Selected);
		for (AActor* const Actor : Selected)
		{
			if (ActiveToolTargetActors.Contains(Actor))
			{
				return EEditAction::Halt;
			}
		}
	}
	return EEditAction::Skip;
}


void UMeshTerrainMode::ActorSelectionChangeNotify()
{
	// would like to clear selection here, but this is called multiple times, including after a transaction when
	// we cannot identify that the selection should not be cleared
}


bool UMeshTerrainMode::CanAutoSave() const
{
	// prevent autosave if any tool is active
	return GetToolManager()->HasAnyActiveTool() == false;
}

bool UMeshTerrainMode::ShouldDrawWidget() const
{
	// hide standard xform gizmo if we have an active tool, unless it explicitly opts in via the IInteractiveToolEditorGizmoAPI
	if (GetInteractiveToolsContext() != nullptr && GetToolManager()->HasAnyActiveTool())
	{
		IInteractiveToolEditorGizmoAPI* GizmoAPI = Cast<IInteractiveToolEditorGizmoAPI>(GetToolManager()->GetActiveTool(EToolSide::Left));
		if (!GizmoAPI || !GizmoAPI->GetAllowStandardEditorGizmos())
		{
			return false;
		}

		// Even if the tool opts in to editor gizmos, try to hide the widget for mesh partition actors/components they're actively sculpting.
		// Covers modifier components, AModifierActor, AMeshPartition, and actors hosting UMeshProviderModifier components.
		// The gizmo still appears when they select something else (e.g. a reference object).
		if (GEditor)
		{
			TArray<UE::MeshPartition::UModifierComponent*> SelectedModifiers;
			GEditor->GetSelectedComponents()->GetSelectedObjects<UE::MeshPartition::UModifierComponent>(SelectedModifiers);
			if (SelectedModifiers.Num() > 0)
			{
				return false;
			}

			TArray<UE::MeshPartition::AModifierActor*> SelectedModifierActors;
			GEditor->GetSelectedActors()->GetSelectedObjects<UE::MeshPartition::AModifierActor>(SelectedModifierActors);
			if (SelectedModifierActors.Num() > 0)
			{
				return false;
			}

			TArray<UE::MeshPartition::AMeshPartition*> SelectedPartitionActors;
			GEditor->GetSelectedActors()->GetSelectedObjects<UE::MeshPartition::AMeshPartition>(SelectedPartitionActors);
			if (SelectedPartitionActors.Num() > 0)
			{
				return false;
			}

			// Base sections are spawned as plain AActors that own a UModifierComponent
			TArray<AActor*> SelectedActors;
			GEditor->GetSelectedActors()->GetSelectedObjects<AActor>(SelectedActors);
			for (const AActor* const SelectedActor : SelectedActors)
			{
				if (SelectedActor && SelectedActor->FindComponentByClass<UE::MeshPartition::UModifierComponent>())
				{
					return false;
				}
			}
		}
	}

	// hide standard xform gizmo if we have an active selection
	if (GetSelectionManager() && (GetSelectionManager()->HasSelection() && GetSelectionManager()->GetMeshTopologyMode() != UGeometrySelectionManager::EMeshTopologyMode::None))
	{
		return false;
	}

	return UBaseLegacyWidgetEdMode::ShouldDrawWidget();
}

bool UMeshTerrainMode::HandleClick(FEditorViewportClient* ViewportClient, HHitProxy* HitProxy, const FViewportClick& InClick)
{
	if (InClick.GetKey() == EKeys::RightMouseButton)
	{
		// The following code must be done before handling the right click.
		// We are just gathering all of the info that might be used by handlers.
		
		if (!ViewportClient || !ViewportClient->Viewport)
		{
			return false;
		}

		const UWorld* World = ViewportClient->GetWorld();
		if (!World)
		{
			return false;
		}

		FSceneViewFamilyContext ViewFamily(FSceneViewFamily::ConstructionValues(
			ViewportClient->Viewport,
			ViewportClient->GetScene(),
			ViewportClient->EngineShowFlags)
		);
			
		const FSceneView* SceneView = ViewportClient->CalcSceneView(&ViewFamily);
		if (!SceneView)
		{
			return false;
		}

		const FVector2D ScreenPos(InClick.GetClickPos().X, InClick.GetClickPos().Y);
		FVector RayOrigin, RayDirection;
		SceneView->DeprojectFVector2D(ScreenPos, RayOrigin, RayDirection);

		constexpr float TraceDistance = 10000.0f;
		const FVector TraceStart = RayOrigin;
		const FVector TraceEnd = TraceStart + (RayDirection * TraceDistance);

		FHitResult HitResult;
		World->LineTraceSingleByChannel(HitResult, TraceStart, TraceEnd, ECC_Visibility);
		
		// This is the block where custom RMB handling is actually done.
		if (FMeshTerrainModeToolkit* ModelingToolkit = static_cast<FMeshTerrainModeToolkit*>(Toolkit.Get()))
		{
			bool bClickHandled = false;
			
			// Utility function for abstracting away menu builder instantiation and spawning the appropriate right click menu.
			auto BuildMenuWrapper = [ModelingToolkit, &bClickHandled](const TFunctionRef<bool(FMenuBuilder&)>& BuildMenuFunc)
			{
				if (bClickHandled)
				{
					return;
				}
				
				FMenuBuilder MenuBuilder(true, ModelingToolkit->GetToolkitCommands());

				if (BuildMenuFunc(MenuBuilder))
				{
					FSlateApplication::Get().PushMenu(
						FSlateApplication::Get().GetActiveTopLevelWindow().ToSharedRef(), 
						FWidgetPath(), 
						MenuBuilder.MakeWidget(), 
						FSlateApplication::Get().GetCursorPos(), 
						FPopupTransitionEffect(FPopupTransitionEffect::None)
					);
				
					bClickHandled = true;
				}
			};

			/**
			 * For the 3 cases below, there is a fixed pattern:
			 * 1) Define a lambda of signature bool(FMenuBuilder&).
			 *    - This lambda should return true if it mutates the builder at all (which will consume the click).
			 * 2) Invoke BuildMenuWrapper and pass the lambda as an argument.
			 * 3) Return true from this function if BuildMenuWrapper returns true.
			 */
			
			// Case 1: Allow the toolkit to handle the right click.
			{
				auto BuildMenu = [&HitResult, ModelingToolkit](FMenuBuilder& InBuilder)
				{
					FClickContext ClickContext;
					ClickContext.HitResult = &HitResult;
					ClickContext.Builder = &InBuilder;
					
					return ModelingToolkit->HandleClick(ClickContext);
				};
			
				BuildMenuWrapper(BuildMenu);
			}

			// Case 2: Present the user with some selection options if MES is enabled and we have a target
			if (SelectionManager && SelectionManager->HasActiveTargets())
			{
				auto BuildMenu = [this, TraceStart, RayDirection](FMenuBuilder& InBuilder)
				{
					const FRay3d WorldRay(TraceStart, RayDirection);
					FInputRayHit OutResult;

					// Note: This hit test is not against anything in the world, just against the current selection targets.
					if (SelectionManager->RayHitTest(WorldRay, OutResult))
					{
						// todo: this should probably be refactored into some shared function which populates a menu with MES options
						InBuilder.AddMenuEntry(FMeshTerrainModeManagerCommands::Get().MeshSelectionModeAction_NoSelection);
						InBuilder.AddSeparator();
						InBuilder.AddMenuEntry(FMeshTerrainModeManagerCommands::Get().MeshSelectionModeAction_MeshVertices);
						InBuilder.AddMenuEntry(FMeshTerrainModeManagerCommands::Get().MeshSelectionModeAction_MeshEdges);
						InBuilder.AddMenuEntry(FMeshTerrainModeManagerCommands::Get().MeshSelectionModeAction_MeshTriangles);
						InBuilder.AddSeparator();
						InBuilder.AddMenuEntry(FMeshTerrainModeManagerCommands::Get().MeshSelectionModeAction_GroupCorners);
						InBuilder.AddMenuEntry(FMeshTerrainModeManagerCommands::Get().MeshSelectionModeAction_GroupEdges);
						InBuilder.AddMenuEntry(FMeshTerrainModeManagerCommands::Get().MeshSelectionModeAction_GroupFaces);
            
						return true;
					}

					return false;
				};
				
				BuildMenuWrapper(BuildMenu);
			}

			// Case 3: Present the user with some quick actions if MES is enabled and we have a selection
			if (SelectionManager && SelectionManager->HasSelection())
			{
				auto BuildMenu = [](FMenuBuilder& InBuilder)
				{
					InBuilder.AddMenuEntry(FMeshTerrainModeManagerCommands::Get().BeginSelectionAction_Extrude);
					InBuilder.AddMenuEntry(FMeshTerrainModeManagerCommands::Get().BeginSelectionAction_Offset);
					InBuilder.AddMenuEntry(FMeshTerrainModeManagerCommands::Get().BeginPolyModelTool_Bevel);

					return true;
				};

				BuildMenuWrapper(BuildMenu);
			}

			if (bClickHandled)
			{
				return true;
			}
		}
	}
	
	return UBaseLegacyWidgetEdMode::HandleClick(ViewportClient, HitProxy, InClick);
}

void UMeshTerrainMode::Tick(FEditorViewportClient* ViewportClient, float DeltaTime)
{
	Super::Tick(ViewportClient, DeltaTime);

	if (Toolkit.IsValid())
	{
		FMeshTerrainModeToolkit* ModelingToolkit = (FMeshTerrainModeToolkit*)Toolkit.Get();
		ModelingToolkit->ShowRealtimeAndModeWarnings(ViewportClient->IsRealtime() == false);
	}
	
	// Recover from invalid selection state, in case objects have been deleted from under the selection manager
	if (SelectionManager && !SelectionManager->ValidateSelectionState())
	{
		SelectionManager->ClearSelection();
		SelectionManager->ClearActiveTargets();
	}
}



void UMeshTerrainMode::Enter()
{
	using namespace MeshTerrainModeLocals;

	UEdMode::Enter();
	
	Owner->OnEditorModeIDChanged().AddUObject(this, &UMeshTerrainMode::OnEditorModeIDChanged);

	UMeshTerrainModeSettings* ModelingModeSettings = GetMutableDefault<UMeshTerrainModeSettings>();
	const UMeshTerrainModeCustomizationSettings* ModelingEditorSettings = GetDefault<UMeshTerrainModeCustomizationSettings>();

	// Register builders for tool targets that the mode uses.
	GetInteractiveToolsContext()->TargetManager->AddTargetFactory(NewObject<UStaticMeshComponentToolTargetFactory>(GetToolManager()));
	GetInteractiveToolsContext()->TargetManager->AddTargetFactory(NewObject<UVolumeComponentToolTargetFactory>(GetToolManager()));
	GetInteractiveToolsContext()->TargetManager->AddTargetFactory(NewObject<UE::MeshPartition::UMultiSectionToolTargetFactory>(GetToolManager()));
	GetInteractiveToolsContext()->TargetManager->AddTargetFactory(NewObject<UE::MeshPartition::USectionToolTargetFactory>(GetToolManager()));
	GetInteractiveToolsContext()->TargetManager->AddTargetFactory(NewObject<UE::MeshPartition::UMeshPartitionToolTargetFactory>(GetToolManager()));
	GetInteractiveToolsContext()->TargetManager->AddTargetFactory(NewObject<UE::MeshPartition::UModifierToolTargetFactory>(GetToolManager()));
	GetInteractiveToolsContext()->TargetManager->AddTargetFactory(NewObject<UE::MeshPartition::UEditableModifierToolTargetFactory>(GetToolManager()));
	GetInteractiveToolsContext()->TargetManager->AddTargetFactory(NewObject<UDynamicMeshComponentToolTargetFactory>(GetToolManager()));
	
	GetInteractiveToolsContext()->TargetManager->SetInputFilterFunction([](const UObject* InputObject)->bool 
	{
		if (bAllowNonEditableTargets)
		{
			return true;
		}
			
		// This check for editability is pretty much the same as the filtering that happens for component outliner in
		//  ShouldAddInstancedActorComponent inside SSCSEditor::BuildSubTreeForActorNode
		const UActorComponent* ActorComp = Cast<UActorComponent>(InputObject);
		const USceneComponent* SceneComp = Cast<USceneComponent>(ActorComp);
		const USceneComponent* ParentSceneComp = SceneComp != nullptr ? SceneComp->GetAttachParent() : nullptr;
		// Only consider the attach parent for the editability check if it's on the same actor, not for the actor-attached-to-actor case
		if (ParentSceneComp != nullptr && ActorComp != nullptr && ParentSceneComp->GetOwner() != ActorComp->GetOwner())
		{
			ParentSceneComp = nullptr;
		}
		if (FComponentEditorUtils::CanEditComponentInstance(ActorComp, ParentSceneComp,/*bAllowUserContructionScript*/ true))
		{
			return true;
		}
		return false;
	});

	// Register read-only skeletal mesh tool targets. Currently tools that write to meshes risk breaking
	// skin weights.
	GetInteractiveToolsContext()->TargetManager->AddTargetFactory(NewObject<USkeletalMeshComponentReadOnlyToolTargetFactory>(GetToolManager()));

	// Register builders for the generic component tool target, to support tools that only need the primitive component (e.g. the transform tool)
	GetInteractiveToolsContext()->TargetManager->AddTargetFactory(NewObject<UPrimitiveComponentToolTargetFactory>(GetToolManager()));

	// listen to post-build
	GetToolManager()->OnToolPostBuild.AddUObject(this, &UMeshTerrainMode::OnToolPostBuild);

	// forward shutdown requests
	GetToolManager()->OnToolShutdownRequest.BindLambda([this](UInteractiveToolManager*, UInteractiveTool* Tool, EToolShutdownType ShutdownType)
	{
		GetInteractiveToolsContext()->EndTool(ShutdownType); 
		return true;
	});

	// register for OnRender and OnDrawHUD extensions
	GetInteractiveToolsContext()->OnRender.AddUObject(this, &UMeshTerrainMode::OnToolsContextRender);
	GetInteractiveToolsContext()->OnDrawHUD.AddUObject(this, &UMeshTerrainMode::OnToolsContextDrawHUD);

	// register stylus event handler
	IToolStylusStateProviderAPI* StylusAPI = nullptr;
	if (ensure(Toolkit.IsValid()))
	{
		StylusAPI = ((FMeshTerrainModeToolkit*)Toolkit.Get())->GetStylusStateProviderAPI();
	}

	// register gizmo helper
	UE::TransformGizmoUtil::RegisterTransformGizmoContextObject(GetInteractiveToolsContext());
	// configure mode-level Gizmo options
	if (ModelingModeSettings)
	{
		GetInteractiveToolsContext()->SetForceCombinedGizmoMode(ModelingModeSettings->bRespectLevelEditorGizmoMode == false);
		GetInteractiveToolsContext()->SetAbsoluteWorldSnappingEnabled(ModelingModeSettings->bEnableAbsoluteWorldSnapping);
	}

	// Now that we have the gizmo helper, bind the numerical UI.
	if (ensure(Toolkit.IsValid()))
	{
		((FMeshTerrainModeToolkit*)Toolkit.Get())->BindGizmoNumericalUI();
	}
	
	// register snapping manager
	UE::Geometry::RegisterSceneSnappingManager(GetInteractiveToolsContext());
	SceneSnappingManager = UE::Geometry::FindModelingSceneSnappingManager(GetToolManager());

	// register tool shutdown button customizer
	if (ensure(Toolkit.IsValid()))
	{
		UMeshTerrainModeHostCustomizationAPI::Register(GetInteractiveToolsContext(), 
			StaticCastSharedRef<FMeshTerrainModeToolkit>(Toolkit.ToSharedRef()));
	}

	// Ensure the ModeManager has initialized a focused viewport client otherwise the tool manager will not Tick.
	FEditorModeTools& ModeManager = GLevelEditorModeTools();
	if (ModeManager.GetFocusedViewportClient() == nullptr && GEditor)
	{
		if (FViewport* ActiveViewport = GEditor->GetActiveViewport())
		{
			ModeManager.ReceivedFocus(static_cast<FEditorViewportClient*>(ActiveViewport->GetClient()), ActiveViewport);
		}
	}

	// register level objects observer that will update the snapping manager as the scene changes
	LevelObjectsObserver = MakeShared<FLevelObjectsObserver>();
	LevelObjectsObserver->OnActorAdded.AddLambda([this](AActor* Actor)
	{
		if (SceneSnappingManager)
		{
			SceneSnappingManager->OnActorAdded(Actor, [](UPrimitiveComponent*) { return true; });
		}
	});
	LevelObjectsObserver->OnActorRemoved.AddLambda([this](AActor* Actor)
	{
		if (SceneSnappingManager)
		{
			SceneSnappingManager->OnActorRemoved(Actor);
		}
	});
	// tracker will auto-populate w/ the current level, but must have registered the handlers first!
	LevelObjectsObserver->Initialize(GetWorld());

	// disable HitProxy rendering, it is not used in Modeling Mode and adds overhead to Render() calls
	GetInteractiveToolsContext()->SetEnableRenderingDuringHitProxyPass(false);

	// register object creation api
	UEditorModelingObjectsCreationAPI* ModelCreationAPI = UEditorModelingObjectsCreationAPI::Register(GetInteractiveToolsContext());
	if (ModelCreationAPI)
	{
		ModelCreationAPI->GetNewAssetPathNameCallback.BindLambda([](const FString& BaseName, const UWorld* TargetWorld, FString SuggestedFolder)
		{
			return UE::MeshTerrain::GetNewAssetPathName(BaseName, TargetWorld, SuggestedFolder);
		});
		MeshCreatedEventHandle = ModelCreationAPI->OnModelingMeshCreated.AddLambda([this](const FCreateMeshObjectResult& CreatedInfo) 
		{
			if (CreatedInfo.NewAsset != nullptr)
			{
				UE::MeshTerrain::OnNewAssetCreated(CreatedInfo.NewAsset);
				// If we are creating a new asset or component, it should be initially unlocked in the Selection system.
				// Currently have no generic way to do this, the Selection Manager does not necessarily support Static Meshes
				// or Brush Components. So doing it here...
				if (UStaticMesh* StaticMesh = Cast<UStaticMesh>(CreatedInfo.NewAsset))
				{
					FStaticMeshSelector::SetAssetUnlockedOnCreation(StaticMesh);
				}
			}
			if ( UBrushComponent* BrushComponent = Cast<UBrushComponent>(CreatedInfo.NewComponent) )
			{
				FVolumeSelector::SetComponentUnlockedOnCreation(BrushComponent);
			}
		});
		TextureCreatedEventHandle = ModelCreationAPI->OnModelingTextureCreated.AddLambda([](const FCreateTextureObjectResult& CreatedInfo)
		{
			if (CreatedInfo.NewAsset != nullptr)
			{
				UE::MeshTerrain::OnNewAssetCreated(CreatedInfo.NewAsset);
			}
		});
		MaterialCreatedEventHandle = ModelCreationAPI->OnModelingMaterialCreated.AddLambda([](const FCreateMaterialObjectResult& CreatedInfo)
		{
			if (CreatedInfo.NewAsset != nullptr)
			{
				UE::MeshTerrain::OnNewAssetCreated(CreatedInfo.NewAsset);
			}
		});
	}

	const FMeshTerrainModeManagerCommands& ToolManagerCommands = FMeshTerrainModeManagerCommands::Get();

	// register tool set

	//
	// primitive tools
	//
	auto RegisterPrimitiveToolFunc  =
		[this](TSharedPtr<FUICommandInfo> UICommand,
								  FString&& ToolIdentifier,
								  UAddPrimitiveToolBuilder::EMakeMeshShapeType ShapeTypeIn)
	{
		auto AddPrimitiveToolBuilder = NewObject<UAddPrimitiveToolBuilder>();
		AddPrimitiveToolBuilder->ShapeType = ShapeTypeIn;
		RegisterTool(UICommand, ToolIdentifier, AddPrimitiveToolBuilder);
	};
	RegisterPrimitiveToolFunc(ToolManagerCommands.BeginAddBoxPrimitiveTool,
							  TEXT("BeginAddBoxPrimitiveTool"),
							  UAddPrimitiveToolBuilder::EMakeMeshShapeType::Box);
	RegisterPrimitiveToolFunc(ToolManagerCommands.BeginAddCylinderPrimitiveTool,
							  TEXT("BeginAddCylinderPrimitiveTool"),
							  UAddPrimitiveToolBuilder::EMakeMeshShapeType::Cylinder);
	RegisterPrimitiveToolFunc(ToolManagerCommands.BeginAddCapsulePrimitiveTool,
	                          TEXT("BeginAddCapsulePrimitiveTool"),
	                          UAddPrimitiveToolBuilder::EMakeMeshShapeType::Capsule);
	RegisterPrimitiveToolFunc(ToolManagerCommands.BeginAddConePrimitiveTool,
							  TEXT("BeginAddConePrimitiveTool"),
							  UAddPrimitiveToolBuilder::EMakeMeshShapeType::Cone);
	RegisterPrimitiveToolFunc(ToolManagerCommands.BeginAddArrowPrimitiveTool,
							  TEXT("BeginAddArrowPrimitiveTool"),
							  UAddPrimitiveToolBuilder::EMakeMeshShapeType::Arrow);
	RegisterPrimitiveToolFunc(ToolManagerCommands.BeginAddRectanglePrimitiveTool,
							  TEXT("BeginAddRectanglePrimitiveTool"),
							  UAddPrimitiveToolBuilder::EMakeMeshShapeType::Rectangle);
	RegisterPrimitiveToolFunc(ToolManagerCommands.BeginAddDiscPrimitiveTool,
							  TEXT("BeginAddDiscPrimitiveTool"),
							  UAddPrimitiveToolBuilder::EMakeMeshShapeType::Disc);
	RegisterPrimitiveToolFunc(ToolManagerCommands.BeginAddTorusPrimitiveTool,
							  TEXT("BeginAddTorusPrimitiveTool"),
							  UAddPrimitiveToolBuilder::EMakeMeshShapeType::Torus);
	RegisterPrimitiveToolFunc(ToolManagerCommands.BeginAddSpherePrimitiveTool,
							  TEXT("BeginAddSpherePrimitiveTool"),
							  UAddPrimitiveToolBuilder::EMakeMeshShapeType::Sphere);
	RegisterPrimitiveToolFunc(ToolManagerCommands.BeginAddStairsPrimitiveTool,
							  TEXT("BeginAddStairsPrimitiveTool"),
							  UAddPrimitiveToolBuilder::EMakeMeshShapeType::Stairs);

	//
	// make shape tools
	//
	auto AddPatchToolBuilder = NewObject<UAddPatchToolBuilder>();
	RegisterTool(ToolManagerCommands.BeginAddPatchTool, TEXT("BeginAddPatchTool"), AddPatchToolBuilder);

	auto RevolveBoundaryToolBuilder = NewObject<URevolveBoundaryToolBuilder>();
	RegisterTool(ToolManagerCommands.BeginRevolveBoundaryTool, TEXT("BeginRevolveBoundaryTool"), RevolveBoundaryToolBuilder);

	auto DrawPolygonToolBuilder = NewObject<UDrawPolygonToolBuilder>();
	RegisterTool(ToolManagerCommands.BeginDrawPolygonTool, TEXT("BeginDrawPolygonTool"), DrawPolygonToolBuilder);

	auto DrawPolyPathToolBuilder = NewObject<UDrawPolyPathToolBuilder>();
	RegisterTool(ToolManagerCommands.BeginDrawPolyPathTool, TEXT("BeginDrawPolyPathTool"), DrawPolyPathToolBuilder);

	auto DrawAndRevolveToolBuilder = NewObject<UDrawAndRevolveToolBuilder>();
	RegisterTool(ToolManagerCommands.BeginDrawAndRevolveTool, TEXT("BeginDrawAndRevolveTool"), DrawAndRevolveToolBuilder);

	auto RevolveSplineToolBuilder = NewObject<URevolveSplineToolBuilder>();
	RegisterTool(ToolManagerCommands.BeginRevolveSplineTool, TEXT("BeginRevolveSplineTool"), RevolveSplineToolBuilder);

	auto ShapeSprayToolBuilder = NewObject<UShapeSprayToolBuilder>();
	RegisterTool(ToolManagerCommands.BeginShapeSprayTool, TEXT("BeginShapeSprayTool"), ShapeSprayToolBuilder);

	auto CubeGridToolBuilder = NewObject<UCubeGridToolBuilder>();
	RegisterTool(ToolManagerCommands.BeginCubeGridTool, TEXT("BeginCubeGridTool"), CubeGridToolBuilder);

	auto DrawSplineToolBuilder = NewObject<UDrawSplineToolBuilder>();
	RegisterTool(ToolManagerCommands.BeginDrawSplineTool, TEXT("BeginDrawSplineTool"), DrawSplineToolBuilder);

	auto TriangulateSplinesToolBuilder = NewObject<UTriangulateSplinesToolBuilder>();
	RegisterTool(ToolManagerCommands.BeginTriangulateSplinesTool, TEXT("BeginTriangulateSplinesTool"), TriangulateSplinesToolBuilder);

	//
	// vertex deform tools
	//

	auto MoveVerticesToolBuilder = NewObject<UE::MeshPartition::UHeightSculptToolBuilder>();
	MoveVerticesToolBuilder->StylusAPI = StylusAPI;
	RegisterTool(ToolManagerCommands.BeginSculptMeshTool, TEXT("BeginSculptMeshTool"), MoveVerticesToolBuilder);

	auto MeshGroupPaintToolBuilder = NewObject<UMeshGroupPaintToolBuilder>();
	MeshGroupPaintToolBuilder->StylusAPI = StylusAPI;
	RegisterTool(ToolManagerCommands.BeginMeshGroupPaintTool, TEXT("BeginMeshGroupPaintTool"), MeshGroupPaintToolBuilder);
	UMeshVertexPaintToolBuilder* MeshVertexPaintToolBuilder = NewObject<UMeshVertexPaintToolBuilder>();
	MeshVertexPaintToolBuilder->StylusAPI = StylusAPI;
	RegisterTool(ToolManagerCommands.BeginMeshVertexPaintTool, TEXT("BeginMeshVertexPaintTool"), MeshVertexPaintToolBuilder);

	// register tools w brushes
	auto RegisterHeightSculptBrushTool = [this, StylusAPI](const TSharedPtr<FUICommandInfo>& CommandInfo, const FString& ToolIdentifier, int32 BrushID,
		const TFunction<bool(UInteractiveToolManager*, EToolSide)>& CanExecuteAction = nullptr)
	{
		UE::MeshPartition::UHeightSculptToolBuilder* SculptBrushToolBuilder = NewObject<UE::MeshPartition::UHeightSculptToolBuilder>();
		SculptBrushToolBuilder->DefaultPrimaryBrushID = BrushID;
		SculptBrushToolBuilder->StylusAPI = StylusAPI;
		RegisterCustomTool(
			CommandInfo,
			ToolIdentifier,
			SculptBrushToolBuilder,
			// ExecuteAction
			[this, ToolIdentifier, BrushID](UInteractiveToolManager* ToolManager, EToolSide ToolSide)
			{
				if (ToolManager)
				{
					// If the active tool is a MeshPartition::UHeightSculptTool, change the active brush type instead of launching a new tool instance.
					if (UE::MeshPartition::UHeightSculptTool* BrushTool = ExactCast<UE::MeshPartition::UHeightSculptTool>(ToolManager->GetActiveTool(ToolSide)))
					{
						ToolManager->SelectActiveToolType(EToolSide::Left, ToolIdentifier);
						BrushTool->SetActiveBrushType(BrushID);
						LastToolIdentifier = ToolIdentifier;
						return true;
					}
				}
				return false;
			},
			CanExecuteAction,
			nullptr // IsActionChecked
		);
	};

	RegisterHeightSculptBrushTool(ToolManagerCommands.BeginSculptMeshMoveBrushTool, TEXT("BeginSculptMeshMoveBrushTool"), (int32)EMeshVertexSculptBrushType::Move);
	RegisterHeightSculptBrushTool(ToolManagerCommands.BeginSculptMeshPullKelvinBrushTool, TEXT("BeginSculptMeshPullKelvinBrushTool"), (int32)EMeshVertexSculptBrushType::PullKelvin);
	RegisterHeightSculptBrushTool(ToolManagerCommands.BeginSculptMeshPullSharpKelvinBrushTool, TEXT("BeginSculptMeshPullSharpKelvinBrushTool"), (int32)EMeshVertexSculptBrushType::PullSharpKelvin);
	RegisterHeightSculptBrushTool(ToolManagerCommands.BeginSculptMeshSmoothBrushTool, TEXT("BeginSculptMeshSmoothBrushTool"), (int32)EMeshVertexSculptBrushType::Smooth);
	RegisterHeightSculptBrushTool(ToolManagerCommands.BeginSculptMeshSmoothFillBrushTool, TEXT("BeginSculptMeshSmoothFillBrushTool"), (int32)EMeshVertexSculptBrushType::SmoothFill);
	RegisterHeightSculptBrushTool(ToolManagerCommands.BeginSculptMeshOffsetBrushTool, TEXT("BeginSculptMeshOffsetBrushTool"), (int32)EMeshVertexSculptBrushType::Offset);
	RegisterHeightSculptBrushTool(ToolManagerCommands.BeginSculptMeshSculptViewBrushTool, TEXT("BeginSculptMeshSculptViewBrushTool"), (int32)EMeshVertexSculptBrushType::SculptView);
	RegisterHeightSculptBrushTool(ToolManagerCommands.BeginSculptMeshSculptMaxBrushTool, TEXT("BeginSculptMeshSculptMaxBrushTool"), (int32)EMeshVertexSculptBrushType::SculptMax);
	RegisterHeightSculptBrushTool(ToolManagerCommands.BeginSculptMeshInflateBrushTool, TEXT("BeginSculptMeshInflateBrushTool"), (int32)EMeshVertexSculptBrushType::Inflate);
	RegisterHeightSculptBrushTool(ToolManagerCommands.BeginSculptMeshScaleKelvinBrushTool, TEXT("BeginSculptMeshScaleKelvinBrushTool"), (int32)EMeshVertexSculptBrushType::ScaleKelvin);
	RegisterHeightSculptBrushTool(ToolManagerCommands.BeginSculptMeshPinchBrushTool, TEXT("BeginSculptMeshPinchBrushTool"), (int32)EMeshVertexSculptBrushType::Pinch);
	RegisterHeightSculptBrushTool(ToolManagerCommands.BeginSculptMeshTwistKelvinBrushTool, TEXT("BeginSculptMeshTwistKelvinBrushTool"), (int32)EMeshVertexSculptBrushType::TwistKelvin);
	RegisterHeightSculptBrushTool(ToolManagerCommands.BeginSculptMeshFlattenBrushTool, TEXT("BeginSculptMeshFlattenBrushTool"), (int32)EMeshVertexSculptBrushType::Flatten);
	RegisterHeightSculptBrushTool(ToolManagerCommands.BeginSculptMeshPlaneBrushTool, TEXT("BeginSculptMeshPlaneBrushTool"), (int32)EMeshVertexSculptBrushType::Plane);
	RegisterHeightSculptBrushTool(ToolManagerCommands.BeginSculptMeshPlaneViewAlignedBrushTool, TEXT("BeginSculptMeshPlaneViewAlignedBrushTool"), (int32)EMeshVertexSculptBrushType::PlaneViewAligned);
	RegisterHeightSculptBrushTool(ToolManagerCommands.BeginSculptMeshFixedPlaneBrushTool, TEXT("BeginSculptMeshFixedPlaneBrushTool"), (int32)EMeshVertexSculptBrushType::FixedPlane);
	// The erase brush can only be run if our target has sculpt layers. So we need a custom CanExecute handler
	RegisterHeightSculptBrushTool(ToolManagerCommands.BeginSculptMeshEraseLayerTool, TEXT("BeginSculptMeshEraseLayerTool"), (int32)EMeshVertexSculptBrushType::EraseSculptLayer,
		[](UInteractiveToolManager* ToolManager, EToolSide ToolSide) -> bool
	{
		if (!ToolManager)
		{
			return false;
		}
		if (UMeshVertexSculptTool* BrushTool = ExactCast<UMeshVertexSculptTool>(ToolManager->GetActiveTool(ToolSide)))
		{
			return BrushTool->DoesTargetHaveSculptLayers();
		}

		if (!ToolManager->CanActivateTool(ToolSide, TEXT("BeginSculptMeshEraseLayerTool")))
		{
			return false;
		}
		if (!ToolManager->GetContextQueriesAPI())
		{
			return false;
		}
		
		FToolBuilderState BuilderState;
		ToolManager->GetContextQueriesAPI()->GetCurrentSelectionState(BuilderState);
		
		int32 Count = ToolBuilderUtil::CountComponents(BuilderState, [](UActorComponent* Component)
		{
			if (IMeshSculptLayersManager* SculptLayersManager = Cast<IMeshSculptLayersManager>(Component))
			{
				return SculptLayersManager->HasSculptLayers();
			}
			return false;
		});
		return Count == 1;
	});
	RegisterHeightSculptBrushTool(ToolManagerCommands.BeginHeightSculptBrushTool, TEXT("BeginHeightSculptBrushTool"), (int32)UE::MeshPartition::UHeightSculptTool::EBrushType::HeightSculpt);
	RegisterHeightSculptBrushTool(ToolManagerCommands.BeginHeightSmoothBrushTool, TEXT("BeginHeightSmoothBrushTool"), (int32)UE::MeshPartition::UHeightSculptTool::EBrushType::HeightSmooth);
	RegisterHeightSculptBrushTool(ToolManagerCommands.BeginHeightFlattenBrushTool, TEXT("BeginHeightFlattenBrushTool"), (int32)UE::MeshPartition::UHeightSculptTool::EBrushType::HeightFlatten);
	RegisterHeightSculptBrushTool(ToolManagerCommands.BeginSlopeErodeBrushTool, TEXT("BeginSlopeErodeBrushTool"), (int32)UE::MeshPartition::UHeightSculptTool::EBrushType::SlopeErode);

	RegisterTool(ToolManagerCommands.BeginPolyEditTool, TEXT("BeginPolyEditTool"), NewObject<UEditMeshPolygonsToolBuilder>());
	UEditMeshPolygonsToolBuilder* TriEditBuilder = NewObject<UEditMeshPolygonsToolBuilder>();
	TriEditBuilder->bTriangleMode = true;
	RegisterTool(ToolManagerCommands.BeginTriEditTool, TEXT("BeginTriEditTool"), TriEditBuilder);
	RegisterTool(ToolManagerCommands.BeginPolyDeformTool, TEXT("BeginPolyDeformTool"), NewObject<UDeformMeshPolygonsToolBuilder>());
	RegisterTool(ToolManagerCommands.BeginSmoothMeshTool, TEXT("BeginSmoothMeshTool"), NewObject<USmoothMeshToolBuilder>());
	RegisterTool(ToolManagerCommands.BeginOffsetMeshTool, TEXT("BeginOffsetMeshTool"), NewObject<UOffsetMeshToolBuilder>());
	RegisterTool(ToolManagerCommands.BeginDisplaceMeshTool, TEXT("BeginDisplaceMeshTool"), NewObject<UDisplaceMeshToolBuilder>());
	RegisterTool(ToolManagerCommands.BeginMeshSpaceDeformerTool, TEXT("BeginMeshSpaceDeformerTool"), NewObject<UMeshSpaceDeformerToolBuilder>());
	RegisterTool(ToolManagerCommands.BeginTransformMeshesTool, TEXT("BeginTransformMeshesTool"), NewObject<UTransformMeshesToolBuilder>());
	RegisterTool(ToolManagerCommands.BeginAddPivotActorTool, TEXT("BeginAddPivotActorTool"), NewObject<UAddPivotActorToolBuilder>());
	RegisterTool(ToolManagerCommands.BeginEditPivotTool, TEXT("BeginEditPivotTool"), NewObject<UEditPivotToolBuilder>());
	RegisterTool(ToolManagerCommands.BeginAlignObjectsTool, TEXT("BeginAlignObjectsTool"), NewObject<UAlignObjectsToolBuilder>());
	RegisterTool(ToolManagerCommands.BeginTransferMeshTool, TEXT("BeginTransferMeshTool"), NewObject<UTransferMeshToolBuilder>());
	RegisterTool(ToolManagerCommands.BeginConvertMeshesTool, TEXT("BeginConvertMeshesTool"), NewObject<UConvertMeshesToolBuilder>());
	RegisterTool(ToolManagerCommands.BeginSplitMeshesTool, TEXT("BeginSplitMeshesTool"), NewObject<USplitMeshesToolBuilder>());
	RegisterTool(ToolManagerCommands.BeginBakeTransformTool, TEXT("BeginBakeTransformTool"), NewObject<UBakeTransformToolBuilder>());
	RegisterTool(ToolManagerCommands.BeginTransformUVIslandsTool, TEXT("BeginTransformUVIslandsTool"), NewObject<UEditUVIslandsToolBuilder>());
	RegisterTool(ToolManagerCommands.BeginLatticeDeformerTool, TEXT("BeginLatticeDeformerTool"), NewObject<ULatticeDeformerToolBuilder>());
	RegisterTool(ToolManagerCommands.BeginSubdividePolyTool, TEXT("BeginSubdividePolyTool"), NewObject<USubdividePolyToolBuilder>());

	// Register Quick Settings customizations
	TSharedPtr<UE::MeshTerrain::FModelingQuickPropertyCustomizations> QuickSettingsCustomizations = static_cast<FMeshTerrainModeToolkit*>(Toolkit.Get())->GetQuickSettingsCustomizations();
	QuickSettingsCustomizations->RegisterQuickPropertyCustomization(UMeshVertexSculptTool::StaticClass(), FGetQuickPropertyCustomization::CreateUObject(this, &UMeshTerrainMode::CreateVSculptToolWidgets));
	QuickSettingsCustomizations->RegisterQuickPropertyCustomization(UE::MeshPartition::UHeightSculptTool::StaticClass(), FGetQuickPropertyCustomization::CreateUObject(this, &UMeshTerrainMode::CreateVSculptToolWidgets));
	QuickSettingsCustomizations->RegisterQuickPropertyCustomization(UE::MeshPartition::UAttributePaintTool::StaticClass(), FGetQuickPropertyCustomization::CreateUObject(this, &UMeshTerrainMode::CreatePaintWeightsToolWidgets));
	QuickSettingsCustomizations->RegisterQuickPropertyStructTypeCustomization(UE::MeshPartition::FChannelName::StaticStruct(), FGetQuickPropertyCustomization::CreateStatic(&UE::MeshTerrain::CreateChannelNameWidget));

	// Register Quick Edit customizations
	TSharedPtr<UE::MeshTerrain::FModelingQuickPropertyCustomizations> QuickEditCustomizations = static_cast<FMeshTerrainModeToolkit*>(Toolkit.Get())->GetQuickEditCustomizations();
	QuickEditCustomizations->RegisterQuickPropertyCustomization(UMeshVertexSculptTool::StaticClass(), FGetQuickPropertyCustomization::CreateUObject(this, &UMeshTerrainMode::CreateVSculptToolWidgets));
	QuickEditCustomizations->RegisterQuickPropertyCustomization(UE::MeshPartition::UHeightSculptTool::StaticClass(), FGetQuickPropertyCustomization::CreateUObject(this, &UMeshTerrainMode::CreateVSculptToolWidgets));
	QuickEditCustomizations->RegisterQuickPropertyStructTypeCustomization(UE::MeshPartition::FChannelName::StaticStruct(), FGetQuickPropertyCustomization::CreateStatic(&UE::MeshTerrain::CreateChannelNameWidget));

	UPatternToolBuilder* PatternToolBuilder = NewObject<UPatternToolBuilder>();
	PatternToolBuilder->bEnableCreateISMCs = true;
	RegisterTool(ToolManagerCommands.BeginPatternTool, TEXT("BeginPatternTool"), PatternToolBuilder);
	RegisterTool(ToolManagerCommands.BeginHarvestInstancesTool, TEXT("BeginHarvestInstancesTool"), NewObject<UHarvestInstancesToolBuilder>());

	UCombineMeshesToolBuilder* CombineMeshesToolBuilder = NewObject<UCombineMeshesToolBuilder>();
	RegisterTool(ToolManagerCommands.BeginCombineMeshesTool, TEXT("BeginCombineMeshesTool"), CombineMeshesToolBuilder);

	UCombineMeshesToolBuilder* DuplicateMeshesToolBuilder = NewObject<UCombineMeshesToolBuilder>();
	DuplicateMeshesToolBuilder->bIsDuplicateTool = true;
	RegisterTool(ToolManagerCommands.BeginDuplicateMeshesTool, TEXT("BeginDuplicateMeshesTool"), DuplicateMeshesToolBuilder);


	ULODManagerToolBuilder* LODManagerToolBuilder = NewObject<ULODManagerToolBuilder>();
	RegisterTool(ToolManagerCommands.BeginLODManagerTool, TEXT("BeginLODManagerTool"), LODManagerToolBuilder);

	UGenerateStaticMeshLODAssetToolBuilder* GenerateSMLODToolBuilder = NewObject<UGenerateStaticMeshLODAssetToolBuilder>();
	GenerateSMLODToolBuilder->bInRestrictiveMode = ModelingModeSettings && ModelingModeSettings->InRestrictiveMode();
	RegisterTool(ToolManagerCommands.BeginGenerateStaticMeshLODAssetTool, TEXT("BeginGenerateStaticMeshLODAssetTool"), GenerateSMLODToolBuilder);

	UISMEditorToolBuilder* ISMEditorToolBuilder = NewObject<UISMEditorToolBuilder>();
	RegisterTool(ToolManagerCommands.BeginISMEditorTool, TEXT("BeginISMEditorTool"), ISMEditorToolBuilder);


	// edit tools


	auto DynaSculptToolBuilder = NewObject<UDynamicMeshSculptToolBuilder>();
	DynaSculptToolBuilder->bEnableRemeshing = true;
	DynaSculptToolBuilder->StylusAPI = StylusAPI;
	RegisterTool(ToolManagerCommands.BeginRemeshSculptMeshTool, TEXT("BeginRemeshSculptMeshTool"), DynaSculptToolBuilder);

	RegisterTool(ToolManagerCommands.BeginRemeshMeshTool, TEXT("BeginRemeshMeshTool"), NewObject<URemeshMeshToolBuilder>());
	RegisterTool(ToolManagerCommands.BeginProjectToTargetTool, TEXT("BeginProjectToTargetTool"), NewObject<UProjectToTargetToolBuilder>());
	RegisterTool(ToolManagerCommands.BeginSimplifyMeshTool, TEXT("BeginSimplifyMeshTool"), NewObject<USimplifyMeshToolBuilder>());

	auto EditNormalsToolBuilder = NewObject<UEditNormalsToolBuilder>();
	RegisterTool(ToolManagerCommands.BeginEditNormalsTool, TEXT("BeginEditNormalsTool"), EditNormalsToolBuilder);

	RegisterTool(ToolManagerCommands.BeginEditTangentsTool, TEXT("BeginEditTangentsTool"), NewObject<UMeshTangentsToolBuilder>());

	auto RemoveOccludedTrianglesToolBuilder = NewObject<URemoveOccludedTrianglesToolBuilder>();
	RegisterTool(ToolManagerCommands.BeginRemoveOccludedTrianglesTool, TEXT("BeginRemoveOccludedTrianglesTool"), RemoveOccludedTrianglesToolBuilder);

	RegisterTool(ToolManagerCommands.BeginHoleFillTool, TEXT("BeginHoleFillTool"), NewObject<UHoleFillToolBuilder>());

	auto UVProjectionToolBuilder = NewObject<UUVProjectionToolBuilder>();
	RegisterTool(ToolManagerCommands.BeginUVProjectionTool, TEXT("BeginUVProjectionTool"), UVProjectionToolBuilder);

	auto UVLayoutToolBuilder = NewObject<UUVLayoutToolBuilder>();
	RegisterTool(ToolManagerCommands.BeginUVLayoutTool, TEXT("BeginUVLayoutTool"), UVLayoutToolBuilder);

#if WITH_PROXYLOD
	auto MergeMeshesToolBuilder = NewObject<UMergeMeshesToolBuilder>();
	RegisterTool(ToolManagerCommands.BeginVoxelMergeTool, TEXT("BeginVoxelMergeTool"), MergeMeshesToolBuilder);

	auto VoxelCSGMeshesToolBuilder = NewObject<UVoxelCSGMeshesToolBuilder>();
	RegisterTool(ToolManagerCommands.BeginVoxelBooleanTool, TEXT("BeginVoxelBooleanTool"), VoxelCSGMeshesToolBuilder);
#endif	// WITH_PROXYLOD

	auto VoxelSolidifyMeshesToolBuilder = NewObject<UVoxelSolidifyMeshesToolBuilder>();
	RegisterTool(ToolManagerCommands.BeginVoxelSolidifyTool, TEXT("BeginVoxelSolidifyTool"), VoxelSolidifyMeshesToolBuilder);

	auto VoxelBlendMeshesToolBuilder = NewObject<UVoxelBlendMeshesToolBuilder>();
	RegisterTool(ToolManagerCommands.BeginVoxelBlendTool, TEXT("BeginVoxelBlendTool"), VoxelBlendMeshesToolBuilder);

	auto VoxelMorphologyMeshesToolBuilder = NewObject<UVoxelMorphologyMeshesToolBuilder>();
	RegisterTool(ToolManagerCommands.BeginVoxelMorphologyTool, TEXT("BeginVoxelMorphologyTool"), VoxelMorphologyMeshesToolBuilder);

	auto SelfUnionMeshesToolBuilder = NewObject<USelfUnionMeshesToolBuilder>();
	RegisterTool(ToolManagerCommands.BeginSelfUnionTool, TEXT("BeginSelfUnionTool"), SelfUnionMeshesToolBuilder);

	auto CSGMeshesToolBuilder = NewObject<UCSGMeshesToolBuilder>();
	RegisterTool(ToolManagerCommands.BeginMeshBooleanTool, TEXT("BeginMeshBooleanTool"), CSGMeshesToolBuilder);

	auto CutMeshWithMeshToolBuilder = NewObject<UCutMeshWithMeshToolBuilder>();
	RegisterTool(ToolManagerCommands.BeginCutMeshWithMeshTool, TEXT("BeginCutMeshWithMeshTool"), CutMeshWithMeshToolBuilder);

	auto TrimMeshesToolBuilder = NewObject<UCSGMeshesToolBuilder>();
	TrimMeshesToolBuilder->bTrimMode = true;
	RegisterTool(ToolManagerCommands.BeginMeshTrimTool, TEXT("BeginMeshTrimTool"), TrimMeshesToolBuilder);

	// BSPConv is disabled in Restrictive Mode.
	if (ToolManagerCommands.BeginBspConversionTool)
	{
		RegisterTool(ToolManagerCommands.BeginBspConversionTool, TEXT("BeginBspConversionTool"), NewObject<UBspConversionToolBuilder>());
	}

	auto MeshToVolumeToolBuilder = NewObject<UMeshToVolumeToolBuilder>();
	RegisterTool(ToolManagerCommands.BeginMeshToVolumeTool, TEXT("BeginMeshToVolumeTool"), MeshToVolumeToolBuilder);

	auto VolumeToMeshToolBuilder = NewObject<UVolumeToMeshToolBuilder>();
	RegisterTool(ToolManagerCommands.BeginVolumeToMeshTool, TEXT("BeginVolumeToMeshTool"), VolumeToMeshToolBuilder);

	auto PlaneCutToolBuilder = NewObject<UPlaneCutToolBuilder>();
	RegisterTool(ToolManagerCommands.BeginPlaneCutTool, TEXT("BeginPlaneCutTool"), PlaneCutToolBuilder);

	auto MirrorToolBuilder = NewObject<UMirrorToolBuilder>();
	RegisterTool(ToolManagerCommands.BeginMirrorTool, TEXT("BeginMirrorTool"), MirrorToolBuilder);

	auto PolygonCutToolBuilder = NewObject<UPolygonOnMeshToolBuilder>();
	RegisterTool(ToolManagerCommands.BeginPolygonCutTool, TEXT("BeginPolygonCutTool"), PolygonCutToolBuilder);

	auto GlobalUVGenerateToolBuilder = NewObject<UParameterizeMeshToolBuilder>();
	RegisterTool(ToolManagerCommands.BeginGlobalUVGenerateTool, TEXT("BeginGlobalUVGenerateTool"), GlobalUVGenerateToolBuilder);

	auto RecomputeUVsToolBuilder = NewObject<URecomputeUVsToolBuilder>();
	RegisterTool(ToolManagerCommands.BeginGroupUVGenerateTool, TEXT("BeginGroupUVGenerateTool"), RecomputeUVsToolBuilder);

	RegisterTool(ToolManagerCommands.BeginUVSeamEditTool, TEXT("BeginUVSeamEditTool"), NewObject< USeamSculptToolBuilder>());

	RegisterUVEditor();

	auto MeshSelectionToolBuilder = NewObject<UMeshSelectionToolBuilder>();
	RegisterTool(ToolManagerCommands.BeginMeshSelectionTool, TEXT("BeginMeshSelectionTool"), MeshSelectionToolBuilder);

	auto EditMeshMaterialsToolBuilder = NewObject<UEditMeshMaterialsToolBuilder>();
	RegisterTool(ToolManagerCommands.BeginEditMeshMaterialsTool, TEXT("BeginEditMeshMaterialsTool"), EditMeshMaterialsToolBuilder);
	
	auto MeshAttributePaintToolBuilder = NewObject<UE::MeshPartition::UAttributePaintToolBuilder>();
	MeshAttributePaintToolBuilder->StylusAPI = StylusAPI;
	RegisterTool(ToolManagerCommands.BeginMeshAttributePaintTool, TEXT("BeginMeshAttributePaintTool"), MeshAttributePaintToolBuilder);

	auto BakeMeshAttributeMapsToolBuilder = NewObject<UBakeMeshAttributeMapsToolBuilder>();
	RegisterTool(ToolManagerCommands.BeginBakeMeshAttributeMapsTool, TEXT("BeginBakeMeshAttributeMapsTool"), BakeMeshAttributeMapsToolBuilder);

	auto BakeMultiMeshAttributeMapsToolBuilder = NewObject<UBakeMultiMeshAttributeMapsToolBuilder>();
	RegisterTool(ToolManagerCommands.BeginBakeMultiMeshAttributeMapsTool, TEXT("BeginBakeMultiMeshAttributeMapsTool"), BakeMultiMeshAttributeMapsToolBuilder);

	auto BakeRenderCaptureToolBuilder = NewObject<UBakeRenderCaptureToolBuilder>();
	RegisterTool(ToolManagerCommands.BeginBakeRenderCaptureTool, TEXT("BeginBakeRenderCaptureTool"), BakeRenderCaptureToolBuilder);

	auto BakeMeshAttributeVertexToolBuilder = NewObject<UBakeMeshAttributeVertexToolBuilder>();
	RegisterTool(ToolManagerCommands.BeginBakeMeshAttributeVertexTool, TEXT("BeginBakeMeshAttributeVertexTool"), BakeMeshAttributeVertexToolBuilder);

	// analysis tools

	RegisterTool(ToolManagerCommands.BeginMeshInspectorTool, TEXT("BeginMeshInspectorTool"), NewObject<UMeshInspectorToolBuilder>());
	RegisterTool(ToolManagerCommands.BeginWeldEdgesTool, TEXT("BeginWeldEdgesTool"), NewObject<UWeldMeshEdgesToolBuilder>());
	RegisterTool(ToolManagerCommands.BeginPolyGroupsTool, TEXT("BeginPolyGroupsTool"), NewObject<UConvertToPolygonsToolBuilder>());
	RegisterTool(ToolManagerCommands.BeginAttributeEditorTool, TEXT("BeginAttributeEditorTool"), NewObject<UAttributeEditorToolBuilder>());


	// Physics Tools

	RegisterTool(ToolManagerCommands.BeginPhysicsInspectorTool, TEXT("BeginPhysicsInspectorTool"), NewObject<UPhysicsInspectorToolBuilder>());
	RegisterTool(ToolManagerCommands.BeginSimpleCollisionEditorTool, TEXT("BeginSimpleCollisionEditorTool"), NewObject<USimpleCollisionEditorToolBuilder>());
	RegisterTool(ToolManagerCommands.BeginSetCollisionGeometryTool, TEXT("BeginSetCollisionGeometryTool"), NewObject<USetCollisionGeometryToolBuilder>());

	auto ExtractCollisionGeoToolBuilder = NewObject<UExtractCollisionGeometryToolBuilder>();
	RegisterTool(ToolManagerCommands.BeginExtractCollisionGeometryTool, TEXT("BeginExtractCollisionGeometryTool"), ExtractCollisionGeoToolBuilder);


	// MegaMesh Tools
	RegisterTool(ToolManagerCommands.BeginConvertMegaMeshTool, TEXT("BeginConvertMegaMeshTool"), NewObject<UE::MeshPartition::UConvertToolBuilder>());
	RegisterTool(ToolManagerCommands.BeginSplitMegaMeshTool, TEXT("BeginSplitMegaMeshTool"), NewObject<UE::MeshPartition::USplitToolBuilder>());
	RegisterTool(ToolManagerCommands.BeginMergeMegaMeshTool, TEXT("BeginMergeMegaMeshTool"), NewObject<UE::MeshPartition::UMergeToolBuilder>());
	RegisterTool(ToolManagerCommands.BeginResectionMeshTool, TEXT("BeginResectionMeshTool"), NewObject<UE::MeshPartition::UMeshPartitionResectionToolBuilder>());
	RegisterTool(ToolManagerCommands.BeginStitchMegaMeshTool, TEXT("BeginStitchMegaMeshTool"), NewObject<UE::MeshPartition::UStitchToolBuilder>());
	RegisterTool(ToolManagerCommands.BeginHeightmapImport, TEXT("BeginHeightmapImport"), NewObject<UE::MeshPartition::UHeightmapImportToolBuilder>());
	TObjectPtr<UE::MeshPartition::UHeightSculptToolBuilder> HeightSculptToolBuilder = NewObject<UE::MeshPartition::UHeightSculptToolBuilder>();
	HeightSculptToolBuilder->StylusAPI = StylusAPI;
	RegisterTool(ToolManagerCommands.BeginHeightSculptTool, TEXT("BeginHeightSculptTool"), HeightSculptToolBuilder);
	TObjectPtr<UE::MeshPartition::UCreateMeshToolBuilder> CreateMegaMeshRectangleToolBuilder = NewObject<UE::MeshPartition::UCreateMeshToolBuilder>();
	CreateMegaMeshRectangleToolBuilder->ShapeType = UE::MeshPartition::UCreateMeshToolBuilder::EMakeMeshShapeType::Rectangle;
	RegisterTool(ToolManagerCommands.BeginCreateMegaMeshRectangleTool, TEXT("BeginCreateMegaMeshRectangleTool"), CreateMegaMeshRectangleToolBuilder);
	RegisterTool(ToolManagerCommands.BeginAddModifierTool, TEXT("BeginAddModifierTool"), NewObject<UE::MeshPartition::UPlaceModifierToolBuilder>());
	RegisterTool(ToolManagerCommands.BeginExpandMegaMeshTool, TEXT("BeginExpandMegaMeshTool"), NewObject<UE::MeshPartition::UExpandToolBuilder>());

	auto RegisterPlaceModifierTool = [this](const TSharedPtr<FUICommandInfo>& CommandInfo, const FString& ToolIdentifier, const int32 ModifierTypeID)
	{
		UE::MeshPartition::UPlaceModifierToolBuilder* PlaceModifierToolBuilder = NewObject<UE::MeshPartition::UPlaceModifierToolBuilder>();
		PlaceModifierToolBuilder->DefaultModifierTypeID = ModifierTypeID;
		RegisterCustomTool(
			CommandInfo,
			ToolIdentifier,
			PlaceModifierToolBuilder,
			// ExecuteAction
			[this, ToolIdentifier](UInteractiveToolManager* ToolManager, EToolSide ToolSide)
			{
				if (UInteractiveToolsContext* ToolContext = GetInteractiveToolsContext())
				{
					ToolContext->StartTool(ToolSide, ToolIdentifier);
					ToolContext->EndTool(ToolSide, EToolShutdownType::Accept);
				}
				return false;
			},
			nullptr, // CanExecuteAction
			nullptr // IsActionChecked
		);
	};

	RegisterPlaceModifierTool(ToolManagerCommands.BeginAddRemeshModifierTool, TEXT("BeginAddRemeshModifierTool"), static_cast<int>(UE::MeshPartition::EModifierClassType::Remesh));
	RegisterPlaceModifierTool(ToolManagerCommands.BeginAddPatchModifierTool, TEXT("BeginAddPatchModifierTool"), static_cast<int>(UE::MeshPartition::EModifierClassType::Patch));
	RegisterPlaceModifierTool(ToolManagerCommands.BeginAddProjectModifierTool, TEXT("BeginAddProjectModifierTool"), static_cast<int>(UE::MeshPartition::EModifierClassType::Project));
	RegisterPlaceModifierTool(ToolManagerCommands.BeginAddInstancedPatchModifierTool, TEXT("BeginAddInstancedPatchModifierTool"), static_cast<int>(UE::MeshPartition::EModifierClassType::InstancedPatch));
	RegisterPlaceModifierTool(ToolManagerCommands.BeginAddTexturePatchModifierTool, TEXT("BeginAddTexturePatchModifierTool"), static_cast<int>(UE::MeshPartition::EModifierClassType::TexturePatch));
	RegisterPlaceModifierTool(ToolManagerCommands.BeginAddSplineModifierTool, TEXT("BeginAddSplineModifierTool"), static_cast<int>(UE::MeshPartition::EModifierClassType::Spline));
	RegisterPlaceModifierTool(ToolManagerCommands.BeginAddMeshLayerModifierTool, TEXT("BeginAddMeshLayerModifierTool"), static_cast<int>(UE::MeshPartition::EModifierClassType::MeshLayer));
	RegisterPlaceModifierTool(ToolManagerCommands.BeginAddNoiseModifierTool, TEXT("BeginAddNoiseModifierTool"), static_cast<int>(UE::MeshPartition::EModifierClassType::Noise));
	RegisterPlaceModifierTool(ToolManagerCommands.BeginAddBooleanModifierTool, TEXT("BeginAddBooleanModifierTool"), static_cast<int>(UE::MeshPartition::EModifierClassType::Boolean));
	RegisterPlaceModifierTool(ToolManagerCommands.BeginAddLatticeModifierTool, TEXT("BeginAddLatticeModifierTool"), static_cast<int>(UE::MeshPartition::EModifierClassType::Lattice));
	RegisterPlaceModifierTool(ToolManagerCommands.BeginAddSplineRemeshModifierTool, TEXT("BeginAddSplineRemeshModifierTool"), static_cast<int>(UE::MeshPartition::EModifierClassType::SplineRemesh));

	// register extensions
	TArray<IMeshTerrainModeToolExtension*> Extensions = IModularFeatures::Get().GetModularFeatureImplementations<IMeshTerrainModeToolExtension>(
		IMeshTerrainModeToolExtension::GetModularFeatureName());
	if (Extensions.Num() > 0)
	{
		FExtensionToolQueryInfo ExtensionQueryInfo;
		ExtensionQueryInfo.ToolsContext = GetInteractiveToolsContext();
		ExtensionQueryInfo.AssetAPI = nullptr;
		ExtensionQueryInfo.StylusAPI = StylusAPI;

		UE_LOGF(LogTemp, Log, "MeshTerrainMode: Found %d Tool Extension Modules", Extensions.Num());
		for (int32 k = 0; k < Extensions.Num(); ++k)
		{
			// TODO: extension name
			FText ExtensionName = Extensions[k]->GetExtensionName();
			FString ExtensionPrefix = FString::Printf(TEXT("[%d][%s]"), k, *ExtensionName.ToString());

			TArray<FExtensionToolDescription> ToolSet;
			Extensions[k]->GetExtensionTools(ExtensionQueryInfo, ToolSet);
			for (const FExtensionToolDescription& ToolInfo : ToolSet)
			{
				UE_LOGF(LogTemp, Log, "%ls - Registering Tool [%ls]", *ExtensionPrefix, *ToolInfo.ToolName.ToString());

				RegisterTool(ToolInfo.ToolCommand, ToolInfo.ToolName.ToString(), ToolInfo.ToolBuilder);
			}

			TArray<FExtensionCustomToolDescription> CustomToolSet;
			Extensions[k]->GetExtensionCustomTools(ExtensionQueryInfo, CustomToolSet);
			for (const FExtensionCustomToolDescription& ToolInfo : CustomToolSet)
			{
				UE_LOGF(LogTemp, Log, "%ls - Registering custom Tool [%ls]", *ExtensionPrefix, *ToolInfo.ToolName.ToString());

				RegisterCustomTool(
					ToolInfo.ToolCommand,
					ToolInfo.ToolName.ToString(),
					ToolInfo.ToolBuilder,
					ToolInfo.ExecuteAction,
					ToolInfo.CanExecuteAction,
					ToolInfo.IsActionChecked);
			}

			TArray<TSubclassOf<UToolTargetFactory>> ExtensionToolTargetFactoryClasses;
			if (Extensions[k]->GetExtensionToolTargets(ExtensionToolTargetFactoryClasses))
			{
				for (const TSubclassOf<UToolTargetFactory>& ExtensionTargetFactoryClass : ExtensionToolTargetFactoryClasses)
				{
					GetInteractiveToolsContext()->TargetManager->AddTargetFactory(NewObject<UToolTargetFactory>(GetToolManager(), ExtensionTargetFactoryClass.Get()));
				}
			}
		}
	}


	GetToolManager()->SelectActiveToolType(EToolSide::Left, TEXT("DynaSculptTool"));

	// Register modeling mode hotkeys. Note that we use the toolkit command list because we would like the hotkeys
	// to work even when the viewport is not focused, provided that nothing else captures the key presses.
	FMeshTerrainModeActionCommands::RegisterCommandBindings(Toolkit->GetToolkitCommands(), [this](EMeshTerrainModeActionCommands Command) {
		ModelingModeShortcutRequested(Command);
	});

	// enable realtime viewport override
	ConfigureRealTimeViewportsOverride(true);

	//
	// Engine Analytics
	//

	// Log mode starting
	if (FEngineAnalytics::IsAvailable())
	{
		LastModeStartTimestamp = FDateTime::UtcNow();

		TArray<FAnalyticsEventAttribute> Attributes;
		Attributes.Add(FAnalyticsEventAttribute(TEXT("Timestamp"), LastModeStartTimestamp.ToString()));

		FEngineAnalytics::GetProvider().RecordEvent(TEXT("Editor.Usage.MeshTerrainMode.Enter"), Attributes);
	}

	// Log tool starting
	GetToolManager()->OnToolStarted.AddLambda([this](UInteractiveToolManager* Manager, UInteractiveTool* Tool)
	{
		if (FEngineAnalytics::IsAvailable())
		{
			LastToolStartTimestamp = FDateTime::UtcNow();

			TArray<FAnalyticsEventAttribute> Attributes;
			Attributes.Add(FAnalyticsEventAttribute(TEXT("ToolName"), GetToolName(*Tool)));
			Attributes.Add(FAnalyticsEventAttribute(TEXT("Timestamp"), LastToolStartTimestamp.ToString()));

			FEngineAnalytics::GetProvider().RecordEvent(TEXT("Editor.Usage.MeshTerrainMode.ToolStarted"), Attributes);
		}
	});

	// Log tool ending
	GetToolManager()->OnToolEnded.AddLambda([this](UInteractiveToolManager* Manager, UInteractiveTool* Tool)
	{
		if (FEngineAnalytics::IsAvailable())
		{
			const FDateTime Now = FDateTime::UtcNow();
			const FTimespan ToolUsageDuration = Now - LastToolStartTimestamp;

			TArray<FAnalyticsEventAttribute> Attributes;
			Attributes.Add(FAnalyticsEventAttribute(TEXT("ToolName"), GetToolName(*Tool)));
			Attributes.Add(FAnalyticsEventAttribute(TEXT("Timestamp"), Now.ToString()));
			Attributes.Add(FAnalyticsEventAttribute(TEXT("Duration.Seconds"), static_cast<float>(ToolUsageDuration.GetTotalSeconds())));

			FEngineAnalytics::GetProvider().RecordEvent(TEXT("Editor.Usage.MeshTerrainMode.ToolEnded"), Attributes);
		}
	});

	InitializeGeometrySelection();
	ModelingModeSettings->OnModified.AddLambda([this](UObject*, FProperty*)
		{
			InitializeGeometrySelection();
		});

	// do any toolkit UI initialization that depends on the mode setup above
	if (Toolkit.IsValid())
	{
		FMeshTerrainModeToolkit* ModelingToolkit = (FMeshTerrainModeToolkit*)Toolkit.Get();
		ModelingToolkit->InitializeAfterModeSetup();
	}

	EditorClosedEventHandle = GEditor->OnEditorClose().AddUObject(this, &UMeshTerrainMode::OnEditorClosed);
}

void UMeshTerrainMode::RegisterUVEditor()
{
	IUVEditorModularFeature* UVEditorAPI = nullptr;
	// We should be allowed to do GetModularFeatureImplementation directly without the check, but currently there is an assert
	// there (despite what the header for that function promises).
	if (IModularFeatures::Get().IsModularFeatureAvailable(IUVEditorModularFeature::GetModularFeatureName()))
	{
		UVEditorAPI = static_cast<IUVEditorModularFeature*>(IModularFeatures::Get().GetModularFeatureImplementation(IUVEditorModularFeature::GetModularFeatureName(), 0));
	}

	if (UVEditorAPI)
	{
		const FMeshTerrainModeManagerCommands& ToolManagerCommands = FMeshTerrainModeManagerCommands::Get();
		const TSharedRef<FUICommandList>& CommandList = Toolkit->GetToolkitCommands();
		CommandList->MapAction(ToolManagerCommands.LaunchUVEditor,
			FExecuteAction::CreateLambda([this, UVEditorAPI]() 
            {
				if (UVEditorAPI) 
                {
					EToolsContextScope ToolScope = GetDefaultToolScope();
					UEditorInteractiveToolsContext* UseToolsContext = GetInteractiveToolsContext(ToolScope);
					if (ensure(UseToolsContext != nullptr) == false)
					{
						return;
					}

					TArray<UObject*> SelectedActors, SelectedComponents;
					TArray<TObjectPtr<UObject>> SelectedObjects;
					UseToolsContext->GetParentEditorModeManager()->GetSelectedActors()->GetSelectedObjects(SelectedActors);
					UseToolsContext->GetParentEditorModeManager()->GetSelectedComponents()->GetSelectedObjects(SelectedComponents);
					SelectedObjects.Append(SelectedActors);
					SelectedObjects.Append(SelectedComponents);
					UVEditorAPI->LaunchUVEditor(SelectedObjects);
				}
				}),
			FCanExecuteAction::CreateLambda([this, UVEditorAPI]() 
            {
				if (UVEditorAPI) 
                {
					EToolsContextScope ToolScope = GetDefaultToolScope();
					UEditorInteractiveToolsContext* UseToolsContext = GetInteractiveToolsContext(ToolScope);
					if (ensure(UseToolsContext != nullptr) == false)
					{
						return false;
					}

					TArray<UObject*> SelectedActors, SelectedComponents;
					TArray<TObjectPtr<UObject>> SelectedObjects;
					UseToolsContext->GetParentEditorModeManager()->GetSelectedActors()->GetSelectedObjects(SelectedActors);
					UseToolsContext->GetParentEditorModeManager()->GetSelectedComponents()->GetSelectedObjects(SelectedComponents);
					SelectedObjects.Append(SelectedActors);
					SelectedObjects.Append(SelectedComponents);
					return UVEditorAPI->CanLaunchUVEditor(SelectedObjects);
				}
				return false;
				})
			);
	}

}

void UMeshTerrainMode::InitializeGeometrySelection()
{
	const UMeshTerrainModeSettings* ModelingModeSettings = GetDefault<UMeshTerrainModeSettings>();
	const bool bNewSelectionSystemEnabled = ModelingModeSettings->GetMeshSelectionsEnabled();
	if (bNewSelectionSystemEnabled == bSelectionSystemEnabled)
	{
		return;
	}

	if (bSelectionSystemEnabled)
	{
		ShutdownGeometrySelection();
		return;
	}

	const UMeshTerrainModeCustomizationSettings* ModelingEditorSettings = GetDefault<UMeshTerrainModeCustomizationSettings>();
	const FMeshTerrainModeManagerCommands& ToolManagerCommands = FMeshTerrainModeManagerCommands::Get();

	// set up SelectionManager and register known factory types
	SelectionManager = NewObject<UGeometrySelectionManager>(GetToolManager());
	SelectionManager->Initialize(GetInteractiveToolsContext(), GetToolManager()->GetContextTransactionsAPI());
	SelectionManager->RegisterSelectorFactory(MakeUnique<FDynamicMeshComponentSelectorFactory>());
	SelectionManager->RegisterSelectorFactory(MakeUnique<FBrushComponentSelectorFactory>());
	SelectionManager->RegisterSelectorFactory(MakeUnique<FStaticMeshComponentSelectorFactory>());

	// this is hopefully temporary? kinda gross...
	GetInteractiveToolsContext()->ContextObjectStore->AddContextObject(SelectionManager);

	// set up the selection interaction
	SelectionInteraction = NewObject<UMeshTerrainModeSelectionInteraction>(GetToolManager());
	SelectionInteraction->Initialize(SelectionManager,
		[this]() { return GetGeometrySelectionChangesAllowed(); },
		[this](const FInputDeviceRay& DeviceRay) { return TestForEditorGizmoHit(DeviceRay); });
	GetInteractiveToolsContext()->InputRouter->RegisterSource(SelectionInteraction);

	SelectionInteraction->OnTransformBegin.AddLambda([this]() 
	{ 
		// Disable the SnappingManager while the SelectionInteraction is editing a mesh via transform gizmo
		SceneSnappingManager->PauseSceneGeometryUpdates();

		// If the transform is happening via the gizmo numerical UI, then we can run into the same slate
		// throttling issues as tools. We need to continue receiving render/tick while user scrubs the slate values.
		FSlateThrottleManager::Get().DisableThrottle(true);
	});
	SelectionInteraction->OnTransformEnd.AddLambda([this]() 
	{ 
		FSlateThrottleManager::Get().DisableThrottle(false);
		SceneSnappingManager->UnPauseSceneGeometryUpdates(); 
	});

	constexpr bool bAlwaysShowAllSelectionCommands = true;

	// this function registers and tracks an active UGeometrySelectionEditCommand and it's associated UICommand
	auto RegisterSelectionTool = [this, bAlwaysShowAllSelectionCommands](TSharedPtr<FUICommandInfo> UICommand, FString ToolIdentifier, UInteractiveToolBuilder* Builder, bool bRequiresActiveTarget, bool bRequiresSelection)
	{
		UEditorInteractiveToolsContext* UseToolsContext = GetInteractiveToolsContext(EToolsContextScope::EdMode);
		const TSharedRef<FUICommandList>& CommandList = Toolkit->GetToolkitCommands();
		UseToolsContext->ToolManager->RegisterToolType(ToolIdentifier, Builder);
		CommandList->MapAction(UICommand,
			FExecuteAction::CreateUObject(UseToolsContext, &UEdModeInteractiveToolsContext::StartTool, ToolIdentifier),
			FCanExecuteAction::CreateWeakLambda(UseToolsContext, [this, ToolIdentifier, UseToolsContext, bRequiresActiveTarget, bRequiresSelection, bAlwaysShowAllSelectionCommands]() {
				return ShouldToolStartBeAllowed(ToolIdentifier) &&
					( GetSelectionManager()->HasActiveTargets() || bRequiresActiveTarget == false) &&
					( GetSelectionManager()->HasSelection() || bRequiresSelection == false ) &&
					( GetSelectionManager()->GetMeshTopologyMode() != UGeometrySelectionManager::EMeshTopologyMode::None || bAlwaysShowAllSelectionCommands ) &&
					UseToolsContext->ToolManager->CanActivateTool(EToolSide::Mouse, ToolIdentifier);
			}),
			FIsActionChecked::CreateUObject(UseToolsContext, &UEdModeInteractiveToolsContext::IsToolActive, EToolSide::Mouse, ToolIdentifier),
			//FIsActionButtonVisible::CreateUObject(GetSelectionManager(), &UGeometrySelectionManager::HasSelection),
			FIsActionButtonVisible::CreateWeakLambda(UseToolsContext, [this, bRequiresActiveTarget, bRequiresSelection, bAlwaysShowAllSelectionCommands]() {
				return (bAlwaysShowAllSelectionCommands) ? true : (
					( GetSelectionManager()->HasActiveTargets() || bRequiresActiveTarget == false) &&
					( GetSelectionManager()->HasSelection() || bRequiresSelection == false ) &&
					( GetSelectionManager()->GetMeshTopologyMode() != UGeometrySelectionManager::EMeshTopologyMode::None ) );
			}),
			EUIActionRepeatMode::RepeatDisabled);
		RegisteredSelectionTools.Add(ToolIdentifier);
	};

	// register mesh-selection-driven tools
	RegisterSelectionTool(ToolManagerCommands.BeginSelectionAction_Extrude, TEXT("BeginSelectionExtrudeTool"), NewObject<UExtrudeMeshSelectionToolBuilder>(), true, true);
	RegisterSelectionTool(ToolManagerCommands.BeginSelectionAction_Offset, TEXT("BeginSelectionOffsetTool"), NewObject<UOffsetMeshSelectionToolBuilder>(), true, true);

	RegisterSelectionTool(ToolManagerCommands.BeginPolyModelTool_PolyEd, TEXT("BeginSelectionPolyEdTool"), NewObject<UEditMeshPolygonsSelectionModeToolBuilder>(), true, false);
	RegisterSelectionTool(ToolManagerCommands.BeginPolyModelTool_TriSel, TEXT("BeginSelectionTriEdTool"), NewObject<UMeshSelectionToolBuilder>(), true, false);


	auto RegisterPolyModelActionTool = [&](EEditMeshPolygonsToolActions Action, TSharedPtr<FUICommandInfo> UICommand, FString StringName, bool bRequiresSelection)
	{
		UEditMeshPolygonsActionModeToolBuilder* ActionModeBuilder = NewObject<UEditMeshPolygonsActionModeToolBuilder>();
		ActionModeBuilder->StartupAction = Action;
		RegisterSelectionTool(UICommand, StringName, ActionModeBuilder, true, bRequiresSelection);
	};
	RegisterPolyModelActionTool(EEditMeshPolygonsToolActions::Inset, ToolManagerCommands.BeginPolyModelTool_Inset, TEXT("PolyEdit_Inset"), true);
	RegisterPolyModelActionTool(EEditMeshPolygonsToolActions::Outset, ToolManagerCommands.BeginPolyModelTool_Outset, TEXT("PolyEdit_Outset"), true);
	RegisterPolyModelActionTool(EEditMeshPolygonsToolActions::CutFaces, ToolManagerCommands.BeginPolyModelTool_CutFaces, TEXT("PolyEdit_CutFaces"), true);
	RegisterPolyModelActionTool(EEditMeshPolygonsToolActions::InsertEdgeLoop, ToolManagerCommands.BeginPolyModelTool_InsertEdgeLoop, TEXT("PolyEdit_InsertEdgeLoop"), false);
	RegisterPolyModelActionTool(EEditMeshPolygonsToolActions::ExtrudeEdges, ToolManagerCommands.BeginPolyModelTool_ExtrudeEdges, TEXT("PolyEdit_ExtrudeEdges"), true);
	RegisterPolyModelActionTool(EEditMeshPolygonsToolActions::PushPull, ToolManagerCommands.BeginPolyModelTool_PushPull, TEXT("PolyEdit_PushPull"), true);
	RegisterPolyModelActionTool(EEditMeshPolygonsToolActions::BevelAuto, ToolManagerCommands.BeginPolyModelTool_Bevel, TEXT("PolyEdit_Bevel"), true);
	

	// set up selection type toggles

	auto RegisterSelectionMode = [this](UGeometrySelectionManager::EMeshTopologyMode TopoMode, UE::Geometry::EGeometryElementType ElementMode, TSharedPtr<FUICommandInfo> UICommand)
	{
		Toolkit->GetToolkitCommands()->MapAction(UICommand,
			FExecuteAction::CreateLambda([this, TopoMode, ElementMode]() {
				if ( GetToolManager() && GetToolManager()->GetContextTransactionsAPI() && GetSelectionManager() )
				{
					GetToolManager()->GetContextTransactionsAPI()->BeginUndoTransaction(LOCTEXT("ChangeSelectionMode", "Selection Mode"));
					GetSelectionManager()->SetMeshSelectionTypeAndMode(ElementMode, TopoMode,  TopoMode != UGeometrySelectionManager::EMeshTopologyMode::None);
					GetToolManager()->GetContextTransactionsAPI()->EndUndoTransaction();
					if (UMeshTerrainModeCustomizationSettings* ModelingEditorSettings = GetMutableDefault<UMeshTerrainModeCustomizationSettings>())
					{
						ModelingEditorSettings->LastMeshSelectionTopologyMode = static_cast<int>(TopoMode);
						ModelingEditorSettings->LastMeshSelectionElementType = static_cast<int>(ElementMode);
						ModelingEditorSettings->SaveConfig();
					}
				}
			}),
			FCanExecuteAction::CreateLambda([this]()
			{
				// necessary due to Exit()->RemoveModelingModeViewportToolbarExtensions->UnregisterOwnerByName->RefreshAllWidgets on next tick
				// otherwise, this GetToolManager will cause crash because ModeToolsContext is null (after exit)
				if (!GetInteractiveToolsContext())
				{
					return false;
				}
				return (GetToolManager() != nullptr && GetToolManager()->HasAnyActiveTool() == false) && GetMeshElementSelectionSystemEnabled() ? true : false;
			}),
			FIsActionChecked::CreateLambda([this, TopoMode, ElementMode]() { return (GetSelectionManager() != nullptr && GetSelectionManager()->GetMeshTopologyMode() == TopoMode && GetSelectionManager()->GetSelectionElementType() == ElementMode); }),
			EUIActionRepeatMode::RepeatDisabled);
	};
	if (GetSelectionManager() != nullptr)
	{
		RegisterSelectionMode(UGeometrySelectionManager::EMeshTopologyMode::None, UGeometrySelectionManager::EGeometryElementType::Face, ToolManagerCommands.MeshSelectionModeAction_NoSelection);
		RegisterSelectionMode(UGeometrySelectionManager::EMeshTopologyMode::Triangle, UGeometrySelectionManager::EGeometryElementType::Face, ToolManagerCommands.MeshSelectionModeAction_MeshTriangles);
		RegisterSelectionMode(UGeometrySelectionManager::EMeshTopologyMode::Triangle, UGeometrySelectionManager::EGeometryElementType::Vertex, ToolManagerCommands.MeshSelectionModeAction_MeshVertices);
		RegisterSelectionMode(UGeometrySelectionManager::EMeshTopologyMode::Triangle, UGeometrySelectionManager::EGeometryElementType::Edge, ToolManagerCommands.MeshSelectionModeAction_MeshEdges);
		RegisterSelectionMode(UGeometrySelectionManager::EMeshTopologyMode::Polygroup, UGeometrySelectionManager::EGeometryElementType::Face, ToolManagerCommands.MeshSelectionModeAction_GroupFaces);
		RegisterSelectionMode(UGeometrySelectionManager::EMeshTopologyMode::Polygroup, UGeometrySelectionManager::EGeometryElementType::Vertex, ToolManagerCommands.MeshSelectionModeAction_GroupCorners);
		RegisterSelectionMode(UGeometrySelectionManager::EMeshTopologyMode::Polygroup, UGeometrySelectionManager::EGeometryElementType::Edge, ToolManagerCommands.MeshSelectionModeAction_GroupEdges);
	}

	// Register Drag Modes
	
	auto MakeDragModeOptionAction = [this](EMeshTerrainModeSelectionInteraction_DragMode DragMode)
	{
		return FUIAction(
			FExecuteAction::CreateLambda([this, DragMode]
			{
				SelectionInteraction->SetActiveDragMode(DragMode);
				UMeshTerrainModeCustomizationSettings* ModelingEditorSettings = GetMutableDefault<UMeshTerrainModeCustomizationSettings>();
				ModelingEditorSettings->LastMeshSelectionDragMode = static_cast<int>(DragMode);
				ModelingEditorSettings->SaveConfig();
			}),
			FCanExecuteAction(),
			FIsActionChecked::CreateLambda([this, DragMode]()
			{
				return SelectionInteraction->GetActiveDragMode() == DragMode;
			}));
	};
	
	const TSharedRef<FUICommandList>& CommandList = Toolkit->GetToolkitCommands();

	// map drag mode commands
	CommandList->MapAction(ToolManagerCommands.SelectionDragMode_None,
		MakeDragModeOptionAction(EMeshTerrainModeSelectionInteraction_DragMode::NoDragInteraction));
	CommandList->MapAction(ToolManagerCommands.SelectionDragMode_Path,
		MakeDragModeOptionAction(EMeshTerrainModeSelectionInteraction_DragMode::PathInteraction));

	// Register Selection Mesh Types

	FUIAction ToggleVolumesAction(
		FExecuteAction::CreateLambda([this]
		{
			bEnableVolumeElementSelection = !bEnableVolumeElementSelection;

			UMeshTerrainModeCustomizationSettings* ModelingEditorSettings = GetMutableDefault<UMeshTerrainModeCustomizationSettings>();
			ModelingEditorSettings->bLastMeshSelectionVolumeToggle = bEnableVolumeElementSelection;
		}),
		FCanExecuteAction(),
		FIsActionChecked::CreateLambda([this]()
		{
			return bEnableVolumeElementSelection;
		}));
	FUIAction ToggleStaticMeshesAction(
		FExecuteAction::CreateLambda([this]
		{	bEnableStaticMeshElementSelection = !bEnableStaticMeshElementSelection;

			UMeshTerrainModeCustomizationSettings* ModelingEditorSettings = GetMutableDefault<UMeshTerrainModeCustomizationSettings>();
			ModelingEditorSettings->bLastMeshSelectionStaticMeshToggle = bEnableStaticMeshElementSelection;
		}),
		FCanExecuteAction(),
		FIsActionChecked::CreateLambda([this]()
		{
			return bEnableStaticMeshElementSelection;
		}));
	
	// map selection mesh type commands
	CommandList->MapAction(ToolManagerCommands.SelectionMeshTypes_Volumes, ToggleVolumesAction);
	CommandList->MapAction(ToolManagerCommands.SelectionMeshTypes_StaticMeshes, ToggleStaticMeshesAction);

	// Register Frame Modes

	auto ToggleFromGeometryAction = [this](EMeshTerrainModeSelectionInteraction_LocalFrameMode LocalFrameMode)
	{
		return FUIAction(
			FExecuteAction::CreateLambda([this, LocalFrameMode]
			{
				SelectionInteraction->SetLocalFrameMode(LocalFrameMode);
				UMeshTerrainModeCustomizationSettings* ModelingEditorSettings = GetMutableDefault<UMeshTerrainModeCustomizationSettings>();
				ModelingEditorSettings->LastMeshSelectionLocalFrameMode = static_cast<int>(LocalFrameMode);
				ModelingEditorSettings->SaveConfig();
			}),
			FCanExecuteAction(),
			FIsActionChecked::CreateLambda([this, LocalFrameMode]()
			{
				return (SelectionInteraction->GetLocalFrameMode() == LocalFrameMode);
			}));
	};
	
	CommandList->MapAction(ToolManagerCommands.SelectionLocalFrameMode_Geometry,
		ToggleFromGeometryAction(EMeshTerrainModeSelectionInteraction_LocalFrameMode::FromGeometry));
	CommandList->MapAction(ToolManagerCommands.SelectionLocalFrameMode_Object,
		ToggleFromGeometryAction( EMeshTerrainModeSelectionInteraction_LocalFrameMode::FromObject));

	// Colors initialized here any time Modeling mode is entered
	SelectionManager->SetSelectionColors(ModelingEditorSettings->UnselectedColor, ModelingEditorSettings->HoverOverSelectedColor, ModelingEditorSettings->HoverOverUnselectedColor, ModelingEditorSettings->GeometrySelectedColor, ModelingEditorSettings->GeometrySoftSelectedColor);

	// this function registers and tracks an active UGeometrySelectionEditCommand and it's associated UICommand
	auto RegisterSelectionCommand = [this, &CommandList](UGeometrySelectionEditCommand* Command, TSharedPtr<FUICommandInfo> UICommand, bool bAlwaysVisible)
	{
		ModelingModeCommands.Add(Command);
		CommandList->MapAction(UICommand,
			FExecuteAction::CreateUObject(GetSelectionManager(), &UGeometrySelectionManager::ExecuteSelectionCommand, Command),
			FCanExecuteAction::CreateUObject(GetSelectionManager(), &UGeometrySelectionManager::CanExecuteSelectionCommand, Command),
			FIsActionChecked(),
			(bAlwaysVisible) ? FIsActionButtonVisible() : 
				FIsActionButtonVisible::CreateUObject(GetSelectionManager(), &UGeometrySelectionManager::CanExecuteSelectionCommand, Command),
			EUIActionRepeatMode::RepeatDisabled);
	};

	// create and register InteractiveCommands for mesh selections
	RegisterSelectionCommand(NewObject<UDeleteGeometrySelectionCommand>(), ToolManagerCommands.BeginSelectionAction_Delete, bAlwaysShowAllSelectionCommands);
	RegisterSelectionCommand(NewObject<UDisconnectGeometrySelectionCommand>(), ToolManagerCommands.BeginSelectionAction_Disconnect, bAlwaysShowAllSelectionCommands);
	RegisterSelectionCommand(NewObject<URetriangulateGeometrySelectionCommand>(), ToolManagerCommands.BeginSelectionAction_Retriangulate, bAlwaysShowAllSelectionCommands);
	RegisterSelectionCommand(NewObject<UModifyGeometrySelectionCommand>(), ToolManagerCommands.BeginSelectionAction_SelectAll, true);
	RegisterSelectionCommand(NewObject<UModifyGeometrySelectionCommand_Invert>(), ToolManagerCommands.BeginSelectionAction_Invert, true);
	RegisterSelectionCommand(NewObject<UModifyGeometrySelectionCommand_ExpandToConnected>(), ToolManagerCommands.BeginSelectionAction_ExpandToConnected, true);
	RegisterSelectionCommand(NewObject<UModifyGeometrySelectionCommand_InvertConnected>(), ToolManagerCommands.BeginSelectionAction_InvertConnected, true);
	RegisterSelectionCommand(NewObject<UModifyGeometrySelectionCommand_Expand>(), ToolManagerCommands.BeginSelectionAction_Expand, true);
	RegisterSelectionCommand(NewObject<UModifyGeometrySelectionCommand_Contract>(), ToolManagerCommands.BeginSelectionAction_Contract, true);

	// Register Selection Lock Toggle
	FUIAction ToggleSelectionLocking(
		FExecuteAction::CreateLambda([this]
		{
			SelectionManager->SetCurrentTargetsLockState(!SelectionManager->GetAnyCurrentTargetsLocked());
		}),
		FCanExecuteAction::CreateLambda([this]
		{
			return Toolkit.IsValid()
			&& StaticCast<FMeshTerrainModeToolkit*>(Toolkit.Get())->IsInActiveTool() == false
			&& SelectionManager->GetMeshTopologyMode() != UGeometrySelectionManager::EMeshTopologyMode::None
			&& SelectionManager->GetAnyCurrentTargetsLockable();
		}),
		FIsActionChecked::CreateLambda([this]()
		{
			return SelectionManager->GetAnyCurrentTargetsLocked();
		})
	);
	CommandList->MapAction(ToolManagerCommands.SelectionLocking, ToggleSelectionLocking);

	// Restore saved selections when tool is cancelled or tool declares it is safe to do so via the IInteractiveToolManageGeometrySelectionAPI
	GetToolManager()->OnToolEndedWithStatus.AddLambda([this](UInteractiveToolManager* Manager, UInteractiveTool* Tool, EToolShutdownType ShutdownType)
	{
		bool bCanRestore = (ShutdownType == EToolShutdownType::Cancel);
		if (IInteractiveToolManageGeometrySelectionAPI* ManageSelectionTool = Cast<IInteractiveToolManageGeometrySelectionAPI>(Tool))
		{
			bCanRestore = bCanRestore || ManageSelectionTool->IsInputSelectionValidOnOutput();
		}
		if (bCanRestore)
		{
			GetSelectionManager()->RestoreSavedSelection();
		}
		else
		{
			GetSelectionManager()->DiscardSavedSelection();
		}
		ensureMsgf(!GetSelectionManager()->HasSavedSelection(), TEXT("Selection manager's saved selection should be cleared on tool end."));
	});

	// Need to know about selection changes to (eg) clear mesh selections. 
	// Listening to USelection::SelectionChangedEvent here instead of the underlying UTypedElementSelectionSet
	// events because they do not fire at the right times, particular wrt undo/redo. 
	SelectionModifiedEventHandle = GetModeManager()->GetSelectedActors()->SelectionChangedEvent.AddLambda(
		[this](const UObject* Object)  { UpdateSelectionManagerOnEditorSelectionChange(); } );

	// restore various settings
	if (GetSelectionManager())
	{
		UE::Geometry::EGeometryElementType LastElementType = static_cast<UE::Geometry::EGeometryElementType>(ModelingEditorSettings->LastMeshSelectionElementType);
		if (LastElementType == UE::Geometry::EGeometryElementType::Edge || LastElementType == UE::Geometry::EGeometryElementType::Face || LastElementType == UE::Geometry::EGeometryElementType::Vertex)
		{
			GetSelectionManager()->SetSelectionElementType(LastElementType);
		}
		UGeometrySelectionManager::EMeshTopologyMode LastTopologyMode = static_cast<UGeometrySelectionManager::EMeshTopologyMode>(ModelingEditorSettings->LastMeshSelectionTopologyMode);
		if (LastTopologyMode == UGeometrySelectionManager::EMeshTopologyMode::None || LastTopologyMode == UGeometrySelectionManager::EMeshTopologyMode::Triangle || LastTopologyMode == UGeometrySelectionManager::EMeshTopologyMode::Polygroup)
		{
			GetSelectionManager()->SetMeshTopologyMode(LastTopologyMode);
		}

		bEnableStaticMeshElementSelection = ModelingEditorSettings->bLastMeshSelectionStaticMeshToggle;
		bEnableVolumeElementSelection = ModelingEditorSettings->bLastMeshSelectionVolumeToggle;
	}
	if ( SelectionInteraction )
	{
		EMeshTerrainModeSelectionInteraction_DragMode LastDragMode = static_cast<EMeshTerrainModeSelectionInteraction_DragMode>(ModelingEditorSettings->LastMeshSelectionDragMode);
		if ( LastDragMode == EMeshTerrainModeSelectionInteraction_DragMode::NoDragInteraction || LastDragMode == EMeshTerrainModeSelectionInteraction_DragMode::PathInteraction )
		{
			SelectionInteraction->SetActiveDragMode(LastDragMode);
		}

		EMeshTerrainModeSelectionInteraction_LocalFrameMode LastLocalFrameMode =
			static_cast<EMeshTerrainModeSelectionInteraction_LocalFrameMode>(ModelingEditorSettings->LastMeshSelectionLocalFrameMode);
		if ( LastLocalFrameMode == EMeshTerrainModeSelectionInteraction_LocalFrameMode::FromGeometry || LastLocalFrameMode == EMeshTerrainModeSelectionInteraction_LocalFrameMode::FromObject )
		{
			SelectionInteraction->SetLocalFrameMode(LastLocalFrameMode);
		}
	}

	// initialize SelectionManager w/ active selection
	UpdateSelectionManagerOnEditorSelectionChange(true);

	// Selection system currently requires the concept of 'locking' for Static Meshes and Volumes. This is maintained
	// by a global list that we do *not* want to clear between invocations of Modeling Mode (v annoying if frequently switching
	// modes) but *do* want to clear when the user loads a new level. So, the first time this runs, register a delegate that listens
	// for level editor map changes. This is a static member and will never be unregistered!
	if (GlobalModelingWorldTeardownEventHandle.IsValid() == false)
	{
		FLevelEditorModule& LevelEditor = FModuleManager::LoadModuleChecked<FLevelEditorModule>("LevelEditor");
		GlobalModelingWorldTeardownEventHandle = LevelEditor.OnMapChanged().AddLambda([](UWorld* World, EMapChangeType ChangeType)
		{
			if (ChangeType == EMapChangeType::TearDownWorld)
			{
				FVolumeSelector::ResetUnlockedBrushComponents();
				FStaticMeshSelector::ResetUnlockedStaticMeshAssets();
			}
		});
	}

	BlueprintPreCompileHandle = GEditor->OnBlueprintPreCompile().AddUObject(this, &UMeshTerrainMode::OnBlueprintPreCompile);

	// Removing levels from the world can happen either by entering/exiting level instance edit mode, or
	// by using the Levels panel. The problem is that any temporary actors we may have spawned in the 
	// level for visualization, gizmos, etc. will be garbage collected. While EdModeInteractiveToolsContext
	// should end the tools for us, we still have to take care of mode-level temporary actors.
	FWorldDelegates::PreLevelRemovedFromWorld.AddWeakLambda(this, [this](ULevel*, UWorld*) {
		// The ideal solution would be to just exit the mode, but we don't have a way to do that- we
		// can only request a mode switch on next tick.Since this is too late to prevent a crash, we
		// hand-clean up temporary actors here.
		if (SelectionInteraction)
		{
			SelectionInteraction->Shutdown();
		}

		// Since we're doing this hand-cleanup above, we could actually register to OnCurrentLevelChanged and
		// reinstate the temporary actors to stay in the mode. That seems a bit brittle, though, and there
		// is still some hope that we can someday exit the mode instead of having to keep track of what is
		// in danger of being garbage collected, so we might as well keep the workflow the same (i.e. exit
		// mode).
		GetModeManager()->ActivateDefaultMode();
	});
	
	// add Mesh Element Selection commands to the new viewport toolbar 
	PopulateModelingModeViewportToolbar("LevelEditor.ViewportToolbar", Toolkit->GetToolkitCommands());

	bSelectionSystemEnabled = true;
}

void UMeshTerrainMode::ShutdownGeometrySelection()
{
	// on Modeling Mode exit, remove Mesh Element Selection commands from new viewport toolbar
	RemoveModelingModeViewportToolbarExtensions();
	
	FWorldDelegates::PreLevelRemovedFromWorld.RemoveAll(this);
	if (BlueprintPreCompileHandle.IsValid())
	{
		GEditor->OnBlueprintPreCompile().Remove(BlueprintPreCompileHandle);
	}

	// shutdown selection interaction
	if (SelectionInteraction != nullptr)
	{
		SelectionInteraction->Shutdown();
		GetInteractiveToolsContext()->InputRouter->DeregisterSource(SelectionInteraction);
		SelectionInteraction = nullptr;
	}

	// stop listening to selection changes. On Editor Shutdown, some of these values become null, which will result in an ensure/crash
	if (SelectionModifiedEventHandle.IsValid() && UObjectInitialized() && GetModeManager() 
		&& ( GetModeManager()->GetSelectedActors() != nullptr) )
	{
		GetModeManager()->GetSelectedActors()->SelectionChangedEvent.Remove(SelectionModifiedEventHandle);
	}

	// exit any exclusive active tools w/ cancel
	if (UInteractiveTool* ActiveTool = GetToolManager()->GetActiveTool(EToolSide::Left))
	{
		if (Cast<IInteractiveToolExclusiveToolAPI>(ActiveTool))
		{
			GetToolManager()->DeactivateTool(EToolSide::Left, EToolShutdownType::Cancel);
		}
	}

	// Shutdown SelectionManager. Wait until after Tool shutdown in case some restore-selection
	// is involved
	if (SelectionManager != nullptr)
	{
		SelectionManager->OnSelectionModified.Remove(SelectionManager_SelectionModifiedHandle);
		SelectionManager->ClearSelection();
		SelectionManager->Shutdown();		// will clear active targets

		// hopefully temporary...remove SelectionManager from ContextObjectStore
		GetInteractiveToolsContext()->ContextObjectStore->RemoveContextObject(SelectionManager);

		SelectionManager = nullptr;
	}

	// Deregister selection tools
	for (const FString& RegisteredTool : RegisteredSelectionTools)
	{
		GetToolManager()->UnregisterToolType(RegisteredTool);
	}

	bSelectionSystemEnabled = false;
}


void UMeshTerrainMode::Exit()
{
	ShutdownGeometrySelection();

	//
	// Engine Analytics
	//
	// Log mode ending
	if (FEngineAnalytics::IsAvailable())
	{
		const FTimespan ModeUsageDuration = FDateTime::UtcNow() - LastModeStartTimestamp;

		TArray<FAnalyticsEventAttribute> Attributes;
		Attributes.Add(FAnalyticsEventAttribute(TEXT("Timestamp"), FDateTime::UtcNow().ToString()));
		Attributes.Add(FAnalyticsEventAttribute(TEXT("Duration.Seconds"), static_cast<float>(ModeUsageDuration.GetTotalSeconds())));

		FEngineAnalytics::GetProvider().RecordEvent(TEXT("Editor.Usage.MeshTerrainMode.Exit"), Attributes);
	}

	UMeshTerrainModeHostCustomizationAPI::Deregister(GetInteractiveToolsContext());

	// TODO: cannot deregister currently because if another mode is also registering, its Enter()
	// will be called before our Exit()
	//UE::TransformGizmoUtil::DeregisterTransformGizmoContextObject(ToolsContext.Get());
	
	// deregister snapping manager and shut down level objects tracker
	LevelObjectsObserver->Shutdown();		// do this first because it is going to fire events on the snapping manager
	LevelObjectsObserver.Reset();
	UE::Geometry::DeregisterSceneSnappingManager(GetInteractiveToolsContext());
	SceneSnappingManager = nullptr;

	// TODO: cannot deregister currently because if another mode is also registering, its Enter()
	// will be called before our Exit()
	UEditorModelingObjectsCreationAPI* ObjectCreationAPI = UEditorModelingObjectsCreationAPI::Find(GetInteractiveToolsContext());
	if (ObjectCreationAPI)
	{
		ObjectCreationAPI->GetNewAssetPathNameCallback.Unbind();
		ObjectCreationAPI->OnModelingMeshCreated.Remove(MeshCreatedEventHandle);
		ObjectCreationAPI->OnModelingTextureCreated.Remove(TextureCreatedEventHandle);
		ObjectCreationAPI->OnModelingMaterialCreated.Remove(MaterialCreatedEventHandle);
		//UEditorModelingObjectsCreationAPI::Deregister(ToolsContext.Get());		// cannot do currently because of shared ToolsContext, revisit in future
	}

	FMeshTerrainModeActionCommands::UnRegisterCommandBindings(Toolkit->GetToolkitCommands());

	// clear realtime viewport override
	ConfigureRealTimeViewportsOverride(false);

	// re-enable HitProxy rendering
	GetInteractiveToolsContext()->SetEnableRenderingDuringHitProxyPass(true);

	// Clear last-tool state so we don't hold stale references across mode re-entry.
	LastTargetComponents.Empty();
	LastToolIdentifier.Empty();
	LastSubmodeName = NAME_None;

	// Call base Exit method to ensure proper cleanup
	UEdMode::Exit();
}

void UMeshTerrainMode::OnEditorClosed()
{
	// On editor close, Exit() should run to clean up, but this happens very late.
	// Close out any active Tools or Selections to mitigate any late-destruction issues.

	if (SelectionManager != nullptr)
	{
		SelectionManager->ClearSelection();
		SelectionManager->ClearActiveTargets();
	}

	if (GetModeManager() != nullptr 
		&& GetInteractiveToolsContext() != nullptr 
		&& GetToolManager() != nullptr 
		&& GetToolManager()->HasAnyActiveTool())
	{
		GetToolManager()->DeactivateTool(EToolSide::Mouse, EToolShutdownType::Cancel);
	}

	if (EditorClosedEventHandle.IsValid() && GEditor)
	{
		GEditor->OnEditorClose().Remove(EditorClosedEventHandle);
	}
	
	// cleanup active toolkit stylus input contexts/windows
	if (Toolkit.IsValid())
	{
		static_cast<FMeshTerrainModeToolkit*>(Toolkit.Get())->DisconnectStylusStateProviderAPI();
	}
}


void UMeshTerrainMode::OnBlueprintPreCompile(UBlueprint* Blueprint)
{
	// if a Blueprint is compiled, all old instances of it in a level go "stale" and new
	// instances are created. Currently SelectionManager does not handle this replacement. 
	// Seems quite hard to know if Blueprint is a parent of any active targets, so if
	// a Blueprint is compiled we will just clear out any active selection & targets to avoid
	// potential crashes. Note that this also breaks undo somewhat, as the FChanges seem to 
	// still be registered against the 'old' instance pointer and hence are ignored/skipped.
	if (SelectionManager != nullptr)
	{
		SelectionManager->ClearSelection();
		SelectionManager->ClearActiveTargets();
	}
}

 void UMeshTerrainMode::PopulateModelingModeViewportToolbar(const FName InMenuName, const TSharedPtr<const FUICommandList>& InCommandList)
{
	using namespace MeshTerrainModeLocals;

	FToolMenuOwnerScoped ScopeOwner(MeshTerrainModeOwnerName);
	UToolMenu* const Menu = UToolMenus::Get()->ExtendMenu(InMenuName);
	
	
	FToolMenuSection& MeshElementSelectionMenu = Menu->FindOrAddSection("Left");
	FToolMenuEntry Entry = FToolMenuEntry::InitSubMenu(
		"Mesh Element Selection",
		LOCTEXT("MeshElementSelectionSubmenuLabel", "Mesh Element Selection"),
		LOCTEXT("MeshElementSelectionSubmenuTooltip", "Mesh Element Selection settings in the viewport"),
		FNewToolMenuDelegate::CreateLambda([ WeakCommandList = InCommandList.ToWeakPtr()](UToolMenu* Submenu) -> void
		{
			FToolMenuSection& MeshElementSelectionOptionsSection =
			Submenu->FindOrAddSection("Element Selection", LOCTEXT("ElementSelectionLabel", "Element Selection"));
			if (const TSharedPtr<const FUICommandList>& CommandList = WeakCommandList.Pin())
			{
				auto CreateToolMenuEntry = [&CommandList, &MeshElementSelectionOptionsSection](const TSharedPtr<FUICommandInfo>& Command)
				{
					FToolMenuEntry MESMode = FToolMenuEntry::InitMenuEntryWithCommandList( Command, CommandList );
					MESMode.SetShowInToolbarTopLevel(true); // add button to toolbar
					MeshElementSelectionOptionsSection.AddEntry(MESMode);
				};
				// add all Mesh Element Selection modes
				CreateToolMenuEntry(FMeshTerrainModeManagerCommands::Get().MeshSelectionModeAction_NoSelection);
				MeshElementSelectionOptionsSection.AddSeparator(NAME_None);
				CreateToolMenuEntry(FMeshTerrainModeManagerCommands::Get().MeshSelectionModeAction_MeshVertices);
				CreateToolMenuEntry(FMeshTerrainModeManagerCommands::Get().MeshSelectionModeAction_MeshEdges);
				CreateToolMenuEntry(FMeshTerrainModeManagerCommands::Get().MeshSelectionModeAction_MeshTriangles);
				MeshElementSelectionOptionsSection.AddSeparator(NAME_None);
				CreateToolMenuEntry(FMeshTerrainModeManagerCommands::Get().MeshSelectionModeAction_GroupCorners);
				CreateToolMenuEntry(FMeshTerrainModeManagerCommands::Get().MeshSelectionModeAction_GroupEdges);
				CreateToolMenuEntry(FMeshTerrainModeManagerCommands::Get().MeshSelectionModeAction_GroupFaces);

				// Section for Selection Edits
				{
					FToolMenuSection& SelectionEditsSection =
						Submenu->FindOrAddSection("SelectionEdits", LOCTEXT("SelectionEditsLabel", "Selection Edits"));
					SelectionEditsSection.AddEntry(
					FToolMenuEntry::InitMenuEntryWithCommandList(
						FMeshTerrainModeManagerCommands::Get().BeginSelectionAction_SelectAll, CommandList));
					SelectionEditsSection.AddEntry(
					FToolMenuEntry::InitMenuEntryWithCommandList(
						FMeshTerrainModeManagerCommands::Get().BeginSelectionAction_ExpandToConnected, CommandList));
					SelectionEditsSection.AddEntry(
					FToolMenuEntry::InitMenuEntryWithCommandList(
						FMeshTerrainModeManagerCommands::Get().BeginSelectionAction_Invert, CommandList));
					SelectionEditsSection.AddEntry(
					FToolMenuEntry::InitMenuEntryWithCommandList(
						FMeshTerrainModeManagerCommands::Get().BeginSelectionAction_InvertConnected, CommandList));
					SelectionEditsSection.AddEntry(
					FToolMenuEntry::InitMenuEntryWithCommandList(
						FMeshTerrainModeManagerCommands::Get().BeginSelectionAction_Expand, CommandList));
					SelectionEditsSection.AddEntry(
					FToolMenuEntry::InitMenuEntryWithCommandList(
						FMeshTerrainModeManagerCommands::Get().BeginSelectionAction_Contract, CommandList));
				}

				// Section for Drag Mode setting
				{
					FToolMenuSection& DragModeSection =
						Submenu->FindOrAddSection("Drag Mode", LOCTEXT("DragModeLabel", "Drag Mode"));
					DragModeSection.AddEntry(
						FToolMenuEntry::InitMenuEntryWithCommandList(
							FMeshTerrainModeManagerCommands::Get().SelectionDragMode_None, CommandList));
					DragModeSection.AddEntry(
						FToolMenuEntry::InitMenuEntryWithCommandList(
							FMeshTerrainModeManagerCommands::Get().SelectionDragMode_Path, CommandList));
				}

				// Section for Mesh Types
				{
					FToolMenuSection& MeshTypesSection =
						Submenu->FindOrAddSection("Selectable Mesh Types", LOCTEXT("SelectableMeshTypesLabel", "Selectable Mesh Types"));
					MeshTypesSection.AddEntry(
						FToolMenuEntry::InitMenuEntryWithCommandList(
							FMeshTerrainModeManagerCommands::Get().SelectionMeshTypes_Volumes, CommandList));
					MeshTypesSection.AddEntry(
						FToolMenuEntry::InitMenuEntryWithCommandList(
							FMeshTerrainModeManagerCommands::Get().SelectionMeshTypes_StaticMeshes, CommandList));
				}
				
				// Section for Local Frame Mode setting
				{
					FToolMenuSection& LocalFrameModeSection =
						Submenu->FindOrAddSection("Local Frame Mode", LOCTEXT("LocalFrameModeLabel", "Local Frame Mode"));
					LocalFrameModeSection.AddEntry(
						FToolMenuEntry::InitMenuEntryWithCommandList(
							FMeshTerrainModeManagerCommands::Get().SelectionLocalFrameMode_Geometry, CommandList));
					LocalFrameModeSection.AddEntry(
						FToolMenuEntry::InitMenuEntryWithCommandList(
							FMeshTerrainModeManagerCommands::Get().SelectionLocalFrameMode_Object, CommandList));
				}

				// Section for locking/unlocking target
				{
					FToolMenuSection& LockingSection =
						Submenu->FindOrAddSection("Locking Section", FText());
					LockingSection.AddSeparator(NAME_None);
					LockingSection.AddEntry(
					FToolMenuEntry::InitMenuEntryWithCommandList(
						FMeshTerrainModeManagerCommands::Get().SelectionLocking, CommandList));
				}
				
			}
		})
	);

	Entry.ToolBarData.ResizeParams.ClippingPriority = 950;
	Entry.ToolBarData.LabelOverride = FText();
	Entry.InsertPosition = FToolMenuInsert("Transform", EToolMenuInsertType::After);
	Entry.Icon = FSlateIcon(FMeshTerrainModeStyle::GetStyleSetName(), "MeshTerrainModeManagerCommands.MeshElementSelection");
	MeshElementSelectionMenu.AddEntry(Entry);
}

void UMeshTerrainMode::RemoveModelingModeViewportToolbarExtensions()
{
	UToolMenus::Get()->UnregisterOwnerByName(MeshTerrainModeLocals::MeshTerrainModeOwnerName);
}

bool UMeshTerrainMode::ShowPressureSensitivityControls() const
{
	if (const FMeshTerrainModeToolkit* const ModeToolkit = static_cast<FMeshTerrainModeToolkit*>(Toolkit.Get()))
	{
		if (ModeToolkit->StylusInputHandler)
		{
			return ModeToolkit->StylusInputHandler->AnyContextSupportsPressure();
		}
	}
	return false;
}

bool UMeshTerrainMode::CreateVSculptToolWidgets(FProperty* Prop, UObject* PropListOwner, TArray<UE::MeshTerrain::FPropertyWidget>& WidgetsToAdd,
	UInteractiveTool* Tool, const UE::MeshTerrain::EQuickPropertyDisplay DisplayType)
{
	if (Prop->GetFName() == FName("BrushSize"))
	{
		return UE::MeshTerrain::CreateBrushSizeToggleWidget(Prop, PropListOwner, WidgetsToAdd, Tool, DisplayType,
			FIsActionButtonVisible::CreateUObject(this, &UMeshTerrainMode::ShowPressureSensitivityControls));
	}
	if (Prop->GetFName() == FName("PrimaryFalloffType"))
	{
		return UE::MeshTerrain::CreateFalloffWidget(Prop, PropListOwner, WidgetsToAdd, Tool, DisplayType);
	}
	if (Prop->GetFName() == FName("Alpha"))
	{
		return UE::MeshTerrain::CreateAlphaWidget(Prop, PropListOwner, WidgetsToAdd, Tool, DisplayType);
	}
	if (Prop->GetFName() == FName("Strength"))
	{
		return UE::MeshTerrain::CreateStrengthWidget(Prop, PropListOwner, WidgetsToAdd, Tool, DisplayType,
			FIsActionButtonVisible::CreateUObject(this, &UMeshTerrainMode::ShowPressureSensitivityControls));
	}
	if (Prop->GetFName() == FName("bAllowEditorGizmo"))
	{
		return UE::MeshTerrain::CreateEditorGizmoToggleWidget(Prop, PropListOwner, WidgetsToAdd, Tool, DisplayType);
	}
	if (Prop->GetFName() == FName("bShowGizmo"))
	{
		return UE::MeshTerrain::CreateShowGizmoToggleWidget(Prop, PropListOwner, WidgetsToAdd, Tool, DisplayType);
	}
	return false;
}

bool UMeshTerrainMode::CreatePaintWeightsToolWidgets(FProperty* Prop, UObject* PropListOwner, TArray<UE::MeshTerrain::FPropertyWidget>& WidgetsToAdd,
	UInteractiveTool* Tool, const UE::MeshTerrain::EQuickPropertyDisplay DisplayType)
{
	if (Prop->GetFName() == FName("BrushRadius"))
	{
		return UE::MeshTerrain::CreateBrushRadiusWidget(Prop, PropListOwner, WidgetsToAdd, Tool, DisplayType);
	}
	if (Prop->GetFName() == FName("BrushSize"))
	{
		return UE::MeshTerrain::CreateBrushSizeWidget(Prop, PropListOwner, WidgetsToAdd, Tool, DisplayType,
			FIsActionButtonVisible::CreateUObject(this, &UMeshTerrainMode::ShowPressureSensitivityControls));
	}
	return false;
}

void UMeshTerrainMode::OnToolsContextRender(IToolsContextRenderAPI* RenderAPI)
{
	if (SelectionInteraction)
	{
		SelectionInteraction->Render(RenderAPI);

		// Bake in transform changes. Note that if we do this in OnToolsContextTick, it will 
		// still block rendering updates if it is too expensive, unless it is only done every second Tick
		SelectionInteraction->ApplyPendingTransformInteractions();
	}

	if (GetSelectionManager())
	{
		// currently relying on debug rendering to visualize selections
		GetSelectionManager()->DebugRender(RenderAPI);
	}
}

void UMeshTerrainMode::OnToolsContextDrawHUD(FCanvas* Canvas, IToolsContextRenderAPI* RenderAPI)
{
	if (SelectionInteraction)
	{
		SelectionInteraction->DrawHUD(Canvas, RenderAPI);
	}
}

bool UMeshTerrainMode::ShouldToolStartBeAllowed(const FString& ToolIdentifier) const
{
	if (UInteractiveToolManager* Manager = GetToolManager())
	{
		if (UInteractiveTool* Tool = Manager->GetActiveTool(EToolSide::Left))
		{
			IInteractiveToolExclusiveToolAPI* ExclusiveAPI = Cast<IInteractiveToolExclusiveToolAPI>(Tool);
			if (ExclusiveAPI)
			{
				return false;
			}
		}
	}
	return Super::ShouldToolStartBeAllowed(ToolIdentifier);
}


bool UMeshTerrainMode::GetGeometrySelectionChangesAllowed() const
{
	// disable selection system if it is...disabled
	if (GetMeshElementSelectionSystemEnabled() == false)
	{
		return false;
	}

	// disable selection system if we are in a Tool
	if ( GetToolManager() && GetToolManager()->HasAnyActiveTool() )
	{
		return false;
	}
	return true;
}

bool UMeshTerrainMode::TestForEditorGizmoHit(const FInputDeviceRay& ClickPos) const
{
	// Because the editor gizmo does not participate in InputRouter behavior system, in some input behaviors
	// we need to filter out clicks on the gizmo. This function can do this check.
	if ( ShouldDrawWidget() )
	{
		if (FViewport* FocusedViewport = GetToolManager()->GetContextQueriesAPI()->GetFocusedViewport())
		{
			HHitProxy* HitResult = FocusedViewport->GetHitProxy(ClickPos.ScreenPosition.X, ClickPos.ScreenPosition.Y);
			if (HitResult && HitResult->IsA(HWidgetAxis::StaticGetType()))
			{
				return true;
			}
		}
	}
	return false;
}


bool UMeshTerrainMode::GetMeshElementSelectionSystemEnabled() const
{
	return bSelectionSystemEnabled;
}

void UMeshTerrainMode::NotifySelectionSystemEnabledStateModified()
{
	UMeshTerrainModeSettings* Settings = GetMutableDefault<UMeshTerrainModeSettings>();
	const bool bNewState = Settings->GetMeshSelectionsEnabled();
	if (bNewState != bSelectionSystemEnabled)
	{
		if (bNewState == true)
		{
			UpdateSelectionManagerOnEditorSelectionChange(true);		// do like a mode enter so that we get an undoable active-target state
		}
		else
		{
			if (SelectionManager && (SelectionManager->HasSelection() || SelectionManager->HasActiveTargets()))
			{
				GetToolManager()->GetContextTransactionsAPI()->BeginUndoTransaction(LOCTEXT("InitializeSelection", "Initialize Selection"));

				SelectionManager->SynchronizeActiveTargets(TArray<FGeometryIdentifier>(),
					[this]() { SelectionManager->ClearSelection(); });

				GetToolManager()->GetContextTransactionsAPI()->EndUndoTransaction();
			}
		}

		// update things

		bSelectionSystemEnabled = bNewState;
		GetInteractiveToolsContext()->PostInvalidation();
	}
}


void UMeshTerrainMode::UpdateSelectionManagerOnEditorSelectionChange(bool bEnteringMode)
{
	if (GetMeshElementSelectionSystemEnabled() == false || SelectionManager == nullptr) return;

	// if we are in undo/redo, ignore selection change notifications, the required
	// changes are handled via FChanges that SelectionManager has emitted
	if (GIsTransacting)
	{
		return;
	}

	// Find selected Component types that are currently supported. Currently determining this via explicit casting,
	// probably it should be handled by the Selector Factories registered in UMeshTerrainMode::Enter(),
	// possibly via the SelectionManager. 

	TArray<UDynamicMeshComponent*> SelectedDynamicMeshComponents;
	TArray<UStaticMeshComponent*> SelectedStaticMeshComponents;
	TArray<UBrushComponent*> SelectedBrushComponents;

	if (GEditor->GetSelectedComponents()->Num() > 0)
	{
		// if we have supported Components selected on a multi-Component Actor, they will be
		// returned via these functions
		GEditor->GetSelectedComponents()->GetSelectedObjects<UDynamicMeshComponent>(SelectedDynamicMeshComponents);
		GEditor->GetSelectedComponents()->GetSelectedObjects<UStaticMeshComponent>(SelectedStaticMeshComponents);
		GEditor->GetSelectedComponents()->GetSelectedObjects<UBrushComponent>(SelectedBrushComponents);
	}
	// Conceivably this could be an 'else', however currently in the Editor when a Volume Actor is selected,
	// GetSelectedComponents()->Num() > 0 but no BrushComponent will be found (it appears to be some kind of TypedElement wrapper).
	// However note that this might result in some unexpected Meshes being Selectable on multi-Component Actors
	if (SelectedDynamicMeshComponents.Num() == 0 && SelectedStaticMeshComponents.Num() == 0 && SelectedBrushComponents.Num() == 0)
	{
		// assume Actor selection, find all valid Components on the selected Actors
		TArray<AActor*> SelectedActors;
		GEditor->GetSelectedActors()->GetSelectedObjects<AActor>(SelectedActors);
		for (AActor* Actor : SelectedActors)
		{
			Actor->ForEachComponent(false, [&](UActorComponent* Component) 
			{
				if (UDynamicMeshComponent* DynamicMeshComponent = Cast<UDynamicMeshComponent>(Component))
				{
					SelectedDynamicMeshComponents.Add(DynamicMeshComponent);
				}
				if (UStaticMeshComponent* StaticMeshComponent = Cast<UStaticMeshComponent>(Component))
				{
					SelectedStaticMeshComponents.Add(StaticMeshComponent);
				}
				if (UBrushComponent* BrushComponent = Cast<UBrushComponent>(Component))
				{
					SelectedBrushComponents.Add(BrushComponent);
				}
			});
		}
	}

	// filter out any dynamic mesh components that aren't editable or aren't element-selectable
	SelectedDynamicMeshComponents.SetNum(Algo::RemoveIf(SelectedDynamicMeshComponents, [](UDynamicMeshComponent* DMC)->bool {return !DMC->IsEditable() || !DMC->AllowsGeometrySelection(); }));

	// convert selected Component types into selection Identifiers
	TArray<FGeometryIdentifier> ValidIdentifiers;
	for (UDynamicMeshComponent* DynamicMeshComponent : SelectedDynamicMeshComponents)
	{
		ValidIdentifiers.Add(FGeometryIdentifier::PrimitiveComponent(DynamicMeshComponent, FGeometryIdentifier::EObjectType::DynamicMeshComponent));
	}
	if (bEnableStaticMeshElementSelection)
	{
		for (UStaticMeshComponent* StaticMeshComponent : SelectedStaticMeshComponents)
		{
			ValidIdentifiers.Add(FGeometryIdentifier::PrimitiveComponent(StaticMeshComponent, FGeometryIdentifier::EObjectType::StaticMeshComponent));
		}
	}
	if (bEnableVolumeElementSelection)
	{
		for (UBrushComponent* BrushComponent : SelectedBrushComponents)
		{
			ValidIdentifiers.Add(FGeometryIdentifier::PrimitiveComponent(BrushComponent, FGeometryIdentifier::EObjectType::BrushComponent));
		}
	}


	// This is gross. If we are entering the Mode, we need to update the SelectionManager w/ the current state. However this
	// update needs to be undoable. Since we are not part of whatever Transaction was involved in changing modes, we are going
	// to have to emit our own Transaction, which will then be an explicit undo step the user has to go through :(
	bool bPendingCloseTransaction = false;
	if (bEnteringMode && ValidIdentifiers.Num() > 0)
	{
		GetToolManager()->GetContextTransactionsAPI()->BeginUndoTransaction(LOCTEXT("InitializeSelection", "Initialize Selection"));
		bPendingCloseTransaction = true;
	}

	// If Editor is creating a transaction, we assume we must be in a selection change.
	// Need to handle all SelectionManager changes (deselect + change-targets) during
	// the transaction so that undo/redo works properly.
	// (note that if we are bEnteringMode, we just opened a transaction and so this branch will still be taken...)
	bool bCreatingTransaction = (GUndo != nullptr);
	if (bCreatingTransaction)
	{
		SelectionManager->SynchronizeActiveTargets(ValidIdentifiers,
			[this]() {
				SelectionManager->ClearSelection();
			});
	}

	// close out transaction if it was still open
	if (bPendingCloseTransaction)
	{
		GetToolManager()->GetContextTransactionsAPI()->EndUndoTransaction();
	}
}


bool UMeshTerrainMode::BoxSelect(FBox& InBox, bool InSelect) 
{
	// not handling yet
	return false;
}

bool UMeshTerrainMode::FrustumSelect(const FConvexVolume& InFrustum, FEditorViewportClient* InViewportClient, bool InSelect)
{
	if (GetMeshElementSelectionSystemEnabled()
		&& SelectionManager
		&& SelectionManager->HasActiveTargets() 
		&& SelectionManager->GetMeshTopologyMode() != UGeometrySelectionManager::EMeshTopologyMode::None)
	{
		UE::Geometry::FGeometrySelectionUpdateConfig UpdateConfig;
		UpdateConfig.ChangeType = UE::Geometry::EGeometrySelectionChangeType::Replace;
		if ( InViewportClient->IsShiftPressed() )
		{
			UpdateConfig.ChangeType = UE::Geometry::EGeometrySelectionChangeType::Add;
		}
		else if ( InViewportClient->IsCtrlPressed() && InViewportClient->IsAltPressed() == false )
		{
			UpdateConfig.ChangeType = UE::Geometry::EGeometrySelectionChangeType::Remove;
		}			

		UE::Geometry::FGeometrySelectionUpdateResult Result;
		SelectionManager->UpdateSelectionViaConvex(
			InFrustum, UpdateConfig, Result);

		return true;	// always consume marquee even if it missed, as the miss will usually just be a mistake
	}

	// not handling yet
	return false;
}



void UMeshTerrainMode::CreateToolkit()
{
	Toolkit = MakeShareable(new FMeshTerrainModeToolkit);
}

void UMeshTerrainMode::OnToolPostBuild(
	UInteractiveToolManager* InToolManager, EToolSide InSide, 
	UInteractiveTool* InBuiltTool, UInteractiveToolBuilder* InToolBuilder, const FToolBuilderState& ToolState)
{
	// Snapshot the actors the tool was launched against
	ActiveToolTargetActors.Reset();
	for (AActor* const Actor : ToolState.SelectedActors)
	{
		if (Actor)
		{
			ActiveToolTargetActors.Add(Actor);
		}
	}

	// Want to clear active selection when a Tool starts, but we have to wait until after it has been
	// built, so that the Tool has a chance to see the Selection
	if (GetSelectionManager() && GetSelectionManager()->HasSelection())
	{
		ensureMsgf(!GetSelectionManager()->HasSavedSelection(), TEXT("Selection manager should not already have a saved selection before we save-on-clear here in tool setup."));
		GetSelectionManager()->ClearSelection(/*bSaveSelectionBeforeClear*/ true);
	}
}

void UMeshTerrainMode::OnToolStarted(UInteractiveToolManager* Manager, UInteractiveTool* Tool)
{
	using namespace MeshTerrainModeLocals;

	// disable slate throttling so that Tool background computes responding to sliders can properly be processed
	// on Tool Tick. Otherwise, when a Tool kicks off a background update in a background thread, the computed
	// result will be ignored until the user moves the slider, ie you cannot hold down the mouse and wait to see
	// the result. This apparently broken behavior is currently by-design.
	FSlateThrottleManager::Get().DisableThrottle(true);
	
	bIsToolActive = true;

	FMeshTerrainModeToolActionCommands::UpdateToolCommandBinding(Tool, Toolkit->GetToolkitCommands(), false);
	
	if( FEngineAnalytics::IsAvailable() )
	{
		FEngineAnalytics::GetProvider().RecordEvent(TEXT("Editor.Usage.MeshTerrainMode.ToolStarted"),
		                                            TEXT("ToolName"),
		                                            GetToolName(*Tool));
	}

	// Track the last target+tool for the relaunch hotkey.
	// We check the tool's actual target rather than the editor selection to ensure
	// the tool is genuinely operating on the target.
	if (USingleSelectionTool* const SingleSelectionTool = Cast<USingleSelectionTool>(Tool))
	{
		UToolTarget* const Target = SingleSelectionTool->GetTarget();

		// Helper to record state common to all target types.
		const auto RecordLastTool = [this, Manager](TArray<TWeakObjectPtr<USceneComponent>>&& Components)
		{
			LastTargetComponents = MoveTemp(Components);
			LastToolIdentifier = Manager->GetActiveToolName(EToolSide::Mouse);
			if (const FMeshTerrainModeToolkit* const ModeToolkit = static_cast<FMeshTerrainModeToolkit*>(Toolkit.Get()))
			{
				LastSubmodeName = ModeToolkit->ActiveSubmode ? ModeToolkit->ActiveSubmode->GetName() : NAME_None;
			}
		};

		if (const UE::MeshPartition::UModifierToolTarget* const ModifierTarget =
				Cast<UE::MeshPartition::UModifierToolTarget>(Target))
		{
			UE::MeshPartition::UEditableModifierBase* const Modifier =
				Cast<UE::MeshPartition::UEditableModifierBase>(ModifierTarget->GetOwnerSceneComponent());
			if (Modifier)
			{
				RecordLastTool({ Modifier });
			}
		}
		else if (const UE::MeshPartition::UMultiSectionToolTarget* const MultiTarget =
					 Cast<UE::MeshPartition::UMultiSectionToolTarget>(Target))
		{
			TArray<TWeakObjectPtr<USceneComponent>> Components;
			for (const TWeakObjectPtr<UE::MeshPartition::UMeshProviderModifier>& Section : MultiTarget->GetBaseSections())
			{
				if (Section.IsValid())
				{
					Components.Add(Section.Get());
				}
			}
			if (!Components.IsEmpty())
			{
				RecordLastTool(MoveTemp(Components));
			}
		}
		else if (const UE::MeshPartition::USectionToolTarget* const SectionTarget =
					 Cast<UE::MeshPartition::USectionToolTarget>(Target))
		{
			if (USceneComponent* const OwnerComponent = Cast<USceneComponent>(SectionTarget->GetOwnerComponent()))
			{
				UE::MeshPartition::UMeshProviderModifier* const BaseSection =
					Cast<UE::MeshPartition::UMeshProviderModifier>(OwnerComponent->GetAttachParent());
				if (BaseSection)
				{
					RecordLastTool({ BaseSection });
				}
			}
		}
		else if (const UE::MeshPartition::UMeshPartitionToolTarget* const PartitionTarget =
					 Cast<UE::MeshPartition::UMeshPartitionToolTarget>(Target))
		{
			if (USceneComponent* const EditorComponent = PartitionTarget->GetOwnerSceneComponent())
			{
				RecordLastTool({ EditorComponent });
			}
		}
	}

	UContextObjectStore* ContextStore = GetInteractiveToolsContext()->ToolManager->GetContextObjectStore();
	UToolsContextCursorAPI* ToolsContextCursorAPI = ContextStore->FindContext<UToolsContextCursorAPI>();
	if (ToolsContextCursorAPI)
	{
		ToolsContextCursorAPI->ClearCursorOverride();		
	}
}

void UMeshTerrainMode::OnToolEnded(UInteractiveToolManager* Manager, UInteractiveTool* Tool)
{
	using namespace MeshTerrainModeLocals;

	ActiveToolTargetActors.Reset();

	// re-enable slate throttling (see OnToolStarted)
	FSlateThrottleManager::Get().DisableThrottle(false);

	bIsToolActive = false;

	FMeshTerrainModeToolActionCommands::UpdateToolCommandBinding(Tool, Toolkit->GetToolkitCommands(), true);
	
	// We may require a gizmo location update despite not having changed the selection (transform tool,
	// edit pivot, etc).
	GUnrealEd->UpdatePivotLocationForSelection();

	if( FEngineAnalytics::IsAvailable() )
	{
		FEngineAnalytics::GetProvider().RecordEvent(TEXT("Editor.Usage.MeshTerrainMode.ToolEnded"),
		                                            TEXT("ToolName"),
		                                            GetToolName(*Tool));
	}

	UContextObjectStore* ContextStore = GetInteractiveToolsContext()->ToolManager->GetContextObjectStore();
	UToolsContextCursorAPI* ToolsContextCursorAPI = ContextStore->FindContext<UToolsContextCursorAPI>();
	if (ToolsContextCursorAPI)
	{
		ToolsContextCursorAPI->ClearCursorOverride();		
	}
}

void UMeshTerrainMode::BindCommands()
{
	Super::BindCommands();

	const FMeshTerrainModeManagerCommands& ToolManagerCommands = FMeshTerrainModeManagerCommands::Get();
	const TSharedRef<FUICommandList>& CommandList = Toolkit->GetToolkitCommands();

	CommandList->MapAction(
		ToolManagerCommands.AcceptActiveTool,
		FExecuteAction::CreateLambda([this]() { 
			GetInteractiveToolsContext()->EndTool(EToolShutdownType::Accept); 
		}),
		FCanExecuteAction::CreateLambda([this]() { return GetInteractiveToolsContext()->CanAcceptActiveTool(); }),
		FGetActionCheckState(),
		FIsActionButtonVisible::CreateLambda([this]() {return GetInteractiveToolsContext()->ActiveToolHasAccept(); }),
		EUIActionRepeatMode::RepeatDisabled
		);

	CommandList->MapAction(
		ToolManagerCommands.CancelActiveTool,
		FExecuteAction::CreateLambda([this]() { GetInteractiveToolsContext()->EndTool(EToolShutdownType::Cancel); }),
		FCanExecuteAction::CreateLambda([this]() { return GetInteractiveToolsContext()->CanCancelActiveTool(); }),
		FGetActionCheckState(),
		FIsActionButtonVisible::CreateLambda([this]() {return GetInteractiveToolsContext()->ActiveToolHasAccept(); }),
		EUIActionRepeatMode::RepeatDisabled
		);

	CommandList->MapAction(
		ToolManagerCommands.CompleteActiveTool,
		FExecuteAction::CreateLambda([this]() { GetInteractiveToolsContext()->EndTool(EToolShutdownType::Completed); }),
		FCanExecuteAction::CreateLambda([this]() { return GetInteractiveToolsContext()->CanCompleteActiveTool(); }),
		FGetActionCheckState(),
		FIsActionButtonVisible::CreateLambda([this]() {return GetInteractiveToolsContext()->CanCompleteActiveTool(); }),
		EUIActionRepeatMode::RepeatDisabled
		);

	// These aren't activated by buttons but have default chords that bind the keypresses to the action.
	CommandList->MapAction(
		ToolManagerCommands.AcceptOrCompleteActiveTool,
		FExecuteAction::CreateLambda([this]() { AcceptActiveToolActionOrTool(); }),
		FCanExecuteAction::CreateLambda([this]() {
				return GetInteractiveToolsContext()->CanAcceptActiveTool() || GetInteractiveToolsContext()->CanCompleteActiveTool();
			}),
		FGetActionCheckState(),
		FIsActionButtonVisible(),
		EUIActionRepeatMode::RepeatDisabled);

	CommandList->MapAction(
		ToolManagerCommands.CancelOrCompleteActiveTool,
		FExecuteAction::CreateLambda([this]() { OnEscapeKey(); }),
		FCanExecuteAction::CreateLambda([this]() {
				return GetInteractiveToolsContext()->CanCompleteActiveTool() || GetInteractiveToolsContext()->CanCancelActiveTool();
			}),
		FGetActionCheckState(),
		FIsActionButtonVisible(),
		EUIActionRepeatMode::RepeatDisabled);

	// Relaunch last tool. Hotkey is unbound by default - this is an advanced workflow
	// and users can assign their preferred binding through Editor Preferences.
	CommandList->MapAction(
		ToolManagerCommands.RelaunchLastTool,
		FExecuteAction::CreateLambda([this]() { ExecuteRelaunchLastTool(); }),
		FCanExecuteAction::CreateLambda([this]() {
			return !LastTargetComponents.IsEmpty() && !LastToolIdentifier.IsEmpty();
		}));
}


void UMeshTerrainMode::AcceptActiveToolActionOrTool()
{
	// if we have an active Tool that implements 
	if (GetToolManager()->HasAnyActiveTool())
	{
		UInteractiveTool* Tool = GetToolManager()->GetActiveTool(EToolSide::Mouse);
		IInteractiveToolNestedAcceptCancelAPI* CancelAPI = Cast<IInteractiveToolNestedAcceptCancelAPI>(Tool);
		if (CancelAPI && CancelAPI->SupportsNestedAcceptCommand() && CancelAPI->CanCurrentlyNestedAccept())
		{
			bool bAccepted = CancelAPI->ExecuteNestedAcceptCommand();
			if (bAccepted)
			{
				return;
			}
		}
	}

	const EToolShutdownType ShutdownType = GetInteractiveToolsContext()->CanAcceptActiveTool() ? EToolShutdownType::Accept : EToolShutdownType::Completed;
	GetInteractiveToolsContext()->EndTool(ShutdownType);
}


void UMeshTerrainMode::CancelActiveToolActionOrTool()
{
	// if we have an active Tool that implements 
	if (GetToolManager()->HasAnyActiveTool())
	{
		UInteractiveTool* Tool = GetToolManager()->GetActiveTool(EToolSide::Mouse);
		IInteractiveToolNestedAcceptCancelAPI* CancelAPI = Cast<IInteractiveToolNestedAcceptCancelAPI>(Tool);
		if (CancelAPI && CancelAPI->SupportsNestedCancelCommand() && CancelAPI->CanCurrentlyNestedCancel())
		{
			bool bCancelled = CancelAPI->ExecuteNestedCancelCommand();
			if (bCancelled)
			{
				return;
			}
		}
	}

	const EToolShutdownType ShutdownType = GetInteractiveToolsContext()->CanCancelActiveTool() ? EToolShutdownType::Cancel : EToolShutdownType::Completed;
	GetInteractiveToolsContext()->EndTool(ShutdownType);
}

void UMeshTerrainMode::ExecuteRelaunchLastTool()
{
	if (LastTargetComponents.IsEmpty() || LastToolIdentifier.IsEmpty())
	{
		return;
	}

	UEditorInteractiveToolsContext* const ToolsContext = GetInteractiveToolsContext();
	if (!ToolsContext || !ToolsContext->ToolManager)
	{
		return;
	}

	// Don't interrupt an active tool - the user should accept/cancel it first.
	if (ToolsContext->ToolManager->HasAnyActiveTool())
	{
		FNotificationInfo NotifyInfo(LOCTEXT("RelaunchToolBusy", "Accept or cancel the current tool before relaunching"));
		FSlateNotificationManager::Get().AddNotification(NotifyInfo);
		return;
	}

	// Don't start tools during undo/redo, PIE/SIE, or while the user is mid-drag.
	if (GIsTransacting || !ShouldToolStartBeAllowed(LastToolIdentifier))
	{
		return;
	}
	if (ToolsContext->InputRouter && ToolsContext->InputRouter->HasActiveMouseCapture())
	{
		return;
	}

	// Prune dead weak pointers.
	LastTargetComponents.RemoveAll([](const TWeakObjectPtr<USceneComponent>& Ptr) { return !Ptr.IsValid(); });
	if (LastTargetComponents.IsEmpty())
	{
		FNotificationInfo NotifyInfo(LOCTEXT("RelaunchToolTargetsGone", "Last target(s) no longer exist"));
		FSlateNotificationManager::Get().AddNotification(NotifyInfo);
		LastToolIdentifier.Empty();
		LastSubmodeName = NAME_None;
		return;
	}

	// If the single target is an editable modifier, check that it still supports tool editing.
	if (LastTargetComponents.Num() == 1)
	{
		if (UE::MeshPartition::UEditableModifierBase* const Modifier =
				Cast<UE::MeshPartition::UEditableModifierBase>(LastTargetComponents[0].Get()))
		{
			if (!Modifier->SupportsToolEditing())
			{
				FNotificationInfo NotifyInfo(LOCTEXT("RelaunchToolNotEditable", "Last modifier no longer supports tool editing"));
				FSlateNotificationManager::Get().AddNotification(NotifyInfo);
				return;
			}
		}
	}

	// Verify all targets are still part of valid actors in the world.
	for (const TWeakObjectPtr<USceneComponent>& ComponentPtr : LastTargetComponents)
	{
		const AActor* const OwnerActor = ComponentPtr->GetOwner();
		if (!OwnerActor || OwnerActor->GetWorld() != GetWorld())
		{
			FNotificationInfo NotifyInfo(LOCTEXT("RelaunchToolInvalidOwner", "Last target is no longer in the current level"));
			FSlateNotificationManager::Get().AddNotification(NotifyInfo);
			LastTargetComponents.Empty();
			LastToolIdentifier.Empty();
			LastSubmodeName = NAME_None;
			return;
		}
	}

	// Restore the submode that was active when the tool was last used, so that
	// overlay widgets (e.g. QuickSettings) are configured correctly for the tool.
	if (!LastSubmodeName.IsNone())
	{
		if (FMeshTerrainModeToolkit* const ModeToolkit = static_cast<FMeshTerrainModeToolkit*>(Toolkit.Get()))
		{
			const FName ActiveName = ModeToolkit->ActiveSubmode ? ModeToolkit->ActiveSubmode->GetName() : NAME_None;
			if (ActiveName != LastSubmodeName)
			{
				if (const TSharedPtr<UE::MeshTerrain::FSubmode>* const FoundSubmode = ModeToolkit->Submodes.Find(LastSubmodeName))
				{
					ModeToolkit->ActivateSubmode(*FoundSubmode);
				}
			}
		}
	}

	// Select the target component(s) in the editor.
	GEditor->SelectNone(/*bNoteSelectionChange=*/false, /*bDeselectBSPSurfs=*/true);
	for (int32 ComponentIndex = 0; ComponentIndex < LastTargetComponents.Num(); ++ComponentIndex)
	{
		const bool bNotify = (ComponentIndex == LastTargetComponents.Num() - 1);
		GEditor->SelectComponent(LastTargetComponents[ComponentIndex].Get(), /*bInSelected=*/true, bNotify);
	}

	// Start the tool.
	if (ToolsContext->ToolManager->CanActivateTool(EToolSide::Mouse, LastToolIdentifier))
	{
		ToolsContext->StartTool(LastToolIdentifier);
	}
	else
	{
		FNotificationInfo NotifyInfo(LOCTEXT("RelaunchToolCannotActivate", "Cannot start the last tool on this target"));
		FSlateNotificationManager::Get().AddNotification(NotifyInfo);
	}
}

void UMeshTerrainMode::OnEscapeKey()
{
	if (GetDefault<UMeshTerrainModeCustomizationSettings>()->bEscapeAcceptsToolResult)
	{
		AcceptActiveToolActionOrTool();
	}
	else
	{
		CancelActiveToolActionOrTool();
	}
}

void UMeshTerrainMode::ModelingModeShortcutRequested(EMeshTerrainModeActionCommands Command)
{
	if (Command == EMeshTerrainModeActionCommands::FocusViewToCursor)
	{
		FocusCameraAtCursorHotkey();
	}
	else if (Command == EMeshTerrainModeActionCommands::ToggleSelectionLockState)
	{
		if (SelectionManager)
		{
			if (SelectionManager->GetAnyCurrentTargetsLocked())
			{
				SelectionManager->SetCurrentTargetsLockState(false);
			}
			else
			{
				SelectionManager->SetCurrentTargetsLockState(true);
			}
		}
	}
}


void UMeshTerrainMode::FocusCameraAtCursorHotkey()
{
	FRay Ray = GetInteractiveToolsContext()->GetLastWorldRay();

	double NearestHitDist = (double)HALF_WORLD_MAX;
	FVector HitPoint = FVector::ZeroVector;

	// cast ray against visible objects
	FHitResult WorldHitResult;
	if (ToolSceneQueriesUtil::FindNearestVisibleObjectHit( USceneSnappingManager::Find(GetToolManager()), WorldHitResult, Ray) )
	{
		HitPoint = WorldHitResult.ImpactPoint;
		NearestHitDist = (double)Ray.GetParameter(HitPoint);
	}

	// cast ray against tool
	if (GetToolManager()->HasAnyActiveTool())
	{
		UInteractiveTool* Tool = GetToolManager()->GetActiveTool(EToolSide::Mouse);
		IInteractiveToolCameraFocusAPI* FocusAPI = Cast<IInteractiveToolCameraFocusAPI>(Tool);
		if (FocusAPI && FocusAPI->SupportsWorldSpaceFocusPoint())
		{
			FVector ToolHitPoint;
			if (FocusAPI->GetWorldSpaceFocusPoint(Ray, ToolHitPoint))
			{
				double HitDepth = (double)Ray.GetParameter(ToolHitPoint);
				if (HitDepth < NearestHitDist)
				{
					NearestHitDist = HitDepth;
					HitPoint = ToolHitPoint;
				}
			}
		}
	}


	if (NearestHitDist < (double)HALF_WORLD_MAX && GCurrentLevelEditingViewportClient)
	{
		GCurrentLevelEditingViewportClient->CenterViewportAtPoint(HitPoint, false);
	}
}


FBox UMeshTerrainMode::ComputeCustomViewportFocus() const
{
	// Modeling mode prefers a slightly farther-out focus
	auto ProcessFocusBoxFunc = [](FBox& FocusBoxInOut)
	{
		double MaxDimension = FocusBoxInOut.GetExtent().GetMax();
		FocusBoxInOut = FocusBoxInOut.ExpandBy(MaxDimension * 0.2);
	};

	FBox FocusBox = Super::ComputeCustomViewportFocus();
	if (FocusBox.IsValid)
	{
		ProcessFocusBoxFunc(FocusBox);
		return FocusBox;
	}

	// if we have an active Selection we can focus on that
	if (GetSelectionManager() && GetSelectionManager()->HasSelection())
	{
		UE::Geometry::FGeometrySelectionBounds SelectionBounds;
		GetSelectionManager()->GetSelectionBounds(SelectionBounds);
		FocusBox = (FBox)SelectionBounds.WorldBounds;
		ProcessFocusBoxFunc(FocusBox);
		return FocusBox;
	}

	// did not set a focus box, return a default (invalid) box
	return FBox();
}

bool UMeshTerrainMode::HasCustomViewportFocus() const
{
	if (Super::HasCustomViewportFocus())
	{
		return true;
	}

	// if we have an active Selection we can focus on that
	if (GetSelectionManager() && GetSelectionManager()->HasSelection())
	{
		return true;
	}

	// no mode-specific focus behavior
	return false;
}


bool UMeshTerrainMode::GetPivotForOrbit(FVector& OutPivot) const
{
	if (GCurrentLevelEditingViewportClient)
	{
		OutPivot = GCurrentLevelEditingViewportClient->GetViewTransform().GetLookAt();
		return true;
	}
	return false;
}



void UMeshTerrainMode::ConfigureRealTimeViewportsOverride(bool bEnable)
{
	FLevelEditorModule& LevelEditorModule = FModuleManager::GetModuleChecked<FLevelEditorModule>("LevelEditor");
	TSharedPtr<ILevelEditor> LevelEditor = LevelEditorModule.GetFirstLevelEditor();
	if (LevelEditor.IsValid())
	{
		TArray<TSharedPtr<SLevelViewport>> Viewports = LevelEditor->GetViewports();
		for (const TSharedPtr<SLevelViewport>& ViewportWindow : Viewports)
		{
			if (ViewportWindow.IsValid())
			{
				FEditorViewportClient& Viewport = ViewportWindow->GetAssetViewportClient();
				const FText SystemDisplayName = LOCTEXT("RealtimeOverrideMessage_MeshTerrainMode", "Mesh Terrain Mode");
				if (bEnable)
				{
					Viewport.AddRealtimeOverride(bEnable, SystemDisplayName);
				}
				else
				{
					Viewport.RemoveRealtimeOverride(SystemDisplayName, false);
				}
			}
		}
	}
}

void UMeshTerrainMode::OnEditorModeIDChanged(const FEditorModeID& InID, bool bIsActive)
{
	if (InID == GetID())
	{
		if (bIsActive)
		{
			OnActivate();
		}
		else
		{
			OnDeactivate();
		}
	}
}


void UMeshTerrainMode::OnActivate()
{
	
}


void UMeshTerrainMode::OnDeactivate()
{
	if (Toolkit.IsValid())
	{
		((FMeshTerrainModeToolkit*)Toolkit.Get())->UninitializeOnModeDeactivate();
	}
}


void UMeshTerrainMode::RegisterCustomTool(
	TSharedPtr<FUICommandInfo> UICommand,
	FString ToolIdentifier,
	UInteractiveToolBuilder* Builder,
	const TFunction<bool(UInteractiveToolManager*, EToolSide)>& ExecuteAction,
	const TFunction<bool(UInteractiveToolManager*, EToolSide)>& CanExecuteAction,
	const TFunction<bool(UInteractiveToolManager*, EToolSide)>& IsActionChecked)
{
	if (!Toolkit.IsValid())
	{
		return;
	}

	constexpr EToolsContextScope ToolScope = EToolsContextScope::EdMode;
	UEditorInteractiveToolsContext* UseToolsContext = GetInteractiveToolsContext(ToolScope);
	if (ensure(UseToolsContext != nullptr) == false)
	{
		return;
	}

	constexpr EToolSide ToolSide = EToolSide::Left;
	const TSharedRef<FUICommandList>& CommandList = Toolkit->GetToolkitCommands();
	UseToolsContext->ToolManager->RegisterToolType(ToolIdentifier, Builder);
	CommandList->MapAction(UICommand,
		FExecuteAction::CreateWeakLambda(UseToolsContext, [this, ToolIdentifier, UseToolsContext, ExecuteAction]
		{
			if (!ExecuteAction || !ExecuteAction(UseToolsContext->ToolManager, ToolSide))
			{
				UseToolsContext->StartTool(ToolIdentifier);
			}
			else
			{
				// TODO : revisit this
				((FMeshTerrainModeToolkit*)Toolkit.Get())->RebuildQuickSettingsWidget();
				((FMeshTerrainModeToolkit*)Toolkit.Get())->RebuildSectionedDetailsOverlayWidget();
			}
		}),
		FCanExecuteAction::CreateWeakLambda(UseToolsContext, [this, ToolIdentifier, UseToolsContext, CanExecuteAction]()
		{
			bool bResult = false;
			if (ShouldToolStartBeAllowed(ToolIdentifier))
			{
				bResult = CanExecuteAction ?
					CanExecuteAction(UseToolsContext->ToolManager, ToolSide) :
					UseToolsContext->ToolManager->CanActivateTool(ToolSide, ToolIdentifier);
			}
			return bResult;
		}),
		FIsActionChecked::CreateWeakLambda(UseToolsContext, [this, ToolIdentifier, UseToolsContext, IsActionChecked]()
		{
			if (IsActionChecked)
			{
				return IsActionChecked(UseToolsContext->ToolManager, ToolSide);
			}

			if (UseToolsContext->ToolManager->GetActiveTool(ToolSide))
			{
				// Read the ActiveToolType rather than the ActiveToolName so that clients can leverage
				// that storage on the ToolsContext through SelectActiveToolType to notify of changes
				// in the active tool without actually requiring a tool change.
				const FString& ActiveToolIdentifier = UseToolsContext->ToolManager->GetActiveToolType(ToolSide);
				return ActiveToolIdentifier == ToolIdentifier;
			}
			return false;
		}),
		EUIActionRepeatMode::RepeatDisabled);
}
}

#undef LOCTEXT_NAMESPACE
