// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Tools/LegacyEdModeWidgetHelpers.h"
#include "Widgets/MeshTerrainModeUtil.h"

#include "MeshTerrainMode.generated.h"

class IToolsContextRenderAPI;
enum class EToolSide;
struct FInputDeviceRay;
struct FToolBuilderState;
struct FHitResult;

class FEditorComponentSourceFactory;
class FUICommandList;
class FLevelObjectsObserver;
class FMenuBuilder;
class UModelingSceneSnappingManager;
class UMeshTerrainModeSelectionInteraction;
class UGeometrySelectionManager;
class UInteractiveCommand;
class UInteractiveToolManager;
class UBlueprint;

namespace UE::MeshTerrain
{
enum class EMeshTerrainModeActionCommands;

struct FClickContext
{
	FHitResult* HitResult = nullptr;
	FMenuBuilder* Builder = nullptr;
};

UCLASS(Transient)
class UMeshTerrainMode : public UBaseLegacyWidgetEdMode, public ILegacyEdModeSelectInterface
{
	GENERATED_BODY()
public:
	MESHTERRAINMODE_API const static FEditorModeID EM_MeshTerrainEditorModeId;

	UMeshTerrainMode();
	UMeshTerrainMode(FVTableHelper& Helper);
	~UMeshTerrainMode();

	////////////////
	// UEdMode interface
	////////////////

	virtual bool HandleClick(FEditorViewportClient* ViewportClient, HHitProxy* HitProxy, const FViewportClick& InClick) override;
	
	virtual void Tick(FEditorViewportClient* ViewportClient, float DeltaTime) override;

	virtual void ActorSelectionChangeNotify() override;

	virtual bool ShouldDrawWidget() const override;
	virtual bool ProcessEditDelete() override;
	virtual bool ProcessEditCut() override;
	virtual EEditAction::Type GetActionEditDuplicate() override;

	virtual bool CanAutoSave() const override;

	virtual bool GetPivotForOrbit(FVector& OutPivot) const override;

	virtual bool RequiresLegacyViewportInteractions() const override { return false; }

	/*
	 * focus events
	 */

	// called when we "start" this editor mode (ie switch to this tab)
	virtual void Enter() override;

	// called when we "end" this editor mode (ie switch to another tab)
	virtual void Exit() override;

	virtual bool ShouldToolStartBeAllowed(const FString& ToolIdentifier) const override;

	//////////////////
	// End of UEdMode interface
	//////////////////


	// ILegacyEdModeSelectInterface
	virtual bool BoxSelect(FBox& InBox, bool InSelect = true) override;
	virtual bool FrustumSelect(const FConvexVolume& InFrustum, FEditorViewportClient* InViewportClient, bool InSelect) override;


	// Manage viewport focus
	virtual bool HasCustomViewportFocus() const override;
	virtual FBox ComputeCustomViewportFocus() const override;



	//
	// Selection System configuration, this will likely move elsewhere
	//

	virtual UGeometrySelectionManager* GetSelectionManager() const
	{
		return SelectionManager;
	}
	virtual UMeshTerrainModeSelectionInteraction* GetSelectionInteraction() const
	{
		return SelectionInteraction;
	}

	UPROPERTY()
	bool bEnableVolumeElementSelection = true;

	UPROPERTY()
	bool bEnableStaticMeshElementSelection = true;

	bool GetMeshElementSelectionSystemEnabled() const;
	void NotifySelectionSystemEnabledStateModified();

protected:
	virtual void BindCommands() override;
	virtual void CreateToolkit() override;
	virtual void OnToolStarted(UInteractiveToolManager* Manager, UInteractiveTool* Tool) override;
	virtual void OnToolEnded(UInteractiveToolManager* Manager, UInteractiveTool* Tool) override;
	
	virtual void OnToolPostBuild(UInteractiveToolManager* InToolManager, EToolSide InSide, UInteractiveTool* InBuiltTool, UInteractiveToolBuilder* InToolBuilder, const FToolBuilderState& ToolState);

	// Method to optionally register the UV Editor launcher in the Modeling Mode UV category if the plugin is available.
	void RegisterUVEditor();

	FDelegateHandle MeshCreatedEventHandle;
	FDelegateHandle TextureCreatedEventHandle;
	FDelegateHandle MaterialCreatedEventHandle;
	FDelegateHandle SelectionModifiedEventHandle;

	FDelegateHandle EditorClosedEventHandle;
	void OnEditorClosed();

	TSharedPtr<FLevelObjectsObserver> LevelObjectsObserver;

	UPROPERTY()
	TObjectPtr<UModelingSceneSnappingManager> SceneSnappingManager;

	void InitializeGeometrySelection();
	void ShutdownGeometrySelection();
	
	TArray<FString> RegisteredSelectionTools;

	UPROPERTY()
	TObjectPtr<UGeometrySelectionManager> SelectionManager;

	UPROPERTY()
	TObjectPtr<UMeshTerrainModeSelectionInteraction> SelectionInteraction;

	FDelegateHandle SelectionManager_SelectionModifiedHandle;

	bool GetGeometrySelectionChangesAllowed() const;
	bool TestForEditorGizmoHit(const FInputDeviceRay&) const;


	bool bSelectionSystemEnabled = false;

	void UpdateSelectionManagerOnEditorSelectionChange(bool bEnteringMode = false);

	void OnToolsContextRender(IToolsContextRenderAPI* RenderAPI);
	void OnToolsContextDrawHUD(FCanvas* Canvas, IToolsContextRenderAPI* RenderAPI);

	void ModelingModeShortcutRequested(EMeshTerrainModeActionCommands Command);
	void FocusCameraAtCursorHotkey();

	void AcceptActiveToolActionOrTool();
	void CancelActiveToolActionOrTool();
	void OnEscapeKey();

	void ConfigureRealTimeViewportsOverride(bool bEnable);

	FDelegateHandle BlueprintPreCompileHandle;
	void OnBlueprintPreCompile(UBlueprint* Blueprint);


	// UInteractiveCommand support. Currently implemented by creating instances of
	// commands on mode startup and holding onto them. This perhaps should be revisited,
	// command instances could probably be created as needed...

	UPROPERTY()
	TArray<TObjectPtr<UInteractiveCommand>> ModelingModeCommands;


	// analytics tracking
	static FDateTime LastModeStartTimestamp;
	static FDateTime LastToolStartTimestamp;

	// tracking of unlocked stuff
	static FDelegateHandle GlobalModelingWorldTeardownEventHandle;
	
	/** Re-select the last target(s) and restart the last tool. */
	void ExecuteRelaunchLastTool();

	/** Last-tool state for "Relaunch Last Tool" hotkey.
	    1 UEditableModifierBase (modifier), N UMeshProviderModifiers (base sections),
	    or 1 UMeshPartitionEditorComponent (whole mesh partition). */
	TArray<TWeakObjectPtr<USceneComponent>> LastTargetComponents;
	FString LastToolIdentifier;
	FName LastSubmodeName;

private:
	bool bIsToolActive = false;

	// Actors that the active tool was launched against, captured at OnToolPostBuild and cleared at OnToolEnded.
	TSet<TWeakObjectPtr<AActor>> ActiveToolTargetActors;

	void OnEditorModeIDChanged(const FEditorModeID& InID, bool bIsActive);
	void OnActivate();
	void OnDeactivate();
	
	// add modeling-mode-specific portion of new viewport toolbar
	static void PopulateModelingModeViewportToolbar(const FName InMenuName, const TSharedPtr<const FUICommandList>& InCommandList);
	// remove modeling-mode-specific portion of new viewport toolbar
	static void RemoveModelingModeViewportToolbarExtensions();

	// appends all customized Widgets connected to the MeshVertexSculptTool to the provided Array
	bool CreateVSculptToolWidgets(FProperty* Prop, UObject* PropListOwner, TArray<UE::MeshTerrain::FPropertyWidget>& WidgetsToAdd,
		UInteractiveTool* Tool, const UE::MeshTerrain::EQuickPropertyDisplay DisplayType = UE::MeshTerrain::EQuickPropertyDisplay::WidgetAndLabel);

	// appends all customized Widgets connected to the AttributePaintTool to the provided Array
	bool CreatePaintWeightsToolWidgets(FProperty* Prop, UObject* PropListOwner, TArray<UE::MeshTerrain::FPropertyWidget>& WidgetsToAdd,
	UInteractiveTool* Tool, const UE::MeshTerrain::EQuickPropertyDisplay DisplayType = UE::MeshTerrain::EQuickPropertyDisplay::WidgetAndLabel);
	
	bool ShowPressureSensitivityControls() const;
	
	void RegisterCustomTool(
		TSharedPtr<FUICommandInfo> UICommand,
		FString ToolIdentifier,
		UInteractiveToolBuilder* Builder,
		const TFunction<bool(UInteractiveToolManager*, EToolSide)>& ExecuteAction,
		const TFunction<bool(UInteractiveToolManager*, EToolSide)>& CanExecuteAction,
		const TFunction<bool(UInteractiveToolManager*, EToolSide)>& IsActionChecked);
};

}