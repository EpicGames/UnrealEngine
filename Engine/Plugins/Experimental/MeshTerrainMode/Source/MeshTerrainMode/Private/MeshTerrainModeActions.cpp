// Copyright Epic Games, Inc. All Rights Reserved.

#include "MeshTerrainModeActions.h"
#include "Styling/AppStyle.h"
#include "DynamicMeshSculptTool.h"
#include "EditMeshMaterialsTool.h"
#include "MeshVertexSculptTool.h"
#include "MeshGroupPaintTool.h"
#include "MeshVertexPaintTool.h"
#include "MeshPartitionAttributePaintTool.h"
#include "MeshInspectorTool.h"
#include "CubeGridTool.h"
#include "DrawPolygonTool.h"
#include "EditMeshPolygonsTool.h"
#include "ShapeSprayTool.h"
#include "MeshSpaceDeformerTool.h"
#include "Tools/StandardToolModeCommands.h"
#include "TransformMeshesTool.h"
#include "PlaneCutTool.h"
#include "EditMeshPolygonsTool.h"
#include "DrawAndRevolveTool.h"
#include "MeshPartitionHeightSculptTool.h"

#define LOCTEXT_NAMESPACE "MeshTerrainModeCommands"

namespace UE::MeshTerrain
{


FMeshTerrainModeActionCommands::FMeshTerrainModeActionCommands() :
	TCommands<FMeshTerrainModeActionCommands>(
		"MeshTerrainModeCommands", // Context name for fast lookup
		NSLOCTEXT("Contexts", "MeshTerrainModeCommands", "Mesh Terrain Mode"), // Localized context name for displaying
		NAME_None, // Parent
		FAppStyle::GetAppStyleSetName() // Icon Style Set
		)
{
}


void FMeshTerrainModeActionCommands::RegisterCommands()
{
	UI_COMMAND(FocusViewCommand, "Focus View at Cursor", "Focuses the camera at the scene hit location under the cursor", EUserInterfaceActionType::None, FInputChord(EKeys::C));
	UI_COMMAND(ToggleSelectionLockStateCommand, "Toggle Selection Lock State", "Toggles the Locked/Unlocked state of the active Selection Target", EUserInterfaceActionType::None, FInputChord(EKeys::U));
}


void FMeshTerrainModeActionCommands::RegisterCommandBindings(TSharedPtr<FUICommandList> UICommandList, TFunction<void(EMeshTerrainModeActionCommands)> OnCommandExecuted)
{
	const FMeshTerrainModeActionCommands& Commands = FMeshTerrainModeActionCommands::Get();

	UICommandList->MapAction(
		Commands.FocusViewCommand,
		FExecuteAction::CreateLambda([OnCommandExecuted]() { OnCommandExecuted(EMeshTerrainModeActionCommands::FocusViewToCursor); }));

	UICommandList->MapAction(
		Commands.ToggleSelectionLockStateCommand,
		FExecuteAction::CreateLambda([OnCommandExecuted]() { OnCommandExecuted(EMeshTerrainModeActionCommands::ToggleSelectionLockState); }));
}

void FMeshTerrainModeActionCommands::UnRegisterCommandBindings(TSharedPtr<FUICommandList> UICommandList)
{
	const FMeshTerrainModeActionCommands& Commands = FMeshTerrainModeActionCommands::Get();
	UICommandList->UnmapAction(Commands.FocusViewCommand);
	UICommandList->UnmapAction(Commands.ToggleSelectionLockStateCommand);
}



FMeshTerrainModeToolActionCommands::FMeshTerrainModeToolActionCommands() : 
	TInteractiveToolCommands<FMeshTerrainModeToolActionCommands>(
		"MeshTerrainModeToolCommands", // Context name for fast lookup
		NSLOCTEXT("Contexts", "MeshTerrainModeToolCommands", "Mesh Terrain - Shared Shortcuts"), // Localized context name for displaying
		NAME_None, // Parent
		FAppStyle::GetAppStyleSetName() // Icon Style Set
	)
{
}


void FMeshTerrainModeToolActionCommands::GetToolDefaultObjectList(TArray<UInteractiveTool*>& ToolCDOs)
{
	ToolCDOs.Add(GetMutableDefault<UMeshInspectorTool>());
	//ToolCDOs.Add(GetMutableDefault<UDrawPolygonTool>());
	//ToolCDOs.Add(GetMutableDefault<UEditMeshPolygonsTool>());
	ToolCDOs.Add(GetMutableDefault<UMeshSpaceDeformerTool>());
	ToolCDOs.Add(GetMutableDefault<UShapeSprayTool>());
	//ToolCDOs.Add(GetMutableDefault<UPlaneCutTool>());
}



void FMeshTerrainModeToolActionCommands::RegisterAllToolActions()
{
	FMeshTerrainModeToolActionCommands::Register();
	FSculptToolActionCommands::Register();
	FVertexSculptToolActionCommands::Register();
	FMeshGroupPaintToolActionCommands::Register();
	FMeshVertexPaintToolActionCommands::Register();
	FMeshAttributePaintToolActionCommands::Register();
	FDrawPolygonToolActionCommands::Register();
	FTransformToolActionCommands::Register();
	FMeshSelectionToolActionCommands::Register();
	FEditMeshMaterialsToolActionCommands::Register();
	FMeshPlaneCutToolActionCommands::Register();
	FCubeGridToolActionCommands::Register();
	FEditMeshPolygonsToolActionCommands::Register();
	FDrawAndRevolveToolActionCommands::Register();
	FMegaMeshHeightSculptToolCommands::Register();
}

void FMeshTerrainModeToolActionCommands::UnregisterAllToolActions()
{
	FMeshTerrainModeToolActionCommands::Unregister();
	FSculptToolActionCommands::Unregister();
	FVertexSculptToolActionCommands::Unregister();
	FMeshGroupPaintToolActionCommands::Unregister();
	FMeshVertexPaintToolActionCommands::Unregister();
	FMeshAttributePaintToolActionCommands::Unregister();
	FDrawPolygonToolActionCommands::Unregister();
	FTransformToolActionCommands::Unregister();
	FMeshSelectionToolActionCommands::Unregister();
	FEditMeshMaterialsToolActionCommands::Unregister();
	FMeshPlaneCutToolActionCommands::Unregister();
	FCubeGridToolActionCommands::Unregister();
	FEditMeshPolygonsToolActionCommands::Unregister();
	FDrawAndRevolveToolActionCommands::Unregister();
	FMegaMeshHeightSculptToolCommands::Unregister();
}



void FMeshTerrainModeToolActionCommands::UpdateToolCommandBinding(UInteractiveTool* Tool, TSharedPtr<FUICommandList> UICommandList, bool bUnbind)
{
#define UPDATE_BINDING(CommandsType)  if (!bUnbind) CommandsType::Get().BindCommandsForCurrentTool(UICommandList, Tool); else CommandsType::Get().UnbindActiveCommands(UICommandList);


	if (ExactCast<UTransformMeshesTool>(Tool) != nullptr)
	{
		UPDATE_BINDING(FTransformToolActionCommands);
	}
	else if (ExactCast<UDynamicMeshSculptTool>(Tool) != nullptr)
	{
		UPDATE_BINDING(FSculptToolActionCommands);
	}
	else if (ExactCast<UMeshVertexSculptTool>(Tool) != nullptr)
	{
		UPDATE_BINDING(FVertexSculptToolActionCommands);
	}
	else if (ExactCast<UMeshGroupPaintTool>(Tool) != nullptr)
	{
		UPDATE_BINDING(FMeshGroupPaintToolActionCommands);
	}
	else if (ExactCast<UMeshVertexPaintTool>(Tool) != nullptr)
	{
		UPDATE_BINDING(FMeshVertexPaintToolActionCommands);
	}
	else if (ExactCast<UE::MeshPartition::UAttributePaintTool>(Tool) != nullptr)
	{
		UPDATE_BINDING(FMeshAttributePaintToolActionCommands);
	}
	else if (ExactCast<UDrawPolygonTool>(Tool) != nullptr)
	{
		UPDATE_BINDING(FDrawPolygonToolActionCommands);
	}
	else if (ExactCast<UMeshSelectionTool>(Tool) != nullptr)
	{
		UPDATE_BINDING(FMeshSelectionToolActionCommands);
	}
	else if (ExactCast<UEditMeshMaterialsTool>(Tool) != nullptr)
	{
		UPDATE_BINDING(FEditMeshMaterialsToolActionCommands);
	}
	else if (ExactCast<UPlaneCutTool>(Tool) != nullptr)
	{
		UPDATE_BINDING(FMeshPlaneCutToolActionCommands);
	}
	else if (ExactCast<UCubeGridTool>(Tool) != nullptr)
	{
		UPDATE_BINDING(FCubeGridToolActionCommands);
	}
	else if (ExactCast<UEditMeshPolygonsTool>(Tool) != nullptr)
	{
		UPDATE_BINDING(FEditMeshPolygonsToolActionCommands);
	}
	else if (ExactCast<UDrawAndRevolveTool>(Tool) != nullptr)
	{
		UPDATE_BINDING(FDrawAndRevolveToolActionCommands);
	}
	else if (ExactCast<UE::MeshPartition::UHeightSculptTool>(Tool) != nullptr)
	{
		UPDATE_BINDING(FMegaMeshHeightSculptToolCommands);
	}
	else
	{
		UPDATE_BINDING(FMeshTerrainModeToolActionCommands);
	}
}





#define DEFINE_TOOL_ACTION_COMMANDS(CommandsClassName, ContextNameString, SettingsDialogString, ToolClassName ) \
CommandsClassName::CommandsClassName() : TInteractiveToolCommands<CommandsClassName>( \
ContextNameString, NSLOCTEXT("Contexts", ContextNameString, SettingsDialogString), NAME_None, FAppStyle::GetAppStyleSetName()) {} \
void CommandsClassName::GetToolDefaultObjectList(TArray<UInteractiveTool*>& ToolCDOs) \
{\
	ToolCDOs.Add(GetMutableDefault<ToolClassName>()); \
}




DEFINE_TOOL_ACTION_COMMANDS(FSculptToolActionCommands, "ModelingToolsProtoSculptTool", "Modeling Tools - Sculpt Tool", UDynamicMeshSculptTool);
DEFINE_TOOL_ACTION_COMMANDS(FVertexSculptToolActionCommands, "ModelingToolsProtoVertexSculptTool", "Modeling Tools - Vertex Sculpt Tool", UMeshVertexSculptTool);
DEFINE_TOOL_ACTION_COMMANDS(FMeshGroupPaintToolActionCommands, "ModelingToolsProtoMeshGroupPaintTool", "Modeling Tools - Group Paint Tool", UMeshGroupPaintTool);
DEFINE_TOOL_ACTION_COMMANDS(FMeshVertexPaintToolActionCommands, "ModelingToolsProtoMeshVertexPaintTool", "Modeling Tools - Vertex Paint Tool", UMeshVertexPaintTool);
DEFINE_TOOL_ACTION_COMMANDS(FMeshAttributePaintToolActionCommands, "ModelingToolsProtoMeshAttributePaintTool", "Modeling Tools - Attribute Paint Tool", UE::MeshPartition::UAttributePaintTool);
DEFINE_TOOL_ACTION_COMMANDS(FTransformToolActionCommands, "ModelingToolsProtoTransformTool", "Modeling Tools - Transform Tool", UTransformMeshesTool);
DEFINE_TOOL_ACTION_COMMANDS(FDrawPolygonToolActionCommands, "ModelingToolsProtoDrawPolygonTool", "Modeling Tools - Draw Polygon Tool", UDrawPolygonTool);
DEFINE_TOOL_ACTION_COMMANDS(FMeshSelectionToolActionCommands, "ModelingToolsProtoMeshSelectionTool", "Modeling Tools - Mesh Selection Tool", UMeshSelectionTool);
DEFINE_TOOL_ACTION_COMMANDS(FEditMeshMaterialsToolActionCommands, "ModelingToolsProtoEditMeshMaterials", "Modeling Tools - Edit Materials Tool", UEditMeshMaterialsTool);
DEFINE_TOOL_ACTION_COMMANDS(FMeshPlaneCutToolActionCommands, "ModelingToolsProtoMeshPlaneCutTool", "Modeling Tools - Mesh Plane Cut Tool", UPlaneCutTool);
DEFINE_TOOL_ACTION_COMMANDS(FCubeGridToolActionCommands, "ModelingToolsProtoCubeGridTool", "Modeling Tools - Cube Grid Tool", UCubeGridTool);
DEFINE_TOOL_ACTION_COMMANDS(FEditMeshPolygonsToolActionCommands, "ModelingToolsProtoEditMeshPolygonsTool", "Modeling Tools - Edit Mesh Polygons Tool", UEditMeshPolygonsTool);
DEFINE_TOOL_ACTION_COMMANDS(FDrawAndRevolveToolActionCommands, "ModelingToolsProtoDrawAndRevolveTool", "Modeling Tools - Draw-and-Revolve Tool", UDrawAndRevolveTool);
DEFINE_TOOL_ACTION_COMMANDS(FMegaMeshHeightSculptToolCommands, "HeightSculpt", "MegaMesh - Height Sculpt", UE::MeshPartition::UHeightSculptTool);

}



#undef LOCTEXT_NAMESPACE

