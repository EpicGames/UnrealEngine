// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Tools/InteractiveToolsCommands.h"


namespace UE::MeshTerrain
{

enum class EMeshTerrainModeActionCommands
{
	FocusViewToCursor,
	ToggleSelectionLockState
};


class FMeshTerrainModeActionCommands : public TCommands<FMeshTerrainModeActionCommands>
{
public:
	FMeshTerrainModeActionCommands();

	virtual void RegisterCommands() override;

	TSharedPtr<FUICommandInfo> FocusViewCommand;
	TSharedPtr<FUICommandInfo> ToggleSelectionLockStateCommand;

	static void RegisterCommandBindings(TSharedPtr<FUICommandList> UICommandList, TFunction<void(EMeshTerrainModeActionCommands)> OnCommandExecuted);
	static void UnRegisterCommandBindings(TSharedPtr<FUICommandList> UICommandList);
};



/**
 * TInteractiveToolCommands implementation for this module that provides standard Editor hotkey support
 */
class FMeshTerrainModeToolActionCommands : public TInteractiveToolCommands<FMeshTerrainModeToolActionCommands>
{
public:
	FMeshTerrainModeToolActionCommands();

	virtual void GetToolDefaultObjectList(TArray<UInteractiveTool*>& ToolCDOs) override;

	/**
	 * interface that hides various per-tool action sets
	 */

	/**
	 * Register all Tool command sets. Call this in module startup
	 */
	static void RegisterAllToolActions();

	/**
	 * Unregister all Tool command sets. Call this from module shutdown.
	 */
	static void UnregisterAllToolActions();

	/**
	 * Add or remove commands relevant to Tool to the given UICommandList.
	 * Call this when the active tool changes (eg on ToolManager.OnToolStarted / OnToolEnded)
	 * @param bUnbind if true, commands are removed, otherwise added
	 */
	static void UpdateToolCommandBinding(UInteractiveTool* Tool, TSharedPtr<FUICommandList> UICommandList, bool bUnbind = false);

};


// Each TCommands can only have a single (chord,action) binding, regardless of whether these
// would ever be used at the same time. And we cannot create/register TCommands at runtime.
// So, we have to define a separate TCommands instance for each Tool. This is unfortunate.

#define DECLARE_TOOL_ACTION_COMMANDS(CommandsClassName) \
class CommandsClassName : public TInteractiveToolCommands<CommandsClassName> \
{\
public:\
	CommandsClassName();\
	virtual void GetToolDefaultObjectList(TArray<UInteractiveTool*>& ToolCDOs) override;\
};\


DECLARE_TOOL_ACTION_COMMANDS(FSculptToolActionCommands);
DECLARE_TOOL_ACTION_COMMANDS(FVertexSculptToolActionCommands);
DECLARE_TOOL_ACTION_COMMANDS(FMeshGroupPaintToolActionCommands);
DECLARE_TOOL_ACTION_COMMANDS(FMeshVertexPaintToolActionCommands);
DECLARE_TOOL_ACTION_COMMANDS(FMeshAttributePaintToolActionCommands);
DECLARE_TOOL_ACTION_COMMANDS(FTransformToolActionCommands);
DECLARE_TOOL_ACTION_COMMANDS(FDrawPolygonToolActionCommands);
DECLARE_TOOL_ACTION_COMMANDS(FMeshSelectionToolActionCommands);
DECLARE_TOOL_ACTION_COMMANDS(FEditMeshMaterialsToolActionCommands);
DECLARE_TOOL_ACTION_COMMANDS(FMeshPlaneCutToolActionCommands);
DECLARE_TOOL_ACTION_COMMANDS(FCubeGridToolActionCommands);
DECLARE_TOOL_ACTION_COMMANDS(FEditMeshPolygonsToolActionCommands);
DECLARE_TOOL_ACTION_COMMANDS(FDrawAndRevolveToolActionCommands);
DECLARE_TOOL_ACTION_COMMANDS(FMegaMeshHeightSculptToolCommands);
	
}