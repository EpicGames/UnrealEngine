// Copyright Epic Games, Inc. All Rights Reserved.

#include "MeshTerrainModeAssetUtils.h"
#include "MeshTerrainModeSettings.h"
#include "ContentBrowserItemPath.h"

#include "AssetRegistry/AssetRegistryModule.h"
#include "AssetToolsModule.h"
#include "FileHelpers.h"
#include "IAssetTools.h"
#include "Misc/Paths.h"
#include "Misc/PackageName.h"
#include "ModelingObjectsCreationAPI.h"
#include "Editor/EditorEngine.h"
#include "EditorDirectories.h"

// for content-browser things
#include "ContentBrowserModule.h"
#include "IContentBrowserSingleton.h"
#include "Trace/Trace.inl"

extern UNREALED_API UEditorEngine* GEditor;

#define LOCTEXT_NAMESPACE "MeshTerrainModeAssetUtils"

namespace UE::MeshTerrain
{
namespace Local
{

FString GetActiveAssetFolderPath()
{
	IContentBrowserSingleton& ContentBrowser = FModuleManager::LoadModuleChecked<FContentBrowserModule>("ContentBrowser").Get();
	const FContentBrowserItemPath CurrentPath = ContentBrowser.GetCurrentPath();
	return CurrentPath.HasInternalPath() ? CurrentPath.GetInternalPathString() : FString();
}


FString MakeUniqueAssetName(const FString& FolderPath, const FString& AssetBaseName)
{
	FString UniquePackageName, UniqueAssetName;
	FAssetToolsModule& AssetToolsModule = FModuleManager::LoadModuleChecked<FAssetToolsModule>("AssetTools");
	AssetToolsModule.Get().CreateUniqueAssetName(
		FolderPath + TEXT("/") + AssetBaseName, TEXT(""), UniquePackageName, UniqueAssetName);
	return UniqueAssetName;
}


FString InteractiveSelectAssetPath(const FString& DefaultAssetName, const FText& DialogTitleMessage)
{
	IContentBrowserSingleton& ContentBrowser = FModuleManager::LoadModuleChecked<FContentBrowserModule>("ContentBrowser").Get();

	FString UseDefaultAssetName = DefaultAssetName;
	FString CurrentPath = GetActiveAssetFolderPath();
	if (CurrentPath.IsEmpty() == false)
	{
		UseDefaultAssetName = MakeUniqueAssetName(CurrentPath, DefaultAssetName);
	}

	FSaveAssetDialogConfig Config;
	Config.DefaultAssetName = UseDefaultAssetName;
	Config.DialogTitleOverride = DialogTitleMessage;
	Config.DefaultPath = CurrentPath;
	return ContentBrowser.CreateModalSaveAssetDialog(Config);
}


// Replaces each non-alphanumeric character with an underscore.
void SanitizeObjectNameToAlphaNumeric(FString& ObjectName)
{
	for (TCHAR& Character : ObjectName)
	{
		if (!FChar::IsAlnum(Character))
		{
			Character = TEXT('_');
		}
	}
}


}
}


FString UE::MeshTerrain::GetWorldRelativeAssetRootPath(const UWorld* World)
{
	if (World == nullptr)
	{
		World = GEditor->bIsSimulatingInEditor ? GEditor->GetPIEWorldContext()->World() : GEditor->GetEditorWorldContext().World();
	}

	if (ensure(World->GetPackage() != nullptr) == false)
	{
		return GetGlobalAssetRootPath();
	}

	const FString WorldPackageName = World->GetPackage()->GetName();
	const FString WorldPackageFolder = FPackageName::GetLongPackagePath(WorldPackageName);
	return WorldPackageFolder;
}

FString UE::MeshTerrain::GetGlobalAssetRootPath()
{
	// There currently is no mechanism to explicitly get the root content folder.
	// The effectively best way to do this is to request the "last directory" for an invalid identifier, which falls back to the root content folder. 

	const FString RootFilePath = FEditorDirectories::Get().GetLastDirectory(ELastDirectory::MAX);
	if (FString RootRelativePath, FailureReason; FPackageName::TryConvertFilenameToLongPackageName(RootFilePath, RootRelativePath, &FailureReason))
	{
		return RootRelativePath;
	}
	else
	{
		UE_LOGF(LogTemp, Error, "Could not create global asset root path. Faiure Reason: %ls", *FailureReason);
		return {};
	}
}

FString UE::MeshTerrain::GetNewAssetPathName(const FString& BaseNameIn, const UWorld* TargetWorld, FString SuggestedFolder)
{
	const UMeshTerrainModeSettings* Settings = GetDefault<UMeshTerrainModeSettings>();
	const EMeshTerrainModeAssetGenerationBehavior AutoGenMode = Settings->GetAssetGenerationMode();

	FString PackageFolderPath = GetGlobalAssetRootPath();
	bool bFallBackToInteractiveSave = false;

	if (SuggestedFolder.Len() > 0 && SuggestedFolder.StartsWith("/") )
	{
		// if we had a suggested folder, use it (todo: check that this folder exists? or can be created?)
		PackageFolderPath = SuggestedFolder;
	}
	else
	{
		// Figure out the appropriate package path to use. If the level is in a plugin, this should be the plugin top-level path.
		if (Settings->GetAssetGenerationLocation() == EMeshTerrainModeAssetGenerationLocation::AutoGeneratedWorldRelativeAssetPath)
		{
			PackageFolderPath = GetWorldRelativeAssetRootPath(TargetWorld);
		}

		// If the level is unsaved, then GetWorldRelativeAssetRootPath will return "/Temp". This is an undesirable
		// place to even temporarily save assets because they must be manually moved elsewhere to save, and the /Temp folder 
		// is somewhat troublesome to get to in the Editor.
		if (PackageFolderPath.StartsWith("/Temp"))
		{
			if (Settings->InRestrictiveMode())
			{
				// Fall back to the interactively choosing the asset path.
				bFallBackToInteractiveSave = true;
			}
			else
			{
				// If the flag below is set, then we will fall back to the global path instead.
				// Projects with strict asset polices (due to P4/etc) may not want to allow this by default, so it is configurable.
				if (Settings->bStoreUnsavedLevelAssetsInTopLevelGameFolder)
				{
					PackageFolderPath = GetGlobalAssetRootPath();
				}
			}
		}

		if (Settings->InRestrictiveMode())
		{
			// combine with fixed AutoGen path name if it is not empty
			const FString RestrictiveModeAutogenAssetPath = Settings->GetRestrictiveModeAutoGeneratedAssetPath(); 
			if (RestrictiveModeAutogenAssetPath.Len() > 0)
			{
				PackageFolderPath = FPaths::Combine(PackageFolderPath, RestrictiveModeAutogenAssetPath);
			}
		}
		else
		{
			// combine with fixed AutoGen path name if it is not empty
			if (Settings->AutoGeneratedAssetPath.Len() > 0)
			{
				PackageFolderPath = FPaths::Combine(PackageFolderPath, Settings->AutoGeneratedAssetPath);
			}

			// append username-specific subfolder
			if (Settings->bUsePerUserAutogenSubfolder)
			{
				FString UsernameString = Settings->AutogenSubfolderUserNameOverride.TrimStartAndEnd();
				if (UsernameString.IsEmpty())
				{
					UsernameString = FPlatformProcess::UserName();
				}
				if (!UsernameString.IsEmpty())
				{
					PackageFolderPath = FPaths::Combine(PackageFolderPath, UsernameString);
				}
			}
		}

		// if we want to use the currently-visible asset browser path, try to find one (this can fail if no asset browser is visible/etc)
		if (Settings->GetAssetGenerationLocation() == EMeshTerrainModeAssetGenerationLocation::CurrentAssetBrowserPathIfAvailable)
		{
			FString CurrentAssetPath = Local::GetActiveAssetFolderPath();
			if (CurrentAssetPath.IsEmpty() == false)
			{
				PackageFolderPath = CurrentAssetPath;
			}
		}
	}

	FString ObjectBaseName = BaseNameIn;

	// If we are in interactive mode, show the modal dialog and then get the path/name.
	// If the user cancels, we are going to discard the asset
	if (AutoGenMode == EMeshTerrainModeAssetGenerationBehavior::InteractivePromptToSave || bFallBackToInteractiveSave)
	{
		const FString SelectedPath = Local::InteractiveSelectAssetPath(
			ObjectBaseName, LOCTEXT("GenerateStaticMeshActorPathDialogWarning", "Choose Folder Path and Name for New Asset. Cancel to Discard New Asset."));
		if (SelectedPath.IsEmpty() == false)
		{
			PackageFolderPath = FPaths::GetPath(SelectedPath);
			ObjectBaseName = FPaths::GetBaseFilename(SelectedPath, true);
		}
		else
		{
			return FString();
		}
	}

	FString UseBaseName = ObjectBaseName;

	if (Settings->InRestrictiveMode())
	{
		Local::SanitizeObjectNameToAlphaNumeric(UseBaseName);
	}

	if (Settings->bAppendRandomStringToName)
	{
		FString GuidString = UE::Modeling::GenerateRandomShortHexString();
		UseBaseName = FString::Printf(TEXT("%s_%s"), *UseBaseName, *GuidString);
	}


	FString PackageNameOut, AssetNameOut;
	FAssetToolsModule& AssetToolsModule = FModuleManager::LoadModuleChecked<FAssetToolsModule>("AssetTools");
	AssetToolsModule.Get().CreateUniqueAssetName(
		FPaths::Combine(PackageFolderPath, UseBaseName), TEXT(""),
		PackageNameOut, AssetNameOut);

	return PackageNameOut;
}



bool UE::MeshTerrain::AutoSaveAsset(UObject* Asset)
{
	UPackage* AssetPackage = Asset->GetPackage();
	if (ensure(AssetPackage) == false)
	{
		return false;
	}

	Asset->MarkPackageDirty();
	FAssetRegistryModule::AssetCreated(Asset);

	TArray<UPackage*> PackagesToSave;
	PackagesToSave.Add(AssetPackage);

	bool bCheckDirty = true;
	bool bPromptToSave = false;		// because we are autosaving
	FEditorFileUtils::EPromptReturnCode Result = FEditorFileUtils::PromptForCheckoutAndSave(PackagesToSave, bCheckDirty, bPromptToSave);
	ensure(Result == FEditorFileUtils::PR_Success);
	return (Result == FEditorFileUtils::PR_Success);
}





void UE::MeshTerrain::OnNewAssetCreated(UObject* Asset)
{
	const UMeshTerrainModeSettings* Settings = GetDefault<UMeshTerrainModeSettings>();
	EMeshTerrainModeAssetGenerationBehavior AutoGenMode = Settings->GetAssetGenerationMode();

	// Auto-save the new asset if the user has requested this behavior, or if an interactive
	// save/path dialog was shown earlier in the process (InteractivePromptToSave mode)
	if (AutoGenMode == EMeshTerrainModeAssetGenerationBehavior::AutoGenerateAndAutosave
		 || AutoGenMode == EMeshTerrainModeAssetGenerationBehavior::InteractivePromptToSave)
	{
		AutoSaveAsset(Asset);
	}
	else if (AutoGenMode == EMeshTerrainModeAssetGenerationBehavior::AutoGenerateButDoNotAutosave)
	{
		Asset->MarkPackageDirty();
		FAssetRegistryModule::AssetCreated(Asset);
	}
}


#undef LOCTEXT_NAMESPACE
