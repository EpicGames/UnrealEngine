// Copyright Epic Games, Inc. All Rights Reserved.

#include "MeshTerrainModeEditablePaletteConfig.h"

TObjectPtr<UMeshTerrainModeEditableToolPaletteConfig> UMeshTerrainModeEditableToolPaletteConfig::Instance = nullptr;

void UMeshTerrainModeEditableToolPaletteConfig::Initialize()
{
	if(!Instance)
	{
		Instance = NewObject<UMeshTerrainModeEditableToolPaletteConfig>(); 
		Instance->AddToRoot();
	}
}

FEditableToolPaletteSettings* UMeshTerrainModeEditableToolPaletteConfig::GetMutablePaletteConfig(const FName& InstanceName)
{
	if (InstanceName.IsNone())
	{
		return nullptr;
	}

	return &EditableToolPalettes.FindOrAdd(InstanceName);
}

const FEditableToolPaletteSettings* UMeshTerrainModeEditableToolPaletteConfig::GetConstPaletteConfig(const FName& InstanceName)
{
	if (InstanceName.IsNone())
	{
		return nullptr;
	}

	return EditableToolPalettes.Find(InstanceName);
}

void UMeshTerrainModeEditableToolPaletteConfig::SavePaletteConfig(const FName&)
{
	SaveEditorConfig();
}