// Copyright Epic Games, Inc. All Rights Reserved.

#include "MeshTerrainModeHostCustomizationAPI.h"

#include "ContextObjectStore.h"
#include "InteractiveToolsContext.h"
#include "MeshTerrainModeToolkit.h"

namespace UE::MeshTerrain
{

bool UMeshTerrainModeHostCustomizationAPI::RequestAcceptCancelButtonOverride(FAcceptCancelButtonOverrideParams& Params)
{
	if (TSharedPtr<FMeshTerrainModeToolkit> ToolkitPinned = Toolkit.Pin())
	{
		return ToolkitPinned->RequestAcceptCancelButtonOverride(Params);
	}
	return false;
}

bool UMeshTerrainModeHostCustomizationAPI::RequestCompleteButtonOverride(FCompleteButtonOverrideParams& Params)
{
	if (TSharedPtr<FMeshTerrainModeToolkit> ToolkitPinned = Toolkit.Pin())
	{
		return ToolkitPinned->RequestCompleteButtonOverride(Params);
	}
	return false;
}

void UMeshTerrainModeHostCustomizationAPI::ClearButtonOverrides()
{
	if (TSharedPtr<FMeshTerrainModeToolkit> ToolkitPinned = Toolkit.Pin())
	{
		ToolkitPinned->ClearButtonOverrides();
	}
}

UMeshTerrainModeHostCustomizationAPI* UMeshTerrainModeHostCustomizationAPI::Register(
	UInteractiveToolsContext* ToolsContext, TSharedRef<FMeshTerrainModeToolkit> Toolkit)
{
	if (!ensure(ToolsContext && ToolsContext->ContextObjectStore))
	{
		return nullptr;
	}

	UMeshTerrainModeHostCustomizationAPI* Found = ToolsContext->ContextObjectStore->FindContext<UMeshTerrainModeHostCustomizationAPI>();
	if (Found)
	{
		if (!ensureMsgf(Found->Toolkit == Toolkit,
			TEXT("UMeshTerrainModeHostCustomizationAPI already registered, but with different toolkit. "
				"Do not expect multiple toolkits to provide tool overlays to customize.")))
		{
			Found->Toolkit = Toolkit;
		}
		return Found;
	}
	else
	{
		UMeshTerrainModeHostCustomizationAPI* Instance = NewObject<UMeshTerrainModeHostCustomizationAPI>();
		Instance->Toolkit = Toolkit;
		ensure(ToolsContext->ContextObjectStore->AddContextObject(Instance));
		return Instance;
	}
}

bool UMeshTerrainModeHostCustomizationAPI::Deregister(UInteractiveToolsContext* ToolsContext)
{
	if (!ensure(ToolsContext && ToolsContext->ContextObjectStore))
	{
		return false;
	}

	ToolsContext->ContextObjectStore->RemoveContextObjectsOfType(UMeshTerrainModeHostCustomizationAPI::StaticClass());
	return true;
}
	
}
