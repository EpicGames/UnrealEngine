// Copyright Epic Games, Inc. All Rights Reserved.

#include "MeshTerrainModeManagerActions.h"
#include "MeshTerrainModeStyle.h"
#include "MeshTerrainModeSettings.h"
#include "Styling/ISlateStyle.h"

#define LOCTEXT_NAMESPACE "MeshTerrainModeManagerCommands"



FMeshTerrainModeManagerCommands::FMeshTerrainModeManagerCommands() :
	TCommands<FMeshTerrainModeManagerCommands>(
		"MeshTerrainModeManagerCommands", // Context name for fast lookup
		NSLOCTEXT("Contexts", "MeshTerrainModeManagerCommands", "Modeling Mode - Tools"), // Localized context name for displaying
		NAME_None, // Parent
		FMeshTerrainModeStyle::Get()->GetStyleSetName() // Icon Style Set
		)
{
}



TSharedPtr<FUICommandInfo> FMeshTerrainModeManagerCommands::FindToolByName(FString Name, bool& bFound) const
{
	bFound = false;
	for (const FStartToolCommand& Command : RegisteredTools)
	{
		if (Command.ToolUIName.Equals(Name, ESearchCase::IgnoreCase)
		 || (Command.ToolCommand.IsValid() && Command.ToolCommand->GetLabel().ToString().Equals(Name, ESearchCase::IgnoreCase)))
		{
			bFound = true;
			return Command.ToolCommand;
		}
	}
	return TSharedPtr<FUICommandInfo>();
}

FText FMeshTerrainModeManagerCommands::GetCommandLabel(TSharedPtr<const FUICommandInfo> Command, bool bShortName) const
{
	if (!ensure(Command.IsValid()))
	{
		return FText();
	}
	if (const int32* Index = CommandToRegisteredToolsIndex.Find(Command.Get()))
	{
		if (RegisteredTools.IsValidIndex(*Index) && !RegisteredTools[*Index].ShortName.IsEmpty())
		{
			return RegisteredTools[*Index].ShortName;
		}
	}
	return Command->GetLabel();
}


void FMeshTerrainModeManagerCommands::RegisterCommands()
{
	const UMeshTerrainModeSettings* Settings = GetDefault<UMeshTerrainModeSettings>();

	// Some points about registering commands:
	// 1. UI_COMMAND expands to a LOCTEXT macro, so the helpers need to be compile-time macros.
	// 2. The LOCTEXT expansion uses ToolCommandInfo as the key, so trying to conditionally register different
	//  tool names (e.g. short names) for the same ToolCommandInfo will lead to LOCTEXT key collisions unless
	//  you define a different LOCTEXT_NAMESPACE in the different branches. We could have done that for handling
	//  short names, but decided instead to store short names ourselves.
	// 3. LOCTEXT processor is a simple pattern matcher (though with special handling for UI_COMMAND) and
	//  does not run after preprocessing, so trying to assemble a LOCTEXT in the macro, such as 
	//  "LOCTEXT(ToolName"_short", ShortName)" will not work. Hence us passing a ready LOCTEXT
	//  to our macros for the short name.

#define REGISTER_MODELING_TOOL_COMMAND(ToolCommandInfo, ToolName, ToolTip) \
		UI_COMMAND(ToolCommandInfo, ToolName, ToolTip, EUserInterfaceActionType::ToggleButton, FInputChord()); \
		RegisteredTools.Add(FStartToolCommand{ ToolName, ToolCommandInfo });
#define REGISTER_MODELING_TOOL_COMMAND_WITH_SHORTNAME(ToolCommandInfo, ToolName, ShortNameFText, ToolTip) \
		UI_COMMAND(ToolCommandInfo, ToolName, ToolTip, EUserInterfaceActionType::ToggleButton, FInputChord()); \
		RegisteredTools.Add(FStartToolCommand{ ToolName, ToolCommandInfo, ShortNameFText });
#define REGISTER_MODELING_TOOL_COMMAND_RADIO(ToolCommandInfo, ToolName, ToolTip) \
		UI_COMMAND(ToolCommandInfo, ToolName, ToolTip, EUserInterfaceActionType::RadioButton, FInputChord()); \
		RegisteredTools.Add(FStartToolCommand{ ToolName, ToolCommandInfo });
#define REGISTER_MODELING_ACTION_COMMAND(ToolCommandInfo, ToolName, ToolTip) \
		UI_COMMAND(ToolCommandInfo, ToolName, ToolTip, EUserInterfaceActionType::Button, FInputChord()); \
		RegisteredTools.Add(FStartToolCommand{ ToolName, ToolCommandInfo });
#define REGISTER_MODELING_ACTION_COMMAND_WITH_SHORTNAME(ToolCommandInfo, ToolName, ShortNameFText, ToolTip) \
		UI_COMMAND(ToolCommandInfo, ToolName, ToolTip, EUserInterfaceActionType::Button, FInputChord()); \
		RegisteredTools.Add(FStartToolCommand{ ToolName, ToolCommandInfo, ShortNameFText });

	// Shapes
	REGISTER_MODELING_TOOL_COMMAND(BeginAddBoxPrimitiveTool, "Box", "Create new box objects");
	REGISTER_MODELING_TOOL_COMMAND(BeginAddSpherePrimitiveTool, "Sphere", "Create new sphere objects");
	REGISTER_MODELING_TOOL_COMMAND(BeginAddConePrimitiveTool, "Cone", "Create new cone objects");
	REGISTER_MODELING_TOOL_COMMAND(BeginAddTorusPrimitiveTool, "Torus", "Create new torus objects");
	REGISTER_MODELING_TOOL_COMMAND(BeginAddArrowPrimitiveTool, "Arrow", "Create new arrow objects");
	REGISTER_MODELING_TOOL_COMMAND(BeginAddDiscPrimitiveTool, "Disc", "Create new disc objects");
	REGISTER_MODELING_TOOL_COMMAND(BeginAddStairsPrimitiveTool, "Stairs", "Create new stairs objects");
	REGISTER_MODELING_TOOL_COMMAND(BeginAddCapsulePrimitiveTool, "Capsule", "Create new capsule objects");
	REGISTER_MODELING_TOOL_COMMAND_WITH_SHORTNAME(BeginAddCylinderPrimitiveTool, "Cylinder", LOCTEXT("BeginAddCylinderPrimitiveTool_short", "Cyl"), "Create new cylinder objects");
	REGISTER_MODELING_TOOL_COMMAND_WITH_SHORTNAME(BeginAddRectanglePrimitiveTool, "Rectangle", LOCTEXT("BeginAddRectanglePrimitiveTool_short", "Rect"), "Create new rectangle objects");


	// Create
	REGISTER_MODELING_TOOL_COMMAND_WITH_SHORTNAME(BeginDrawPolygonTool, "Extrude Polygon", LOCTEXT("BeginDrawPolygonTool_short", "PolyExt"), "Draw and extrude 2D Polygons to create new objects");
	REGISTER_MODELING_TOOL_COMMAND_WITH_SHORTNAME(BeginDrawPolyPathTool, "Extrude Path", LOCTEXT("BeginDrawPolyPathTool_short", "PathExt"), "Draw and extrude 2D Paths to create new objects");
	REGISTER_MODELING_TOOL_COMMAND_WITH_SHORTNAME(BeginDrawAndRevolveTool, "Revolve Path", LOCTEXT("BeginDrawAndRevolveTool_short", "PathRev"), "Draw and revolve 2D Paths to create new objects");
	REGISTER_MODELING_TOOL_COMMAND_WITH_SHORTNAME(BeginRevolveSplineTool, "Revolve Spline", LOCTEXT("BeginRevolveSplineTool_short", "SplnRev"), "Revolve splines to create new objects");
	REGISTER_MODELING_TOOL_COMMAND_WITH_SHORTNAME(BeginRevolveBoundaryTool, "Revolve Boundary", LOCTEXT("BeginRevolveBoundaryTool_short", "BdryRev"), "Revolve Mesh boundary loops to create new objects");
	REGISTER_MODELING_TOOL_COMMAND(BeginCombineMeshesTool, "Merge", "Merge multiple Meshes to create new objects");
	REGISTER_MODELING_TOOL_COMMAND_WITH_SHORTNAME(BeginDuplicateMeshesTool, "Duplicate", LOCTEXT("BeginDuplicateMeshesTool_short", "Dupe"), "Duplicate single Meshes to create new objects");
	REGISTER_MODELING_TOOL_COMMAND_WITH_SHORTNAME(BeginDrawSplineTool, "Draw Spline", LOCTEXT("BeginDrawSplineTool_short", "DrwSpln"), "Draw a spline in the level");
	REGISTER_MODELING_TOOL_COMMAND_WITH_SHORTNAME(BeginTriangulateSplinesTool, "Mesh Splines", LOCTEXT("BeginTriangulateSplinesTool_short", "MshSpln"), "Triangulate the Spline Components of selected actors to create new objects");

	// PolyModel
	REGISTER_MODELING_TOOL_COMMAND_WITH_SHORTNAME(BeginPolyEditTool, "PolyGroup Edit", LOCTEXT("BeginPolyEditTool_short", "PolyEd"), "Edit Meshes via PolyGroups");
	REGISTER_MODELING_TOOL_COMMAND_WITH_SHORTNAME(BeginPolyDeformTool, "Deform PolyGroups", LOCTEXT("BeginPolyDeformTool_short", "PolyDef"), "Deform Meshes via PolyGroups");
	REGISTER_MODELING_TOOL_COMMAND_WITH_SHORTNAME(BeginCubeGridTool, "CubeGrid", LOCTEXT("BeginCubeGridTool_short", "CubeGr"), "Create block out Meshes using a repositionable grid");
	REGISTER_MODELING_TOOL_COMMAND_WITH_SHORTNAME(BeginMeshBooleanTool, "Boolean", LOCTEXT("BeginMeshBooleanTool_short", "MshBool"), "Apply Boolean operations to the two selected Meshes");
	REGISTER_MODELING_TOOL_COMMAND_WITH_SHORTNAME(BeginCutMeshWithMeshTool, "Mesh Cut", LOCTEXT("BeginCutMeshWithMeshTool_short", "MshCut"), "Split one Mesh into parts using a second Mesh");
	REGISTER_MODELING_TOOL_COMMAND_WITH_SHORTNAME(BeginSubdividePolyTool, "Subdivide", LOCTEXT("BeginSubdividePolyTool_short", "SubDiv"), "Subdivide the selected Mesh via PolyGroups or triangles");

	// TriModel
	REGISTER_MODELING_TOOL_COMMAND_WITH_SHORTNAME(BeginMeshSelectionTool, "Tri Select", LOCTEXT("BeginMeshSelectionTool_short", "TriSel"), "Select and edit Mesh triangles with a brush interface");
	REGISTER_MODELING_TOOL_COMMAND_WITH_SHORTNAME(BeginTriEditTool, "Triangle Edit", LOCTEXT("BeginTriEditTool_short", "TriEd"), "Select and Edit the Mesh vertices, edges, and triangles");
	REGISTER_MODELING_TOOL_COMMAND_WITH_SHORTNAME(BeginHoleFillTool, "Fill Holes", LOCTEXT("BeginHoleFillTool_short", "HFill"), "Fill holes in the selected Mesh");
	REGISTER_MODELING_TOOL_COMMAND_WITH_SHORTNAME(BeginPlaneCutTool, "Plane Cut", LOCTEXT("BeginPlaneCutTool_short", "PlnCut"), "Cut the selected Meshes with a 3D plane");
	REGISTER_MODELING_TOOL_COMMAND(BeginMirrorTool, "Mirror", "Mirror the selected Meshes");
	REGISTER_MODELING_TOOL_COMMAND(BeginPolygonCutTool, "PolyCut", "Cut the selected Mesh with an extruded polygon");
	REGISTER_MODELING_TOOL_COMMAND(BeginMeshTrimTool, "Trim", "Trim/Cut the selected Mesh with the second selected Mesh");

	// Deform
	REGISTER_MODELING_TOOL_COMMAND_WITH_SHORTNAME(BeginSculptMeshTool, "Vertex Sculpt", LOCTEXT("BeginSculptMeshTool_short", "VSclpt"), "Vertex sculpting");
	REGISTER_MODELING_TOOL_COMMAND_WITH_SHORTNAME(BeginRemeshSculptMeshTool, "Dynamic Sculpt", LOCTEXT("BeginRemeshSculptMeshTool_short", "DSclpt"), "Dynamic mesh sculpting");
	REGISTER_MODELING_TOOL_COMMAND_WITH_SHORTNAME(BeginDisplaceMeshTool, "Displace", LOCTEXT("BeginDisplaceMeshTool_short", "Displce"), "Tessellate and Displace the selected Mesh");
	REGISTER_MODELING_TOOL_COMMAND(BeginSmoothMeshTool, "Smooth", "Smooth the shape of the selected Mesh");
	REGISTER_MODELING_TOOL_COMMAND(BeginOffsetMeshTool, "Offset", "Offset the surface of the selected Mesh");
	REGISTER_MODELING_TOOL_COMMAND(BeginMeshSpaceDeformerTool, "Warp", "Reshape the selected Mesh using space deformers");
	REGISTER_MODELING_TOOL_COMMAND(BeginLatticeDeformerTool, "Lattice", "Deform the selected Mesh using a 3D lattice/grid");

	// Vertex Sculpt - Brushes
	REGISTER_MODELING_TOOL_COMMAND_WITH_SHORTNAME(BeginSculptMeshMoveBrushTool, "Move", LOCTEXT("BeginSculptMeshMoveBrushTool_short", "VSclptMove"), "Move Brush");
	REGISTER_MODELING_TOOL_COMMAND_WITH_SHORTNAME(BeginSculptMeshPullKelvinBrushTool, "Grab", LOCTEXT("BeginSculptMeshPullKelvinBrushTool_short", "VSclptGrab"), "Kelvin Grab Brush");
	REGISTER_MODELING_TOOL_COMMAND_WITH_SHORTNAME(BeginSculptMeshPullSharpKelvinBrushTool, "GrabSharp", LOCTEXT("BeginSculptMeshPullSharpKelvinBrushTool_short", "VSclptSharpGrab"), "Kelvin Sharp Grab Brush");
	REGISTER_MODELING_TOOL_COMMAND_WITH_SHORTNAME(BeginSculptMeshSmoothBrushTool, "Smooth", LOCTEXT("BeginSculptMeshSmoothBrushTool_short", "VSclptSmooth"), "Smooth Brush");
	REGISTER_MODELING_TOOL_COMMAND_WITH_SHORTNAME(BeginSculptMeshSmoothFillBrushTool, "SmoothFill", LOCTEXT("BeginSculptMeshSmoothFillBrushTool_short", "VSclptSmoothFill"), "Smooth Fill Brush");
	REGISTER_MODELING_TOOL_COMMAND_WITH_SHORTNAME(BeginSculptMeshOffsetBrushTool, "SculptN", LOCTEXT("BeginSculptMeshOffsetBrushTool_short", "VSclptOffset"), "Sculpt Brush");
	REGISTER_MODELING_TOOL_COMMAND_WITH_SHORTNAME(BeginSculptMeshSculptViewBrushTool, "SculptV", LOCTEXT("BeginSculptMeshSculptViewBrushTool_short", "VSclptSculptV"), "Sculpt to View Brush");
	REGISTER_MODELING_TOOL_COMMAND_WITH_SHORTNAME(BeginSculptMeshSculptMaxBrushTool, "SculptMx", LOCTEXT("BeginSculptMeshSculptMaxBrushTool_short", "VSclptSculptMx"), "Sculpt Max Brush");
	REGISTER_MODELING_TOOL_COMMAND_WITH_SHORTNAME(BeginSculptMeshInflateBrushTool, "Inflate", LOCTEXT("BeginSculptMeshInflateBrushTool_short", "VSclptInflate"), "Inflate Brush");
	REGISTER_MODELING_TOOL_COMMAND_WITH_SHORTNAME(BeginSculptMeshScaleKelvinBrushTool, "Scale", LOCTEXT("BeginSculptMeshScaleKelvinBrushTool_short", "VSclptScaleK"), "Kelvin Scale Brush");
	REGISTER_MODELING_TOOL_COMMAND_WITH_SHORTNAME(BeginSculptMeshPinchBrushTool, "Pinch", LOCTEXT("BeginSculptMeshPinchBrushTool_short", "VSclptPinch"), "Pinch Brush");
	REGISTER_MODELING_TOOL_COMMAND_WITH_SHORTNAME(BeginSculptMeshTwistKelvinBrushTool, "Twist", LOCTEXT("BeginSculptMeshTwistKelvinBrushTool_short", "VSclptKTwist"), "Kelvin Twist Brush");
	REGISTER_MODELING_TOOL_COMMAND_WITH_SHORTNAME(BeginSculptMeshFlattenBrushTool, "Flatten", LOCTEXT("BeginSculptMeshFlattenBrushTool_short", "VSclptFlatten"), "Flatten Brush");
	REGISTER_MODELING_TOOL_COMMAND_WITH_SHORTNAME(BeginSculptMeshPlaneBrushTool, "PlaneN", LOCTEXT("BeginSculptMeshPlaneBrushTool_short", "VSclptPlaneN"), "Plane Brush");
	REGISTER_MODELING_TOOL_COMMAND_WITH_SHORTNAME(BeginSculptMeshPlaneViewAlignedBrushTool, "PlaneV", LOCTEXT("BeginSculptMeshPlaneViewAlignedBrushTool_short", "VSclptPlaneV"), "View Plane Brush");
	REGISTER_MODELING_TOOL_COMMAND_WITH_SHORTNAME(BeginSculptMeshFixedPlaneBrushTool, "PlaneW", LOCTEXT("BeginSculptMeshFixedPlaneBrushTool_short", "VSclptFixedPlane"), "Fixed Plane Brush");
	REGISTER_MODELING_TOOL_COMMAND_WITH_SHORTNAME(BeginSculptMeshEraseLayerTool, "Erase", LOCTEXT("BeginSculptMeshEraseLayerTool_short", "Layer Erase"), "Erase Sculpt Layer");

	// Transform
	REGISTER_MODELING_TOOL_COMMAND_WITH_SHORTNAME(BeginTransformMeshesTool, "Transform", LOCTEXT("BeginTransformMeshesTool_short", "XForm"), "Transform the selected Meshes");
	REGISTER_MODELING_TOOL_COMMAND_WITH_SHORTNAME(BeginEditPivotTool, "Edit Pivot", LOCTEXT("BeginEditPivotTool_short", "Pivot"), "Edit the pivot points of the selected Meshes");
	REGISTER_MODELING_TOOL_COMMAND_WITH_SHORTNAME(BeginAddPivotActorTool, "Pivot Actor", LOCTEXT("BeginAddPivotActorTool_short", "PivotAct"), "Add actor to act as a pivot for child components");
	REGISTER_MODELING_TOOL_COMMAND_WITH_SHORTNAME(BeginBakeTransformTool, "Bake Transform", LOCTEXT("BeginBakeTransformTool_short", "BakeRS"), "Bake rotation and scale into the selected Meshes");
	REGISTER_MODELING_TOOL_COMMAND(BeginAlignObjectsTool, "Align", "Align the selected Objects");
	REGISTER_MODELING_TOOL_COMMAND(BeginTransferMeshTool, "Transfer", "Copy the first selected Mesh to the second selected Mesh");
	REGISTER_MODELING_TOOL_COMMAND(BeginConvertMeshesTool, "Convert", "Convert the selected Meshes to a different type of Mesh Object");
	REGISTER_MODELING_TOOL_COMMAND(BeginSplitMeshesTool, "Split", "Split the selected Meshes into separate parts based on connectivity, selection, material ID or PolyGroup");
	REGISTER_MODELING_TOOL_COMMAND(BeginPatternTool, "Pattern", "Create patterns of Meshes");
	REGISTER_MODELING_TOOL_COMMAND(BeginHarvestInstancesTool, "Harvest Instances", "Extract a set of InstancedStaticMeshComponents from a set of Actors");
	

	// MeshOps
	REGISTER_MODELING_TOOL_COMMAND_WITH_SHORTNAME(BeginSimplifyMeshTool, "Simplify", LOCTEXT("BeginSimplifyMeshTool_short", "Simplfy"), "Simplify the selected Mesh");
	REGISTER_MODELING_TOOL_COMMAND(BeginRemeshMeshTool, "Remesh", "Re-triangulate the selected Mesh");
	REGISTER_MODELING_TOOL_COMMAND(BeginWeldEdgesTool, "Weld", "Weld overlapping Mesh edges");
	REGISTER_MODELING_TOOL_COMMAND(BeginRemoveOccludedTrianglesTool, "Jacket", "Remove hidden triangles from the selected Meshes");
	REGISTER_MODELING_TOOL_COMMAND(BeginSelfUnionTool, "Union", "Boolean Union the selected Meshes, including Self-Union to resolve self-intersections");
	REGISTER_MODELING_TOOL_COMMAND(BeginProjectToTargetTool, "Project", "Map/re-mesh the first selected Mesh onto the second selected Mesh");


	// VoxOps
	REGISTER_MODELING_TOOL_COMMAND_WITH_SHORTNAME(BeginVoxelSolidifyTool, "Voxel Wrap", LOCTEXT("BeginVoxelSolidifyTool_short", "VoxWrap"), "Wrap the selected Meshes using SDFs/voxels");
	REGISTER_MODELING_TOOL_COMMAND_WITH_SHORTNAME(BeginVoxelBlendTool, "Voxel Blend", LOCTEXT("BeginVoxelBlendTool_short", "VoxBlnd"), "Blend the selected Meshes using SDFs/voxels");
	REGISTER_MODELING_TOOL_COMMAND_WITH_SHORTNAME(BeginVoxelMorphologyTool, "Voxel Offset", LOCTEXT("BeginVoxelMorphologyTool_short", "VoxMrph"), "Offset/Inset the selected Meshes using SDFs/voxels");
#if WITH_PROXYLOD
	// The ProxyLOD plugin is currently only available on Windows. Without it, the following tools do not work as expected.
	REGISTER_MODELING_TOOL_COMMAND_WITH_SHORTNAME(BeginVoxelBooleanTool, "Voxel Boolean", LOCTEXT("BeginVoxelBooleanTool_short", "VoxBool"), "Boolean the selected Meshes using SDFs/voxels");
	REGISTER_MODELING_TOOL_COMMAND_WITH_SHORTNAME(BeginVoxelMergeTool, "Voxel Merge", LOCTEXT("BeginVoxelMergeTool_short", "VoxMrg"), "Merge the selected Meshes using SDFs/voxels");
#endif	// WITH_PROXYLOD

	// Attributes
	REGISTER_MODELING_TOOL_COMMAND_WITH_SHORTNAME(BeginMeshInspectorTool, "Inspect", LOCTEXT("BeginMeshInspectorTool_short", "Inspct"), "Inspect Mesh attributes");
	REGISTER_MODELING_TOOL_COMMAND_WITH_SHORTNAME(BeginEditNormalsTool, "Normals", LOCTEXT("BeginEditNormalsTool_short", "Nrmls"), "Recompute or Repair Normals");
	REGISTER_MODELING_TOOL_COMMAND_WITH_SHORTNAME(BeginEditTangentsTool, "Tangents", LOCTEXT("BeginEditTangentsTool_short", "Tngnts"), "Recompute Tangents");
	REGISTER_MODELING_TOOL_COMMAND_WITH_SHORTNAME(BeginPolyGroupsTool, "Generate PolyGroups", LOCTEXT("BeginPolyGroupsTool_short", "GrpGen"), "Generate new PolyGroups");
	REGISTER_MODELING_TOOL_COMMAND_WITH_SHORTNAME(BeginMeshGroupPaintTool, "Paint PolyGroups", LOCTEXT("BeginMeshGroupPaintTool_short", "GrpPnt"), "Paint new PolyGroups");
	REGISTER_MODELING_TOOL_COMMAND_WITH_SHORTNAME(BeginMeshVertexPaintTool, "Paint Vertex Colors", LOCTEXT("BeginMeshVertexPaintTool_short", "VtxPnt"), "Paint Mesh Vertex Colors");
	REGISTER_MODELING_TOOL_COMMAND_WITH_SHORTNAME(BeginAttributeEditorTool, "Edit Attributes", LOCTEXT("BeginAttributeEditorTool_short", "AttrEd"), "Edit/configure Mesh attributes");
	REGISTER_MODELING_TOOL_COMMAND_WITH_SHORTNAME(BeginMeshAttributePaintTool, "Paint Maps", LOCTEXT("BeginMeshAttributePaintTool_short", "MapPnt"), "Paint attribute maps");
	REGISTER_MODELING_TOOL_COMMAND_WITH_SHORTNAME(BeginEditMeshMaterialsTool, "Edit Materials", LOCTEXT("BeginEditMeshMaterialsTool_short", "MatEd"), "Assign materials to selected triangles");

	// UVs
	REGISTER_MODELING_TOOL_COMMAND(BeginGlobalUVGenerateTool, "AutoUV", "Automatically unwrap and pack UVs");
	// This is done directly, not with the REGISTER_ macro, since we don't want it added to the tool list or use a toggle button
	UI_COMMAND(LaunchUVEditor, "UVEditor", "Launch UV asset editor", EUserInterfaceActionType::Button, FInputChord());
	REGISTER_MODELING_TOOL_COMMAND_WITH_SHORTNAME(BeginGroupUVGenerateTool, "UV Unwrap", LOCTEXT("BeginGroupUVGenerateTool_short", "Unwrap"), "Recompute UVs for existing UV islands or PolyGroups");
	REGISTER_MODELING_TOOL_COMMAND_WITH_SHORTNAME(BeginUVProjectionTool, "Project UVs", LOCTEXT("BeginUVProjectionTool_short", "Project"), "Compute UVs via projecting to simple shapes");
	REGISTER_MODELING_TOOL_COMMAND_WITH_SHORTNAME(BeginUVSeamEditTool, "Edit UV Seams", LOCTEXT("BeginUVSeamEditTool_short", "SeamEd"), "Add UV seams");
	REGISTER_MODELING_TOOL_COMMAND_WITH_SHORTNAME(BeginTransformUVIslandsTool, "Transform UVs", LOCTEXT("BeginTransformUVIslandsTool_short", "XFormUV"), "Transform UV islands in UV space");
	REGISTER_MODELING_TOOL_COMMAND_WITH_SHORTNAME(BeginUVLayoutTool, "Layout UVs", LOCTEXT("BeginUVLayoutTool_short", "Layout"), "Transform and Repack existing UVs");
	
	// Baking
	REGISTER_MODELING_TOOL_COMMAND_WITH_SHORTNAME(BeginBakeMeshAttributeMapsTool, "Bake Textures", LOCTEXT("BeginBakeMeshAttributeMapsTool_short", "BakeTx"), "Bake textures for a target Mesh");
	REGISTER_MODELING_TOOL_COMMAND_WITH_SHORTNAME(BeginBakeMultiMeshAttributeMapsTool, "Bake All", LOCTEXT("BeginBakeMultiMeshAttributeMapsTool_short", "BakeAll"), "Bake textures for a target Mesh from multiple source Meshes");
	REGISTER_MODELING_TOOL_COMMAND_WITH_SHORTNAME(BeginBakeMeshAttributeVertexTool, "Bake Vertex Colors", LOCTEXT("BeginBakeMeshAttributeVertexTool_short", "BakeVtx"), "Bake vertex colors for a target Mesh");
	REGISTER_MODELING_TOOL_COMMAND_WITH_SHORTNAME(BeginBakeRenderCaptureTool, "Bake RC", LOCTEXT("BeginBakeRenderCaptureTool_short", "BakeRC"), "Bake renders into new textures for a target Mesh from multiple source Meshes");

	// Volumes
	REGISTER_MODELING_TOOL_COMMAND_WITH_SHORTNAME(BeginVolumeToMeshTool, "Volume To Mesh", LOCTEXT("BeginVolumeToMeshTool_short", "Vol2Msh"), "Convert a Volume to a new Mesh Object");
	REGISTER_MODELING_TOOL_COMMAND_WITH_SHORTNAME(BeginMeshToVolumeTool, "Mesh To Volume", LOCTEXT("BeginMeshToVolumeTool_short", "Msh2Vol"), "Convert a Mesh to a Volume");
	if (!Settings->InRestrictiveMode())
	{
		REGISTER_MODELING_TOOL_COMMAND_WITH_SHORTNAME(BeginBspConversionTool, "Convert BSPs", LOCTEXT("BeginBspConversionTool_short", "BSPConv"), "Convert BSP to a new Mesh Object");
	}
	REGISTER_MODELING_TOOL_COMMAND_WITH_SHORTNAME(BeginPhysicsInspectorTool, "Inspect Collision", LOCTEXT("BeginPhysicsInspectorTool_short", "PInspct"), "Inspect the physics/collision geometry for selected Meshes");
	REGISTER_MODELING_TOOL_COMMAND_WITH_SHORTNAME(BeginSimpleCollisionEditorTool, "Simple Collision Editor", LOCTEXT("BeginSimpleCollisionEditorTool_short", "SCollEdit"), "Edit the simple collision shapes for the selected Mesh");
	REGISTER_MODELING_TOOL_COMMAND_WITH_SHORTNAME(BeginSetCollisionGeometryTool, "Mesh To Collision", LOCTEXT("BeginSetCollisionGeometryTool_short", "Msh2Coll"), "Convert selected Meshes to Simple Collision Geometry (for last selected)");
	REGISTER_MODELING_TOOL_COMMAND_WITH_SHORTNAME(BeginExtractCollisionGeometryTool, "Collision To Mesh", LOCTEXT("BeginExtractCollisionGeometryTool_short", "Coll2Msh"), "Convert Simple Collision Geometry to a new Mesh Object");

	// LODs
	REGISTER_MODELING_TOOL_COMMAND_WITH_SHORTNAME(BeginLODManagerTool, "LOD Manager", LOCTEXT("BeginLODManagerTool_short", "LODMgr"), "Inspect the LODs of a Static Mesh Asset");
	REGISTER_MODELING_TOOL_COMMAND(BeginGenerateStaticMeshLODAssetTool, "AutoLOD", "Automatically generate a simplified LOD with baked Textures/Materials for a Mesh");
	REGISTER_MODELING_TOOL_COMMAND_WITH_SHORTNAME(BeginISMEditorTool, "ISM Editor", LOCTEXT("BeginISMEditorTool_short", "ISMEd"), "Edit the Instances of Instanced Static Mesh Components");

	REGISTER_MODELING_TOOL_COMMAND(BeginAddPatchTool, "Patch", "Add Patch");
	REGISTER_MODELING_TOOL_COMMAND(BeginShapeSprayTool, "Spray", "Shape Spray");
	REGISTER_MODELING_TOOL_COMMAND(BeginEditCollisionGeometryTool, "EditPhys", "Edit Simple Collision Geometry for selected Mesh");
	
	REGISTER_MODELING_TOOL_COMMAND(BeginSkinWeightsPaintTool, "Edit Weights", "Tune the per-vertex skin weights.");
	REGISTER_MODELING_TOOL_COMMAND(BeginSkinWeightsBindingTool, "Bind Skin", "Create default weights by binding the skin to bones.");

	REGISTER_MODELING_TOOL_COMMAND(BeginSkeletonEditingTool, "Edit Skeleton", "Add, Remove, Reparent, Move and Rename bones.");

	REGISTER_MODELING_TOOL_COMMAND(BeginPolyModelTool_Inset, "Inset", "Inset the current Selected Faces");
	REGISTER_MODELING_TOOL_COMMAND(BeginPolyModelTool_Outset, "Outset", "Outset the current Selected Faces");
	REGISTER_MODELING_TOOL_COMMAND(BeginPolyModelTool_CutFaces, "Cut", "Cut the current Selected Faces");
	REGISTER_MODELING_TOOL_COMMAND_WITH_SHORTNAME(BeginPolyModelTool_ExtrudeEdges, "Extrude Edges", LOCTEXT("BeginPolyModelTool_ExtrudeEdges_short", "ExtEdg"), "Extrude selected boundary edges.");
	REGISTER_MODELING_TOOL_COMMAND(BeginPolyModelTool_PushPull, "PushPull", "Push/Pull the current Selected Faces");
	REGISTER_MODELING_TOOL_COMMAND(BeginPolyModelTool_Bevel, "Bevel", "Bevel the current Mesh Selection (Edges or Faces)");
	REGISTER_MODELING_TOOL_COMMAND_WITH_SHORTNAME(BeginPolyModelTool_InsertEdgeLoop, "Insert Loops", LOCTEXT("BeginPolyModelTool_InsertEdgeLoop_short", "ELoop"), "Insert Edge Loops into the Selected Mesh");

	REGISTER_MODELING_TOOL_COMMAND_WITH_SHORTNAME(BeginPolyModelTool_PolyEd, "PolyGroup Edit", LOCTEXT("BeginPolyModelTool_PolyEd_short", "PolyEd"), "Select / Edit the current Mesh via PolyGroups");
	REGISTER_MODELING_TOOL_COMMAND_WITH_SHORTNAME(BeginPolyModelTool_TriSel, "Tri Select", LOCTEXT("BeginPolyModelTool_TriSel_short", "TriSel"), "Select / Edit the current Mesh triangles with a brush interface");

	REGISTER_MODELING_TOOL_COMMAND_RADIO(MeshSelectionModeAction_NoSelection, "Object", "Disable Mesh Element Selection (Select Objects Only)");
	REGISTER_MODELING_TOOL_COMMAND_RADIO(MeshSelectionModeAction_MeshTriangles, "Mesh Triangles", "Select Mesh Triangles");
	REGISTER_MODELING_TOOL_COMMAND_RADIO(MeshSelectionModeAction_MeshVertices, "Mesh Vertices", "Select Mesh Vertices");
	REGISTER_MODELING_TOOL_COMMAND_RADIO(MeshSelectionModeAction_MeshEdges, "Mesh Edges", "Select Mesh Edges");
	REGISTER_MODELING_TOOL_COMMAND_RADIO(MeshSelectionModeAction_GroupFaces, "PolyGroups", "Select Mesh PolyGroups");
	REGISTER_MODELING_TOOL_COMMAND_RADIO(MeshSelectionModeAction_GroupCorners, "PolyGroup Corners", "Select Mesh PolyGroup Corners/Vertices");
	REGISTER_MODELING_TOOL_COMMAND_RADIO(MeshSelectionModeAction_GroupEdges, "PolyGroup Borders", "Select Mesh PolyGroup Borders/Edges");

	REGISTER_MODELING_ACTION_COMMAND(BeginSelectionAction_Delete, "Delete", "Delete the current Mesh Selection");
	REGISTER_MODELING_ACTION_COMMAND(BeginSelectionAction_Retriangulate, "Clean", "Retriangulate the current Mesh or Mesh Selection");
	REGISTER_MODELING_ACTION_COMMAND_WITH_SHORTNAME(BeginSelectionAction_Disconnect, "Disconnect", LOCTEXT("BeginSelectionAction_Disconnect_short", "Discon"), "Disconnect the current Mesh Selection");
	REGISTER_MODELING_TOOL_COMMAND(BeginSelectionAction_Extrude, "Extrude", "Extrude the current Mesh Selection");
	REGISTER_MODELING_TOOL_COMMAND(BeginSelectionAction_Offset, "Offset", "Offset the current Mesh Selection");

	REGISTER_MODELING_ACTION_COMMAND(BeginSelectionAction_SelectAll, "Select All", "Select All Elements");
	REGISTER_MODELING_ACTION_COMMAND(BeginSelectionAction_ExpandToConnected, "Expand To Connected", "Expand Selection to Geometrically-Connected Elements");
	REGISTER_MODELING_ACTION_COMMAND(BeginSelectionAction_Invert, "Invert Selection", "Invert the current Selection");
	REGISTER_MODELING_ACTION_COMMAND(BeginSelectionAction_InvertConnected, "Invert Connected", "Invert the current Selection to Geometrically-Connected Elements");
	REGISTER_MODELING_ACTION_COMMAND(BeginSelectionAction_Expand, "Expand Selection", "Expand the current Selection by a ring of elements");
	REGISTER_MODELING_ACTION_COMMAND(BeginSelectionAction_Contract, "Contract Selection", "Contract the current Selection by a ring of elements");

	REGISTER_MODELING_TOOL_COMMAND_RADIO(SelectionDragMode_None, "None", "No drag input");
	REGISTER_MODELING_TOOL_COMMAND_RADIO(SelectionDragMode_Path, "Path", "Path drag input");
	REGISTER_MODELING_TOOL_COMMAND_RADIO(SelectionLocalFrameMode_Geometry, "From Geometry", "Gizmo orientation based on selected geometry");
	REGISTER_MODELING_TOOL_COMMAND_RADIO(SelectionLocalFrameMode_Object, "From Object", "Gizmo orientation based on object");
	REGISTER_MODELING_TOOL_COMMAND(SelectionMeshTypes_Volumes, "Volumes", "Toggle whether Volume mesh elements can be selected in the Viewport");
	REGISTER_MODELING_TOOL_COMMAND(SelectionMeshTypes_StaticMeshes, "Static Meshes", "Toggle whether Static Mesh mesh elements can be selected in the Viewport");

	REGISTER_MODELING_TOOL_COMMAND(SelectionLocking, "Lock Target", "Click to Toggle locking on the Selected Object and disallow/allow Mesh Selections");
	
	UI_COMMAND(AddToFavorites, "Add to Favorites", "Add to Favorites", EUserInterfaceActionType::Button, FInputChord());
	UI_COMMAND(RemoveFromFavorites, "Remove from Favorites", "Remove from Favorites", EUserInterfaceActionType::Button, FInputChord());
	UI_COMMAND(LoadFavoritesTools, "Faves", "Favorites", EUserInterfaceActionType::RadioButton, FInputChord());
	UI_COMMAND(LoadSelectionTools, "Select", "Edit Mesh Selections", EUserInterfaceActionType::RadioButton, FInputChord());
	UI_COMMAND(LoadShapesTools, "Shapes", "Shapes", EUserInterfaceActionType::RadioButton, FInputChord());
	UI_COMMAND(LoadCreateTools, "Create", "Create New Shapes", EUserInterfaceActionType::RadioButton, FInputChord());
	UI_COMMAND(LoadPolyTools, "Model", "Shape Modeling", EUserInterfaceActionType::RadioButton, FInputChord());
	UI_COMMAND(LoadTriTools, "Process", "Mesh Processing", EUserInterfaceActionType::RadioButton, FInputChord());
	UI_COMMAND(LoadDeformTools, "Deform", "Deformations", EUserInterfaceActionType::RadioButton, FInputChord());
	UI_COMMAND(LoadTransformTools, "XForm", "Transforms & Conversion", EUserInterfaceActionType::RadioButton, FInputChord());
	UI_COMMAND(LoadMeshOpsTools, "Mesh", "Mesh Processing", EUserInterfaceActionType::RadioButton, FInputChord());
	UI_COMMAND(LoadVoxOpsTools, "Voxel", "Voxel Processing", EUserInterfaceActionType::RadioButton, FInputChord());
	UI_COMMAND(LoadAttributesTools, "Attribs", "Mesh Attributes", EUserInterfaceActionType::RadioButton, FInputChord());
	UI_COMMAND(LoadUVsTools, "UVs", "Create & Edit UVs", EUserInterfaceActionType::RadioButton, FInputChord());
	UI_COMMAND(LoadBakingTools, "Bake", "Bake Textures & Colors", EUserInterfaceActionType::RadioButton, FInputChord());
	UI_COMMAND(LoadVolumeTools, "Volumes", "Volumes", EUserInterfaceActionType::RadioButton, FInputChord());
	UI_COMMAND(LoadLodsTools, "Misc", "Additional Utility Tools", EUserInterfaceActionType::RadioButton, FInputChord());
	UI_COMMAND(LoadSkinTools, "Skin", "Edit Skin Weights", EUserInterfaceActionType::RadioButton, FInputChord());
	UI_COMMAND(LoadSkeletonTools, "Skeleton", "Edit Bones", EUserInterfaceActionType::RadioButton, FInputChord());

	// MegaMesh
	UI_COMMAND(BeginConvertMegaMeshTool, "Convert", "Convert the target mesh into a new Mega Mesh actor", EUserInterfaceActionType::ToggleButton, FInputChord());
	UI_COMMAND(BeginSplitMegaMeshTool, "Split", "Split a mesh into multiple Mega Mesh section actors", EUserInterfaceActionType::ToggleButton, FInputChord());
	UI_COMMAND(BeginMergeMegaMeshTool, "Merge", "Merges multiple Mega Mesh sections into a single section actor", EUserInterfaceActionType::ToggleButton, FInputChord());
	UI_COMMAND(BeginResectionMeshTool, "Resection", "Merge and then re-split selected sections", EUserInterfaceActionType::ToggleButton, FInputChord());
	UI_COMMAND(BeginStitchMegaMeshTool, "Stitch", "Stitch a regular mesh into a Mega Mesh section", EUserInterfaceActionType::ToggleButton, FInputChord());
	UI_COMMAND(BeginHeightmapImport, "Import Heightmap", "Create a new Mega Mesh from a heightmap", EUserInterfaceActionType::ToggleButton, FInputChord());
	UI_COMMAND(BeginHeightSculptTool, "Height Sculpt", "Sculpt height based on a reference surface", EUserInterfaceActionType::ToggleButton, FInputChord());
	UI_COMMAND(BeginExpandMegaMeshTool, "Expand", "Insert new Mega Mesh sections at section boundaries", EUserInterfaceActionType::ToggleButton, FInputChord());
	
	UI_COMMAND(BeginHeightSculptBrushTool, "Height Sculpt", "Height Sculpt", EUserInterfaceActionType::ToggleButton, FInputChord());
	UI_COMMAND(BeginHeightSmoothBrushTool, "Height Smooth", "Height Smooth", EUserInterfaceActionType::ToggleButton, FInputChord());
	UI_COMMAND(BeginHeightFlattenBrushTool, "Height Flatten", "Height Flatten", EUserInterfaceActionType::ToggleButton, FInputChord());
	UI_COMMAND(BeginSlopeErodeBrushTool, "Slope Erode", "Slope Erode", EUserInterfaceActionType::ToggleButton, FInputChord());

	UI_COMMAND(BeginCreateMegaMeshRectangleTool, "Create Rectangle", "Create a new rectangular Mega Mesh", EUserInterfaceActionType::ToggleButton, FInputChord());
	UI_COMMAND(BeginAddModifierTool, "Add Modifier", "Create a new Mega Mesh modifier", EUserInterfaceActionType::ToggleButton, FInputChord());

	UI_COMMAND(BeginAddRemeshModifierTool, "Remesh", "Add Remesh Modifier", EUserInterfaceActionType::ToggleButton, FInputChord());
	UI_COMMAND(BeginAddPatchModifierTool, "Patch", "Add Patch Modifier", EUserInterfaceActionType::ToggleButton, FInputChord());
	UI_COMMAND(BeginAddProjectModifierTool, "Project", "Add Project Modifier", EUserInterfaceActionType::ToggleButton, FInputChord());
	UI_COMMAND(BeginAddInstancedPatchModifierTool, "Instanced Patch", "Add Instanced Patch Modifier", EUserInterfaceActionType::ToggleButton, FInputChord());
	UI_COMMAND(BeginAddTexturePatchModifierTool, "Texture Patch", "Add Texture Patch Modifier", EUserInterfaceActionType::ToggleButton, FInputChord());
	UI_COMMAND(BeginAddSplineModifierTool, "Spline", "Add Spline Modifier", EUserInterfaceActionType::ToggleButton, FInputChord());
	UI_COMMAND(BeginAddMeshLayerModifierTool, "Mesh Layer", "Add Mesh Layer Modifier", EUserInterfaceActionType::ToggleButton, FInputChord());
	UI_COMMAND(BeginAddNoiseModifierTool, "Noise", "Add Noise Modifier", EUserInterfaceActionType::ToggleButton, FInputChord());
	UI_COMMAND(BeginAddBooleanModifierTool, "Boolean", "Add Boolean Modifier", EUserInterfaceActionType::ToggleButton, FInputChord());
	UI_COMMAND(BeginAddLatticeModifierTool, "Lattice", "Add Lattice Modifier", EUserInterfaceActionType::ToggleButton, FInputChord());
	UI_COMMAND(BeginAddSplineRemeshModifierTool, "Spline Remesh", "Add Spline Remesh Modifier", EUserInterfaceActionType::ToggleButton, FInputChord());
	UI_COMMAND(RelaunchLastTool, "Relaunch Last MeshTerrain Tool", "Re-select the last target and restart the last tool", EUserInterfaceActionType::Button, FInputChord());

	UI_COMMAND(EnterShapesSubmode, "Shapes", "Shapes", EUserInterfaceActionType::ToggleButton, FInputChord());
	UI_COMMAND(EnterCreateSubmode, "Create", "Create", EUserInterfaceActionType::ToggleButton, FInputChord());
	UI_COMMAND(EnterEditSubmode, "Edit", "Edit", EUserInterfaceActionType::ToggleButton, FInputChord());
	UI_COMMAND(EnterSculptSubmode, "Sculpt", "Sculpt", EUserInterfaceActionType::ToggleButton, FInputChord());
	UI_COMMAND(EnterPaintSubmode, "Paint", "Paint", EUserInterfaceActionType::ToggleButton, FInputChord());
	UI_COMMAND(EnterModifiersSubmode, "Modifiers", "Modifiers", EUserInterfaceActionType::ToggleButton, FInputChord());

	UI_COMMAND(AcceptActiveTool, "Accept", "Accept the active Tool", EUserInterfaceActionType::Button, FInputChord());
	UI_COMMAND(CancelActiveTool, "Cancel", "Cancel the active Tool", EUserInterfaceActionType::Button, FInputChord());
	UI_COMMAND(CompleteActiveTool, "Done", "Complete the active Tool", EUserInterfaceActionType::Button, FInputChord());

	// Note that passing a chord into one of these calls hooks the key press to the respective action. 
	UI_COMMAND(AcceptOrCompleteActiveTool, "Accept or Complete", "Accept or Complete the active Tool", EUserInterfaceActionType::Button, FInputChord(EKeys::Enter));
	UI_COMMAND(CancelOrCompleteActiveTool, "Cancel or Complete", "Cancel or Complete the active Tool", EUserInterfaceActionType::Button, FInputChord(EKeys::Escape));
	
	for (int32 i = 0; i < RegisteredTools.Num(); ++i)
	{
		CommandToRegisteredToolsIndex.Add(RegisteredTools[i].ToolCommand.Get(), i);
	}
#undef REGISTER_MODELING_TOOL_COMMAND
#undef REGISTER_MODELING_TOOL_COMMAND_WITH_SHORTNAME
#undef REGISTER_MODELING_ACTION_COMMAND
#undef REGISTER_MODELING_ACTION_COMMAND_WITH_SHORTNAME
}


TSharedPtr<FUICommandInfo> FMeshTerrainModeManagerCommands::RegisterExtensionPaletteCommand(
	FName Name,
	const FText& Label,
	const FText& Tooltip,
	const FSlateIcon& Icon)
{
	if (IsRegistered())
	{
		TSharedPtr<FMeshTerrainModeManagerCommands> Commands = GetInstance().Pin();

		for (FDynamicExtensionCommand& ExtensionCommand : Commands->ExtensionPaletteCommands)
		{
			if (ExtensionCommand.RegistrationName == Name)
			{
				return ExtensionCommand.Command;
			}
		}

		TSharedPtr<FUICommandInfo> NewCommandInfo;

		FUICommandInfo::MakeCommandInfo(
			Commands->AsShared(),
			NewCommandInfo,
			Name, Label, Tooltip, Icon,
			EUserInterfaceActionType::RadioButton,
			FInputChord() );

		FDynamicExtensionCommand NewExtensionCommand;
		NewExtensionCommand.RegistrationName = Name;
		NewExtensionCommand.Command = NewCommandInfo;
		Commands->ExtensionPaletteCommands.Add(NewExtensionCommand);
		return NewCommandInfo;
	};

	return TSharedPtr<FUICommandInfo>();
}



#undef LOCTEXT_NAMESPACE
