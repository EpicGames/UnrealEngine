// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Framework/Commands/Commands.h"


/**
 * TInteractiveToolCommands implementation for Modeling Mode Tools
 */
class FMeshTerrainModeManagerCommands : public TCommands<FMeshTerrainModeManagerCommands>
{
public:
	FMeshTerrainModeManagerCommands();

protected:
	struct FStartToolCommand
	{
		FString ToolUIName;
		TSharedPtr<FUICommandInfo> ToolCommand;

		// Optional (not used if empty). Only relevant if using short labels for tools
		FText ShortName;
	};
	TArray<FStartToolCommand> RegisteredTools;		// Tool start-commands listed below are stored in this list

	struct FDynamicExtensionCommand
	{
		FName RegistrationName;
		TSharedPtr<FUICommandInfo> Command;
	};
	TArray<FDynamicExtensionCommand> ExtensionPaletteCommands;

public:
	/**
	 * Find Tool start-command below by registered name (tool icon name in Mode palette)
	 */
	TSharedPtr<FUICommandInfo> FindToolByName(FString Name, bool& bFound) const;
	
	/**
	 * Helper that gets the regular command label (result of GetLabel) or a shortened version
	 *  if one was registered and bShortName is true.
	 */
	FText GetCommandLabel(TSharedPtr<const FUICommandInfo> Command, bool bShortName) const;

	//
	// These commands are set up to launch registered Tools via the ToolManager in ModelingToolsEditorMode.cpp
	//

	TSharedPtr<FUICommandInfo> BeginAddBoxPrimitiveTool;
	TSharedPtr<FUICommandInfo> BeginAddCylinderPrimitiveTool;
	TSharedPtr<FUICommandInfo> BeginAddCapsulePrimitiveTool;
	TSharedPtr<FUICommandInfo> BeginAddConePrimitiveTool;
	TSharedPtr<FUICommandInfo> BeginAddArrowPrimitiveTool;
	TSharedPtr<FUICommandInfo> BeginAddRectanglePrimitiveTool;
	TSharedPtr<FUICommandInfo> BeginAddDiscPrimitiveTool;
	TSharedPtr<FUICommandInfo> BeginAddTorusPrimitiveTool;
	TSharedPtr<FUICommandInfo> BeginAddSpherePrimitiveTool;
	TSharedPtr<FUICommandInfo> BeginAddStairsPrimitiveTool;

	TSharedPtr<FUICommandInfo> BeginAddPatchTool;
	TSharedPtr<FUICommandInfo> BeginRevolveBoundaryTool;
	TSharedPtr<FUICommandInfo> BeginDrawPolygonTool;
	TSharedPtr<FUICommandInfo> BeginDrawPolyPathTool;
	TSharedPtr<FUICommandInfo> BeginDrawAndRevolveTool;
	TSharedPtr<FUICommandInfo> BeginRevolveSplineTool;
	TSharedPtr<FUICommandInfo> BeginShapeSprayTool;
	TSharedPtr<FUICommandInfo> BeginDrawSplineTool;
	TSharedPtr<FUICommandInfo> BeginTriangulateSplinesTool;

	TSharedPtr<FUICommandInfo> BeginSculptMeshTool;
	TSharedPtr<FUICommandInfo> BeginCubeGridTool;
	TSharedPtr<FUICommandInfo> BeginPolyEditTool;
	TSharedPtr<FUICommandInfo> BeginSubdividePolyTool;
	TSharedPtr<FUICommandInfo> BeginTriEditTool;
	TSharedPtr<FUICommandInfo> BeginPolyDeformTool;
	TSharedPtr<FUICommandInfo> BeginSmoothMeshTool;
	TSharedPtr<FUICommandInfo> BeginOffsetMeshTool;
	TSharedPtr<FUICommandInfo> BeginDisplaceMeshTool;
	TSharedPtr<FUICommandInfo> BeginMeshSpaceDeformerTool;
	TSharedPtr<FUICommandInfo> BeginTransformMeshesTool;
	TSharedPtr<FUICommandInfo> BeginEditPivotTool;
	TSharedPtr<FUICommandInfo> BeginAddPivotActorTool;
	TSharedPtr<FUICommandInfo> BeginBakeTransformTool;
	TSharedPtr<FUICommandInfo> BeginCombineMeshesTool;
	TSharedPtr<FUICommandInfo> BeginDuplicateMeshesTool;
	TSharedPtr<FUICommandInfo> BeginAlignObjectsTool;
	TSharedPtr<FUICommandInfo> BeginTransferMeshTool;
	TSharedPtr<FUICommandInfo> BeginConvertMeshesTool;
	TSharedPtr<FUICommandInfo> BeginSplitMeshesTool;
	TSharedPtr<FUICommandInfo> BeginPatternTool;
	TSharedPtr<FUICommandInfo> BeginHarvestInstancesTool;

	TSharedPtr<FUICommandInfo> BeginSculptMeshMoveBrushTool;
	TSharedPtr<FUICommandInfo> BeginSculptMeshPullKelvinBrushTool;
	TSharedPtr<FUICommandInfo> BeginSculptMeshPullSharpKelvinBrushTool;
	TSharedPtr<FUICommandInfo> BeginSculptMeshSmoothBrushTool;
	TSharedPtr<FUICommandInfo> BeginSculptMeshSmoothFillBrushTool;
	TSharedPtr<FUICommandInfo> BeginSculptMeshOffsetBrushTool;
	TSharedPtr<FUICommandInfo> BeginSculptMeshSculptViewBrushTool;
	TSharedPtr<FUICommandInfo> BeginSculptMeshSculptMaxBrushTool;
	TSharedPtr<FUICommandInfo> BeginSculptMeshInflateBrushTool;
	TSharedPtr<FUICommandInfo> BeginSculptMeshScaleKelvinBrushTool;
	TSharedPtr<FUICommandInfo> BeginSculptMeshPinchBrushTool;
	TSharedPtr<FUICommandInfo> BeginSculptMeshTwistKelvinBrushTool;
	TSharedPtr<FUICommandInfo> BeginSculptMeshFlattenBrushTool;
	TSharedPtr<FUICommandInfo> BeginSculptMeshPlaneBrushTool;
	TSharedPtr<FUICommandInfo> BeginSculptMeshPlaneViewAlignedBrushTool;
	TSharedPtr<FUICommandInfo> BeginSculptMeshFixedPlaneBrushTool;
	TSharedPtr<FUICommandInfo> BeginSculptMeshEraseLayerTool;

	TSharedPtr<FUICommandInfo> BeginRemeshSculptMeshTool;
	TSharedPtr<FUICommandInfo> BeginRemeshMeshTool;
	TSharedPtr<FUICommandInfo> BeginProjectToTargetTool;
	TSharedPtr<FUICommandInfo> BeginSimplifyMeshTool;
	TSharedPtr<FUICommandInfo> BeginEditNormalsTool;
	TSharedPtr<FUICommandInfo> BeginEditTangentsTool;
	TSharedPtr<FUICommandInfo> BeginRemoveOccludedTrianglesTool;
	TSharedPtr<FUICommandInfo> BeginUVProjectionTool;
	TSharedPtr<FUICommandInfo> BeginUVLayoutTool;
	TSharedPtr<FUICommandInfo> BeginPlaneCutTool;
	TSharedPtr<FUICommandInfo> BeginMirrorTool;
	TSharedPtr<FUICommandInfo> BeginHoleFillTool;
	TSharedPtr<FUICommandInfo> BeginLatticeDeformerTool;
	TSharedPtr<FUICommandInfo> BeginPolygonCutTool;

#if WITH_PROXYLOD
	TSharedPtr<FUICommandInfo> BeginVoxelMergeTool;
	TSharedPtr<FUICommandInfo> BeginVoxelBooleanTool;
#endif	// WITH_PROXYLOD

	TSharedPtr<FUICommandInfo> BeginVoxelSolidifyTool;
	TSharedPtr<FUICommandInfo> BeginVoxelBlendTool;
	TSharedPtr<FUICommandInfo> BeginVoxelMorphologyTool;
	TSharedPtr<FUICommandInfo> BeginSelfUnionTool;
	TSharedPtr<FUICommandInfo> BeginMeshBooleanTool;
	TSharedPtr<FUICommandInfo> BeginMeshTrimTool;
	TSharedPtr<FUICommandInfo> BeginCutMeshWithMeshTool;
	TSharedPtr<FUICommandInfo> BeginMeshSelectionTool;
	TSharedPtr<FUICommandInfo> BeginBspConversionTool;
	TSharedPtr<FUICommandInfo> BeginMeshToVolumeTool;
	TSharedPtr<FUICommandInfo> BeginVolumeToMeshTool;
	TSharedPtr<FUICommandInfo> BeginMeshGroupPaintTool;
	TSharedPtr<FUICommandInfo> BeginMeshVertexPaintTool;

	TSharedPtr<FUICommandInfo> BeginPhysicsInspectorTool;
	TSharedPtr<FUICommandInfo> BeginSimpleCollisionEditorTool;
	TSharedPtr<FUICommandInfo> BeginSetCollisionGeometryTool;
	TSharedPtr<FUICommandInfo> BeginEditCollisionGeometryTool;
	TSharedPtr<FUICommandInfo> BeginExtractCollisionGeometryTool;

	TSharedPtr<FUICommandInfo> BeginMeshInspectorTool;
	TSharedPtr<FUICommandInfo> BeginGlobalUVGenerateTool;
	TSharedPtr<FUICommandInfo> BeginGroupUVGenerateTool;
	TSharedPtr<FUICommandInfo> BeginWeldEdgesTool;
	TSharedPtr<FUICommandInfo> BeginPolyGroupsTool;
	TSharedPtr<FUICommandInfo> BeginEditMeshMaterialsTool;
	TSharedPtr<FUICommandInfo> BeginTransformUVIslandsTool;
	TSharedPtr<FUICommandInfo> BeginMeshAttributePaintTool;
	TSharedPtr<FUICommandInfo> BeginAttributeEditorTool;
	TSharedPtr<FUICommandInfo> BeginBakeMeshAttributeMapsTool;
	TSharedPtr<FUICommandInfo> BeginBakeMultiMeshAttributeMapsTool;
	TSharedPtr<FUICommandInfo> BeginBakeRenderCaptureTool;
	TSharedPtr<FUICommandInfo> BeginBakeMeshAttributeVertexTool;
	TSharedPtr<FUICommandInfo> BeginUVSeamEditTool;

	TSharedPtr<FUICommandInfo> BeginSkinWeightsPaintTool;
	TSharedPtr<FUICommandInfo> BeginSkinWeightsBindingTool;

	TSharedPtr<FUICommandInfo> BeginSkeletonEditingTool;
	
	TSharedPtr<FUICommandInfo> BeginLODManagerTool;
	TSharedPtr<FUICommandInfo> BeginGenerateStaticMeshLODAssetTool;
	TSharedPtr<FUICommandInfo> BeginISMEditorTool;

	TSharedPtr<FUICommandInfo> BeginPolyModelTool_Inset;
	TSharedPtr<FUICommandInfo> BeginPolyModelTool_Outset;
	TSharedPtr<FUICommandInfo> BeginPolyModelTool_CutFaces;
	TSharedPtr<FUICommandInfo> BeginPolyModelTool_InsertEdgeLoop;
	TSharedPtr<FUICommandInfo> BeginPolyModelTool_ExtrudeEdges;
	TSharedPtr<FUICommandInfo> BeginPolyModelTool_PushPull;
	TSharedPtr<FUICommandInfo> BeginPolyModelTool_Bevel;
	
	TSharedPtr<FUICommandInfo> BeginPolyModelTool_PolyEd;
	TSharedPtr<FUICommandInfo> BeginPolyModelTool_TriSel;


	// selection state actions
	TSharedPtr<FUICommandInfo> MeshSelectionModeAction_NoSelection;
	TSharedPtr<FUICommandInfo> MeshSelectionModeAction_MeshTriangles;
	TSharedPtr<FUICommandInfo> MeshSelectionModeAction_MeshVertices;
	TSharedPtr<FUICommandInfo> MeshSelectionModeAction_MeshEdges;
	TSharedPtr<FUICommandInfo> MeshSelectionModeAction_GroupFaces;
	TSharedPtr<FUICommandInfo> MeshSelectionModeAction_GroupCorners;
	TSharedPtr<FUICommandInfo> MeshSelectionModeAction_GroupEdges;


	TSharedPtr<FUICommandInfo> BeginSelectionAction_Delete;
	TSharedPtr<FUICommandInfo> BeginSelectionAction_Disconnect;
	TSharedPtr<FUICommandInfo> BeginSelectionAction_Retriangulate;

	TSharedPtr<FUICommandInfo> BeginSelectionAction_SelectAll;
	TSharedPtr<FUICommandInfo> BeginSelectionAction_ExpandToConnected;
	TSharedPtr<FUICommandInfo> BeginSelectionAction_Invert;
	TSharedPtr<FUICommandInfo> BeginSelectionAction_InvertConnected;
	TSharedPtr<FUICommandInfo> BeginSelectionAction_Expand;
	TSharedPtr<FUICommandInfo> BeginSelectionAction_Contract;

	TSharedPtr<FUICommandInfo> BeginSelectionAction_Extrude;
	TSharedPtr<FUICommandInfo> BeginSelectionAction_Offset;

	TSharedPtr<FUICommandInfo> SelectionDragMode_None;
	TSharedPtr<FUICommandInfo> SelectionDragMode_Path;
	TSharedPtr<FUICommandInfo> SelectionLocalFrameMode_Geometry;
	TSharedPtr<FUICommandInfo> SelectionLocalFrameMode_Object;
	TSharedPtr<FUICommandInfo> SelectionMeshTypes_Volumes;
	TSharedPtr<FUICommandInfo> SelectionMeshTypes_StaticMeshes;
	TSharedPtr<FUICommandInfo> SelectionLocking;

	TSharedPtr<FUICommandInfo> AddToFavorites;
	TSharedPtr<FUICommandInfo> RemoveFromFavorites;
	TSharedPtr<FUICommandInfo> LoadFavoritesTools;
	TSharedPtr<FUICommandInfo> LoadSelectionTools;
	TSharedPtr<FUICommandInfo> LoadShapesTools;
	TSharedPtr<FUICommandInfo> LoadCreateTools;
	TSharedPtr<FUICommandInfo> LoadPolyTools;
	TSharedPtr<FUICommandInfo> LoadTriTools;
	TSharedPtr<FUICommandInfo> LoadDeformTools;
	TSharedPtr<FUICommandInfo> LoadTransformTools;
	TSharedPtr<FUICommandInfo> LoadMeshOpsTools;
	TSharedPtr<FUICommandInfo> LoadVoxOpsTools;
	TSharedPtr<FUICommandInfo> LoadAttributesTools;
	TSharedPtr<FUICommandInfo> LoadUVsTools;
	TSharedPtr<FUICommandInfo> LoadBakingTools;
	TSharedPtr<FUICommandInfo> LoadVolumeTools;
	TSharedPtr<FUICommandInfo> LoadLodsTools;
	// For connecting ModelingMode with UVEditor plugin
	TSharedPtr<FUICommandInfo> LaunchUVEditor;
	// skeletal mesh editing tools
	TSharedPtr<FUICommandInfo> LoadSkinTools;
	TSharedPtr<FUICommandInfo> LoadSkeletonTools;

	// MegaMesh tools
	TSharedPtr<FUICommandInfo> BeginConvertMegaMeshTool;
	TSharedPtr<FUICommandInfo> BeginSplitMegaMeshTool;
	TSharedPtr<FUICommandInfo> BeginMergeMegaMeshTool;
	TSharedPtr<FUICommandInfo> BeginResectionMeshTool;
	TSharedPtr<FUICommandInfo> BeginStitchMegaMeshTool;
	TSharedPtr<FUICommandInfo> BeginHeightmapImport;
	TSharedPtr<FUICommandInfo> BeginHeightSculptTool;
	TSharedPtr<FUICommandInfo> BeginExpandMegaMeshTool;

	TSharedPtr<FUICommandInfo> BeginHeightSculptBrushTool;
	TSharedPtr<FUICommandInfo> BeginHeightSmoothBrushTool;
	TSharedPtr<FUICommandInfo> BeginHeightFlattenBrushTool;
	TSharedPtr<FUICommandInfo> BeginSlopeErodeBrushTool;

	TSharedPtr<FUICommandInfo> BeginCreateMegaMeshRectangleTool;
	TSharedPtr<FUICommandInfo> BeginAddModifierTool;
	TSharedPtr<FUICommandInfo> BeginAppendModifierTool;

	TSharedPtr<FUICommandInfo> BeginAddRemeshModifierTool;
	TSharedPtr<FUICommandInfo> BeginAddPatchModifierTool;
	TSharedPtr<FUICommandInfo> BeginAddProjectModifierTool;
	TSharedPtr<FUICommandInfo> BeginAddInstancedPatchModifierTool;
	TSharedPtr<FUICommandInfo> BeginAddTexturePatchModifierTool;
	TSharedPtr<FUICommandInfo> BeginAddSplineModifierTool;
	TSharedPtr<FUICommandInfo> BeginAddMeshLayerModifierTool;
	TSharedPtr<FUICommandInfo> BeginAddNoiseModifierTool;
	TSharedPtr<FUICommandInfo> BeginAddBooleanModifierTool;
	TSharedPtr<FUICommandInfo> BeginAddLatticeModifierTool;
	TSharedPtr<FUICommandInfo> BeginAddSplineRemeshModifierTool;

	TSharedPtr<FUICommandInfo> RelaunchLastTool;

	// Submodes
	TSharedPtr<FUICommandInfo> EnterShapesSubmode;
	TSharedPtr<FUICommandInfo> EnterCreateSubmode;
	TSharedPtr<FUICommandInfo> EnterEditSubmode;
	TSharedPtr<FUICommandInfo> EnterSculptSubmode;
	TSharedPtr<FUICommandInfo> EnterBakeSubmode;
	TSharedPtr<FUICommandInfo> EnterPaintSubmode;
	TSharedPtr<FUICommandInfo> EnterModifiersSubmode;

	//
	// Accept/Cancel/Complete commands are used to end the active Tool via ToolManager
	//

	TSharedPtr<FUICommandInfo> AcceptActiveTool;
	TSharedPtr<FUICommandInfo> CancelActiveTool;
	TSharedPtr<FUICommandInfo> CompleteActiveTool;

	TSharedPtr<FUICommandInfo> AcceptOrCompleteActiveTool;
	TSharedPtr<FUICommandInfo> CancelOrCompleteActiveTool;

	/**
	 * Initialize above commands
	 */
	virtual void RegisterCommands() override;


	/**
	 * Dynamically register a new UI Command based on the given Name/Label/Tooltip/Icon.
	 * This is intended to be used outside of RegisterCommands(), eg by Modeling Mode extensions
	 * loaded when the Toolkit is created.
	 */
	static TSharedPtr<FUICommandInfo> RegisterExtensionPaletteCommand(
		FName Name,
		const FText& Label,
		const FText& Tooltip,
		const FSlateIcon& Icon);

private:
	// Gets index into RegisteredTools for a given command.
	TMap<FUICommandInfo*, int32> CommandToRegisteredToolsIndex;
};
