// Copyright Epic Games, Inc. All Rights Reserved.

#include "MeshTerrainModeModule.h"

#include "BaseTools/BaseBrushTool.h"
#include "Sculpting/MeshSculptToolBase.h" // for FBrushToolRadius

#include "MeshTerrainDetailCustomizations.h"
#include "MeshTerrainModeActions.h"
#include "MeshTerrainModeManagerActions.h"
#include "MeshTerrainModeStyle.h"

#include "Modules/ModuleManager.h"
#include "PropertyEditorModule.h"
#include "MeshTerrainDetailsSections.h"
#include "DetailsCustomizations/BrushBasePropertiesCustomizations.h"
#include "DetailsCustomizations/ModelingToolsBrushSizeCustomization.h"
#include "DetailsCustomizations/SculptSubmodeCustomizations.h"

#define LOCTEXT_NAMESPACE "FMeshTerrainModeModule"

namespace UE::MeshTerrain
{

void FMeshTerrainModeModule::StartupModule()
{
	FCoreDelegates::GetOnPostEngineInit().AddRaw(this, &FMeshTerrainModeModule::OnPostEngineInit);

	FMeshTerrainDetailsSections::Initialize();
	RegisterCustomizations();
}

void FMeshTerrainModeModule::ShutdownModule()
{
	FCoreDelegates::GetOnPostEngineInit().RemoveAll(this);

	FMeshTerrainModeToolActionCommands::UnregisterAllToolActions();
	FMeshTerrainModeManagerCommands::Unregister();
	FMeshTerrainModeActionCommands::Unregister();

	// Unregister customizations
	FPropertyEditorModule* PropertyEditorModule = FModuleManager::GetModulePtr<FPropertyEditorModule>("PropertyEditor");
	if (PropertyEditorModule)
	{
		for (FName ClassName : ClassesToUnregisterOnShutdown)
		{
			PropertyEditorModule->UnregisterCustomClassLayout(ClassName);
		}
		for (FName PropertyName : PropertiesToUnregisterOnShutdown)
		{
			PropertyEditorModule->UnregisterCustomPropertyTypeLayout(PropertyName);
		}
	}

	// Unregister slate style overrides
	FMeshTerrainModeStyle::Shutdown();
}


void FMeshTerrainModeModule::OnPostEngineInit()
{
	// Register slate style overrides
	FMeshTerrainModeStyle::Initialize();

	FMeshTerrainModeToolActionCommands::RegisterAllToolActions();
	FMeshTerrainModeManagerCommands::Register();
	FMeshTerrainModeActionCommands::Register();

	PropertiesToUnregisterOnShutdown.Reset();
	ClassesToUnregisterOnShutdown.Reset();
	
	FPropertyEditorModule& PropertyModule = FModuleManager::LoadModuleChecked<FPropertyEditorModule>("PropertyEditor");

	PropertyModule.RegisterCustomClassLayout("BrushBaseProperties", FOnGetDetailCustomizationInstance::CreateStatic(&UE::MeshTerrain::FBrushBasePropertiesDetails::MakeInstance));
	ClassesToUnregisterOnShutdown.Add(UBrushBaseProperties::StaticClass()->GetFName());

	PropertyModule.RegisterCustomPropertyTypeLayout("BrushToolRadius", FOnGetPropertyTypeCustomizationInstance::CreateStatic(&UE::MeshTerrain::FModelingToolsBrushSizeCustomization::MakeInstance));
	PropertiesToUnregisterOnShutdown.Add(FBrushToolRadius::StaticStruct()->GetFName());
}

void FMeshTerrainModeModule::RegisterCustomizations()
{
	UE::MeshTerrain::FMeshTerrainDetailCustomizations* Customizations = UE::MeshTerrain::FMeshTerrainDetailCustomizations::Get();

	// TODO: register all other customizations
	Customizations->RegisterCustomization(FName("SculptBrushProperties"),
		UE::MeshTerrain::FSculptBrushPropertiesCustomizations::MakeInstance());
	Customizations->RegisterCustomization(FName("MeshEditingViewProperties"),
		UE::MeshTerrain::FMeshEditingViewPropertiesCustomizations::MakeInstance());
}
	
IMPLEMENT_MODULE(FMeshTerrainModeModule, MeshTerrainMode)
}

#undef LOCTEXT_NAMESPACE