// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "CoreMinimal.h"
#include "Modules/ModuleInterface.h"

namespace UE::MeshTerrain
{
class IMeshTerrainPropertyCustomization;

class FMeshTerrainModeModule : public IModuleInterface
{
public:

	/** IModuleInterface implementation */
	virtual void StartupModule() override;
	virtual void ShutdownModule() override;

	void OnPostEngineInit();

private:
	/** StaticClass is not safe on shutdown, so we cache the name, and use this to unregister on shut down */
	TArray<FName> ClassesToUnregisterOnShutdown;
	TArray<FName> PropertiesToUnregisterOnShutdown;

	static void RegisterCustomizations();
};

}