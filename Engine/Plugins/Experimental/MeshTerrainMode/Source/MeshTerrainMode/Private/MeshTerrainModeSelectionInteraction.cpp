// Copyright Epic Games, Inc. All Rights Reserved.


#include "MeshTerrainModeSelectionInteraction.h"
#include "BaseGizmos/CombinedTransformGizmo.h"
#include "InteractiveToolsContext.h"
#include "BaseGizmos/TransformGizmoUtil.h"
#include "BaseGizmos/TransformProxy.h"
#include "Selection/GeometrySelectionManager.h"
#include "InteractiveGizmoManager.h"
#include "ToolSceneQueriesUtil.h"
#include "InteractiveToolManager.h"
#include "SceneQueries/SceneSnappingManager.h"

#include "BaseBehaviors/SingleClickOrDragBehavior.h"
#include "BaseBehaviors/MouseHoverBehavior.h"
#include "BaseBehaviors/KeyAsModifierInputBehavior.h"

#include "Mechanics/RectangleMarqueeMechanic.h"
#include "Mechanics/DragAlignmentMechanic.h"

using namespace UE::Geometry;

void UMeshTerrainModeSelectionInteraction::Initialize(
	TObjectPtr<UGeometrySelectionManager> SelectionManagerIn,
	TUniqueFunction<bool()> CanChangeSelectionCallbackIn,
	TUniqueFunction<bool(const FInputDeviceRay&)> ExternalHitCaptureCallbackIn)
{
	SelectionManager = SelectionManagerIn;

	CanChangeSelectionCallback = MoveTemp(CanChangeSelectionCallbackIn);
	ExternalHitCaptureCallback = MoveTemp(ExternalHitCaptureCallbackIn);

	// set up rectangle marquee interaction
	RectangleMarqueeInteraction = NewObject<URectangleMarqueeInteraction>(this);
	RectangleMarqueeInteraction->OnDragRectangleFinished.AddUObject(this, &UMeshTerrainModeSelectionInteraction::OnMarqueeRectangleFinished);

	// set up path selection interaction
	PathSelectionInteraction = NewObject<UMeshTerrainModePathSelectionInteraction>(this);
	PathSelectionInteraction->Setup(this);

	// create click behavior and set ourselves as click target
	ClickOrDragBehavior = NewObject<USingleClickOrDragInputBehavior>();
	ClickOrDragBehavior->Modifiers.RegisterModifier(AddToSelectionModifier, FInputDeviceState::IsShiftKeyDown);
	ClickOrDragBehavior->Modifiers.RegisterModifier(ToggleSelectionModifier, FInputDeviceState::IsCtrlKeyDown);
	ClickOrDragBehavior->bBeginDragIfClickTargetNotHit = false;
	ClickOrDragBehavior->Initialize(this, PathSelectionInteraction);

	HoverBehavior = NewObject<UMouseHoverBehavior>(this);
	HoverBehavior->Initialize(this);

	BehaviorSet = NewObject<UInputBehaviorSet>();
	BehaviorSet->Add(ClickOrDragBehavior, this);
	BehaviorSet->Add(HoverBehavior, this);

	// configure drag mode of ClickOrDragBehavior
	UpdateActiveDragMode();

	TransformProxy = NewObject<UTransformProxy>(this);
	TransformGizmo = UE::TransformGizmoUtil::CreateCustomRepositionableTransformGizmo( 
		SelectionManager->GetToolsContext()->GizmoManager, ETransformGizmoSubElements::FullTranslateRotateScale,
		this, TEXT("ModelingSelectionInteraction") );
	TransformGizmo->SetIsNonUniformScaleAllowedFunction([]() {return true;});
	TransformGizmo->SetActiveTarget(TransformProxy, SelectionManager->GetToolsContext()->GizmoManager);
	TransformGizmo->SetVisibility(false);

	DragAlignmentInteraction = NewObject<UDragAlignmentInteraction>(this);
	DragAlignmentInteraction->Setup( USceneSnappingManager::Find(SelectionManager->GetToolsContext()->GizmoManager) );

	DragAlignmentToggleBehavior = NewObject<UKeyAsModifierInputBehavior>(this);
	DragAlignmentInteraction->RegisterAsBehaviorTarget(DragAlignmentToggleBehavior);
	BehaviorSet->Add(DragAlignmentToggleBehavior, this);
	DragAlignmentInteraction->AddToGizmo(TransformGizmo);

	// listen for change events on transform proxy
	TransformProxy->OnTransformChanged.AddUObject(this, &UMeshTerrainModeSelectionInteraction::OnGizmoTransformChanged);
	TransformProxy->OnBeginTransformEdit.AddUObject(this, &UMeshTerrainModeSelectionInteraction::OnBeginGizmoTransform);
	TransformProxy->OnEndTransformEdit.AddUObject(this, &UMeshTerrainModeSelectionInteraction::OnEndGizmoTransform);

	// listen for selection changes to update gizmo
	OnSelectionModifiedDelegate = SelectionManager->OnSelectionModified.AddUObject(this, &UMeshTerrainModeSelectionInteraction::OnSelectionManager_SelectionModified);
}

void UMeshTerrainModeSelectionInteraction::Shutdown()
{
	if (IsValid(SelectionManager))
	{
		SelectionManager->OnSelectionModified.Remove(OnSelectionModifiedDelegate);
	}

	if (TransformGizmo)
	{
		SelectionManager->GetToolsContext()->GizmoManager->DestroyGizmo(TransformGizmo);
		TransformGizmo = nullptr;
	}
}

void UMeshTerrainModeSelectionInteraction::Render(IToolsContextRenderAPI* RenderAPI)
{
	if (DragAlignmentInteraction)
	{
		DragAlignmentInteraction->Render(RenderAPI);
	}
}

void UMeshTerrainModeSelectionInteraction::DrawHUD(FCanvas* Canvas, IToolsContextRenderAPI* RenderAPI)
{
	if (RectangleMarqueeInteraction)
	{
		RectangleMarqueeInteraction->DrawHUD(Canvas, RenderAPI);
	}
}

void UMeshTerrainModeSelectionInteraction::SetActiveDragMode(EMeshTerrainModeSelectionInteraction_DragMode NewMode)
{
	if (ActiveDragMode != NewMode)
	{
		ActiveDragMode = NewMode;
		UpdateActiveDragMode();
	}
}

void UMeshTerrainModeSelectionInteraction::SetLocalFrameMode(EMeshTerrainModeSelectionInteraction_LocalFrameMode NewLocalFrameMode)
{
	if (LocalFrameMode != NewLocalFrameMode)
	{
		LocalFrameMode = NewLocalFrameMode;
		UpdateGizmoOnSelectionChange();
	}
}

void UMeshTerrainModeSelectionInteraction::UpdateActiveDragMode()
{
	if (ActiveDragMode == EMeshTerrainModeSelectionInteraction_DragMode::PathInteraction)
	{
		ClickOrDragBehavior->SetDragTarget(PathSelectionInteraction);
	}
	else if (ActiveDragMode == EMeshTerrainModeSelectionInteraction_DragMode::RectangleMarqueeInteraction)
	{
		ClickOrDragBehavior->SetDragTarget(RectangleMarqueeInteraction);
	}
	else
	{
		ClickOrDragBehavior->SetDragTarget(nullptr);
	}
}


void UMeshTerrainModeSelectionInteraction::OnUpdateModifierState(int ModifierID, bool bIsOn)
{
	// update modifier state flags
	if (ModifierID == AddToSelectionModifier)
	{
		bAddToSelectionEnabled = bIsOn;
	}
	else if (ModifierID == ToggleSelectionModifier)
	{
		bToggleSelectionEnabled = bIsOn;
	}
}


void UMeshTerrainModeSelectionInteraction::ComputeSceneHits(const FInputDeviceRay& ClickPos,
	bool& bHitActiveObjects, FInputRayHit& ActiveObjectHit,
	bool& bHitInactiveObjectFirst, FInputRayHit& InactiveObjectHit)
{
	ActiveObjectHit = FInputRayHit();
	bHitActiveObjects = SelectionManager->RayHitTest(ClickPos.WorldRay, ActiveObjectHit);

	// We want to filter out hits against nearer objects. This is...tricky.
	// TODO: we probably should not actually do this here. Modeling Mode probably needs to 
	// have a behavior that mimics editor selection, that would have just received the closer
	// FInputRayHit and taken the click, instead of this function...
	bHitInactiveObjectFirst = false;
	InactiveObjectHit = FInputRayHit();
	if (bHitActiveObjects)
	{
		TArray<const UPrimitiveComponent*> IgnoreComponents;
		if (ActiveObjectHit.HitObject.IsValid())
		{
			if (UPrimitiveComponent* SelectionComponent = Cast<UPrimitiveComponent>(ActiveObjectHit.HitObject.Get()) )
			{
				IgnoreComponents.Add(SelectionComponent);
			}
		}
		FHitResult HitResultOut;
		if (ToolSceneQueriesUtil::FindNearestVisibleObjectHit(USceneSnappingManager::Find(SelectionManager->GetToolsContext()->ToolManager), 
			HitResultOut, ClickPos.WorldRay, &IgnoreComponents, nullptr))
		{
			if (HitResultOut.Distance < ActiveObjectHit.HitDepth)
			{
				bHitInactiveObjectFirst = true;
				InactiveObjectHit.HitDepth = HitResultOut.Distance;
				InactiveObjectHit.bHit = true;
			}
		}
	}
}


FInputRayHit UMeshTerrainModeSelectionInteraction::IsHitByClick(const FInputDeviceRay& ClickPos)
{
	// ignore hits in these cases
	if (  (SelectionManager->GetMeshTopologyMode() == UGeometrySelectionManager::EMeshTopologyMode::None) 
		|| (CanChangeSelectionCallback() == false)
		|| ExternalHitCaptureCallback(ClickPos) )
	{
		return FInputRayHit();
	}

	bool bHitActiveObjects, bHitInactiveObjectFirst;
	FInputRayHit ActiveObjectHit, InactiveObjectHit;
	ComputeSceneHits(ClickPos, bHitActiveObjects, ActiveObjectHit, bHitInactiveObjectFirst, InactiveObjectHit);

	// Note: currently this flow will produce somewhat undesirable behavior, in that if 
	// some other object not in the selection target(s) is clicked, the selection will be 
	// cleared but that object will have to be clicked again to select it. We cannot currently 
	// clear the mesh-selection and change the actor-selection inside a single transaction in a
	// single click. This is because we are relying on the Editor to do the actor-selection change.
	// So if we were to (eg) ClearSelection() and return no-hit here, the ClearSelection() call
	// would emit a transaction, and then the actor-selection-change would emit a second transaction.
	// 
	// Behavior in many DCCS is that they can (1) clear active mesh-selection, (2) select new target
	// object, and (3) do new mesh-selection in a single click. This will require quite a bit of effort
	// to achieve in the future. 
	//
	// So for now we set a bClearSelectionOnClicked flag and consume the click. This means the 1-2-3
	// flow above requires 3 separate clicks :( 


	bClearSelectionOnClicked = false;
	if (SelectionManager->HasSelection())
	{
		if (bHitInactiveObjectFirst)
		{
			//bClearSelectionOnClicked = true;
			//return InactiveObjectHit;
			return FInputRayHit();
		}
		else if (bHitActiveObjects)
		{
			return ActiveObjectHit;
		}
		else
		{
			// if we have active selection we still want to capture this hit so that we can clear in OnClicked
			//bClearSelectionOnClicked = true;
			//return FInputRayHit(TNumericLimits<double>::Max());
			return FInputRayHit();
		}
	}
	else
	{
		if (bHitActiveObjects && bHitInactiveObjectFirst == false)
		{
			return ActiveObjectHit;
		}
	}

	return FInputRayHit();
}



void UMeshTerrainModeSelectionInteraction::OnClicked(const FInputDeviceRay& ClickPos)
{
	// this flag is set in IsHitByClick test
	if (bClearSelectionOnClicked)
	{
		SelectionManager->ClearSelection();
		bClearSelectionOnClicked = false;
		return;
	}

	FGeometrySelectionUpdateConfig UpdateConfig = GetActiveSelectionUpdateConfig();

	FGeometrySelectionUpdateResult Result;
	SelectionManager->UpdateSelectionViaRaycast(
		ClickPos.WorldRay,
		UpdateConfig,
		Result);
}


FGeometrySelectionUpdateConfig UMeshTerrainModeSelectionInteraction::GetActiveSelectionUpdateConfig() const
{
	FGeometrySelectionUpdateConfig UpdateConfig;
	UpdateConfig.ChangeType = EGeometrySelectionChangeType::Replace;
	if (bAddToSelectionEnabled)
	{
		UpdateConfig.ChangeType = EGeometrySelectionChangeType::Add;
	}
	else if (bToggleSelectionEnabled)
	{
		UpdateConfig.ChangeType = EGeometrySelectionChangeType::Remove;
	}
	return UpdateConfig;
}



FInputRayHit UMeshTerrainModeSelectionInteraction::BeginHoverSequenceHitTest(const FInputDeviceRay& PressPos)
{
	if ((SelectionManager->GetMeshTopologyMode() == UGeometrySelectionManager::EMeshTopologyMode::None)
		|| (CanChangeSelectionCallback() == false)
		|| ExternalHitCaptureCallback(PressPos))
	{
		return FInputRayHit();
	}

	bool bHitActiveObjects, bHitInactiveObjectFirst;
	FInputRayHit ActiveObjectHit, InactiveObjectHit;
	ComputeSceneHits(PressPos, bHitActiveObjects, ActiveObjectHit, bHitInactiveObjectFirst, InactiveObjectHit);

	if (bHitActiveObjects && bHitInactiveObjectFirst == false)
	{
		return ActiveObjectHit;
	}
	return FInputRayHit();
}

void UMeshTerrainModeSelectionInteraction::OnBeginHover(const FInputDeviceRay& DevicePos)
{
	SelectionManager->UpdateSelectionPreviewViaRaycast(DevicePos.WorldRay);
}

bool UMeshTerrainModeSelectionInteraction::OnUpdateHover(const FInputDeviceRay& DevicePos)
{
	return SelectionManager->UpdateSelectionPreviewViaRaycast(DevicePos.WorldRay);
}

void UMeshTerrainModeSelectionInteraction::OnEndHover()
{
	SelectionManager->ClearSelectionPreview();
}




void UMeshTerrainModeSelectionInteraction::UpdateGizmoOnSelectionChange()
{
	if (SelectionManager->HasSelection() == false)
	{
		TransformGizmo->SetVisibility(false);
	}
	else
	{
		TransformGizmo->SetVisibility(true);

		FFrame3d SelectionFrame;
		if (LocalFrameMode == EMeshTerrainModeSelectionInteraction_LocalFrameMode::FromGeometry)
		{
			SelectionManager->GetSelectionWorldFrame(SelectionFrame);
		}
		else
		{
			SelectionManager->GetTargetWorldFrame(SelectionFrame);
		}
		TransformGizmo->ReinitializeGizmoTransform( SelectionFrame.ToFTransform() );
	}
}


void UMeshTerrainModeSelectionInteraction::OnSelectionManager_SelectionModified()
{
	UpdateGizmoOnSelectionChange();
}



void UMeshTerrainModeSelectionInteraction::OnBeginGizmoTransform(UTransformProxy* Proxy)
{
	FTransform Transform = Proxy->GetTransform();
	InitialGizmoFrame = FFrame3d(Transform);
	InitialGizmoScale = FVector3d(Transform.GetScale3D());

	LastTranslationDelta = FVector3d::Zero();
	LastRotateDelta = FVector4d::Zero();
	LastScaleDelta = FVector3d::Zero();

	bInActiveTransform = SelectionManager->BeginTransformation();

	OnTransformBegin.Broadcast();
}

void UMeshTerrainModeSelectionInteraction::OnEndGizmoTransform(UTransformProxy* Proxy)
{
	if (bInActiveTransform)
	{
		// If we have an update we haven't applied, apply it now. This is particularly important if the
		// user transformed the gizmo by typing a new value into the gizmo UI, in which case there will
		// not have been a chance to do ApplyPendingTransformInteractions() between the update and end calls.
		if (bGizmoUpdatePending)
		{
			ApplyPendingTransformInteractions();
		}

		// Clear the active-transform flag before calling EndTransformation. EndTransformation can synchronously
		// re-enter this handler if the underlying commit (e.g. UStaticMesh::PostEditChange) pumps messages and
		// the resulting focus loss is routed through UInputRouter::ForceTerminateAll back through the gizmo.
		// Clearing the flag first ensures the reentrant call does not double-broadcast OnTransformEnd.
		//
		// Note: this must come after ApplyPendingTransformInteractions, which guards its own work on bInActiveTransform being true.
		bInActiveTransform = false;

		SelectionManager->EndTransformation();

		// reset the transient scaling being stored by the gizmo, otherwise it will be re-applied in the next transformation
		TransformGizmo->SetNewChildScale(FVector::One());

		OnTransformEnd.Broadcast();
	}
}

void UMeshTerrainModeSelectionInteraction::OnGizmoTransformChanged(UTransformProxy* Proxy, FTransform Transform)
{
	if (bInActiveTransform)
	{
		LastUpdateGizmoFrame = FFrame3d(Transform);
		LastUpdateGizmoScale = FVector3d(Transform.GetScale3D());
		bGizmoUpdatePending = true;
	}
}


void UMeshTerrainModeSelectionInteraction::ApplyPendingTransformInteractions()
{
	if (bInActiveTransform == false || bGizmoUpdatePending == false)
	{
		return;
	}

	FFrame3d CurFrame = LastUpdateGizmoFrame;
	FVector3d CurScale = LastUpdateGizmoScale;
	FVector3d TranslationDelta = CurFrame.Origin - InitialGizmoFrame.Origin;
	FQuaterniond RotateDelta = CurFrame.Rotation - InitialGizmoFrame.Rotation;
	FVector3d CurScaleDelta = CurScale - InitialGizmoScale;

	bool bTransformInWorldFrame = TransformGizmo->CurrentCoordinateSystem == EToolContextCoordinateSystem::World;

	// if any of the deltas have deviated from the last delta, forward the transformation change on to targets
	if ( (TranslationDelta - LastTranslationDelta).SizeSquared() > FMathf::ZeroTolerance
		|| ((FVector4d)RotateDelta - LastRotateDelta).SizeSquared() > FMathf::ZeroTolerance
		|| (CurScaleDelta- LastScaleDelta).SizeSquared() > FMathf::ZeroTolerance)
	{
		LastTranslationDelta = TranslationDelta;
		LastRotateDelta = (FVector4d)RotateDelta;
		LastScaleDelta = CurScaleDelta;

		FQuat RotationDelta = FQuat(CurFrame.Rotation * InitialGizmoFrame.Rotation.Inverse());

		SelectionManager->UpdateTransformation(InitialGizmoFrame.ToFTransform(), TranslationDelta, RotationDelta, CurScaleDelta, bTransformInWorldFrame);
	}

	bGizmoUpdatePending = false;
}



void UMeshTerrainModeSelectionInteraction::OnMarqueeRectangleFinished(const FCameraRectangle& Rectangle, bool bCancelled)
{

}




void UMeshTerrainModePathSelectionInteraction::Setup(UMeshTerrainModeSelectionInteraction* SelectionInteractionIn)
{

	SelectionInteraction = SelectionInteractionIn;
}


FInputRayHit UMeshTerrainModePathSelectionInteraction::CanBeginClickDragSequence(const FInputDeviceRay& PressPos)
{
	if (SelectionInteraction->GetSelectionManager()->CanBeginTrackedSelectionChange())
	{
		return SelectionInteraction->IsHitByClick(PressPos);
	}
	return FInputRayHit();
}

void UMeshTerrainModePathSelectionInteraction::OnClickPress(const FInputDeviceRay& PressPos)
{
	FGeometrySelectionUpdateConfig UpdateConfig = SelectionInteraction->GetActiveSelectionUpdateConfig();
	UGeometrySelectionManager* SelectionManager = SelectionInteraction->GetSelectionManager();

	bool bInitialClear = false;
	if (UpdateConfig.ChangeType == EGeometrySelectionChangeType::Replace)
	{
		bInitialClear = true;
		UpdateConfig.ChangeType = EGeometrySelectionChangeType::Add;
	}

	if (SelectionManager->BeginTrackedSelectionChange(UpdateConfig, bInitialClear))
	{
		FGeometrySelectionUpdateResult Result;
		SelectionManager->AccumulateSelectionUpdate_Raycast(PressPos.WorldRay, Result);
	}
}

void UMeshTerrainModePathSelectionInteraction::OnClickDrag(const FInputDeviceRay& DragPos)
{
	UGeometrySelectionManager* SelectionManager = SelectionInteraction->GetSelectionManager();
	if (SelectionManager->IsInTrackedSelectionChange())
	{
		FGeometrySelectionUpdateResult Result;
		SelectionManager->AccumulateSelectionUpdate_Raycast(DragPos.WorldRay, Result);		
	}
}

void UMeshTerrainModePathSelectionInteraction::OnClickRelease(const FInputDeviceRay& ReleasePos)
{
	UGeometrySelectionManager* SelectionManager = SelectionInteraction->GetSelectionManager();
	if (SelectionManager->IsInTrackedSelectionChange())
	{
		SelectionManager->EndTrackedSelectionChange();
	}
}

void UMeshTerrainModePathSelectionInteraction::OnTerminateDragSequence()
{
	UGeometrySelectionManager* SelectionManager = SelectionInteraction->GetSelectionManager();
	if (SelectionManager->IsInTrackedSelectionChange())
	{
		SelectionManager->EndTrackedSelectionChange();
	}
}
