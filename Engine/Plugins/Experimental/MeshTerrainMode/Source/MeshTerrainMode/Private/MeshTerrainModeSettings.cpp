// Copyright Epic Games, Inc. All Rights Reserved.

#include "MeshTerrainModeSettings.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(MeshTerrainModeSettings)


#define LOCTEXT_NAMESPACE "MeshTerrainModeSettings"


FText UMeshTerrainModeSettings::GetSectionText() const 
{ 
	return LOCTEXT("MeshTerrainModeSettingsName", "Mesh Terrain Mode"); 
}

FText UMeshTerrainModeSettings::GetSectionDescription() const
{
	return LOCTEXT("MeshTerrainModeSettingsDescription", "Configure the Mesh Terrain Editor Mode plugin");
}


FText UMeshTerrainModeCustomizationSettings::GetSectionText() const 
{ 
	return LOCTEXT("MeshTerrainModeSettingsName", "Mesh Terrain Mode"); 
}

FText UMeshTerrainModeCustomizationSettings::GetSectionDescription() const
{
	return LOCTEXT("MeshTerrainModeSettingsDescription", "Configure the Mesh Terrain Editor Mode plugin");
}


#undef LOCTEXT_NAMESPACE
