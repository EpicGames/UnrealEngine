// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Engine/DeveloperSettings.h"
#include "Overlay/DragBoxPosition.h"
#include "MeshTerrainModeSettings.generated.h"

#define UE_API MESHTERRAINMODE_API

struct FCollectionReference;



UENUM()
enum class EMeshTerrainModeDefaultMeshObjectType
{
	/** Generate a new Static Mesh Asset (using Generated Asset settings below) and AStaticMeshActor */
	StaticMeshAsset,
	/** Generate a new AVolume Actor */
	VolumeActor,
	/** Generate a new ADynamicMeshActor (stored locally in the Level) */
	DynamicMeshActor
};



UENUM()
enum class EMeshTerrainModeAssetGenerationBehavior
{
	/** Generate and automatically Save new Assets on creation */
	AutoGenerateAndAutosave,

	/** Generate new Assets and mark as Modified but do not automatically Save */
	AutoGenerateButDoNotAutosave,

	/** Prompt to Save each new Asset upon Creation */
	InteractivePromptToSave
};



UENUM()
enum class EMeshTerrainModeAssetGenerationLocation
{
	/** All generated assets will be stored in an AutoGenerated folder that is located relative to the World they are being saved in */
	AutoGeneratedWorldRelativeAssetPath,

	/** All generated assets will be stored in a root-level AutoGenerated folder */
	AutoGeneratedGlobalAssetPath,

	/** Generated assets will be stored in the currently-visible Asset Browser folder if available, otherwise at the Auto Generated Asset Path */
	CurrentAssetBrowserPathIfAvailable
};


/**
 * Settings for the Mesh Terrain Mode plug-in.
 */
UCLASS(MinimalAPI, config=Editor)
class UMeshTerrainModeSettings : public UDeveloperSettings
{
	GENERATED_BODY()

public:

	// UDeveloperSettings overrides

	virtual FName GetContainerName() const override { return FName("Project"); }
	virtual FName GetCategoryName() const override { return FName("Plugins"); }
	virtual FName GetSectionName() const override { return FName("MeshTerrainMode"); }

	UE_API virtual FText GetSectionText() const override;
	UE_API virtual FText GetSectionDescription() const override;

protected:

	/** Enable/Disable the options to emit Dynamic Mesh Actors in Mesh Terrain Mode */
	UPROPERTY()
	bool bEnableDynamicMeshActors = false;

	/** Where should Assets auto-generated by Mesh Terrain be stored by default */
	UPROPERTY(config, EditAnywhere, Category = "Mesh Terrain Mode|Generated Assets")
	EMeshTerrainModeAssetGenerationLocation AssetGenerationLocation = EMeshTerrainModeAssetGenerationLocation::AutoGeneratedWorldRelativeAssetPath;

	/** How should Assets auto-generated by Mesh Terrain be handled in terms of saving, naming, etc */
	UPROPERTY(config, EditAnywhere, Category = "Mesh Terrain Mode|Generated Assets")
	EMeshTerrainModeAssetGenerationBehavior AssetGenerationMode = EMeshTerrainModeAssetGenerationBehavior::AutoGenerateButDoNotAutosave;

public:

	EMeshTerrainModeAssetGenerationLocation GetAssetGenerationLocation() const
	{
		return bRestrictiveMode
				   ? EMeshTerrainModeAssetGenerationLocation::AutoGeneratedGlobalAssetPath
				   : AssetGenerationLocation;
	}

	void SetAssetGenerationLocation(const EMeshTerrainModeAssetGenerationLocation Location)
	{
		AssetGenerationLocation = Location;
	}

	EMeshTerrainModeAssetGenerationBehavior GetAssetGenerationMode() const
	{
		return bRestrictiveMode
					? EMeshTerrainModeAssetGenerationBehavior::AutoGenerateButDoNotAutosave
					: AssetGenerationMode;
	}

	void SetAssetGenerationMode(const EMeshTerrainModeAssetGenerationBehavior Mode)
	{
		AssetGenerationMode = Mode;
	}

	/** What type of Mesh Object should Output Type Setting default to in Mesh Terrain Mode (takes effect after Editor restart) */
	UPROPERTY(config, EditAnywhere, Category = "Mesh Terrain Mode")
	EMeshTerrainModeDefaultMeshObjectType DefaultMeshObjectType = EMeshTerrainModeDefaultMeshObjectType::StaticMeshAsset;

	/** Assets auto-generated by Mesh Terrain are stored at this path, relative to the parent path defined by the Location. Set to an empty string to disable. */
	UPROPERTY(config, EditAnywhere, Category = "Mesh Terrain Mode|Generated Assets")
	FString AutoGeneratedAssetPath = TEXT("_GENERATED");

	/** If true, Auto-Generated Assets created in an unsaved Level will be stored relative to top-level folder, otherwise they will be stored in /Temp and cannot be saved until they are explicitly moved to a permanent location */
	UPROPERTY(config, EditAnywhere, Category = "Mesh Terrain Mode|Generated Assets")
	bool bStoreUnsavedLevelAssetsInTopLevelGameFolder = true;

	/** If true, Autogenerated Assets are stored in per-user folders below the Autogen path */
	UPROPERTY(config, EditAnywhere, Category = "Mesh Terrain Mode|Generated Assets")
	bool bUsePerUserAutogenSubfolder = true;

	/** Overrides the user name used for per-user folders below the Autogen path. This might be necessary to resolve issues with source control, for example. Note that the per-user folder name might not contain the name exactly as provided. */
	UPROPERTY(config, EditAnywhere, Category = "Mesh Terrain Mode|Generated Assets")
	FString AutogenSubfolderUserNameOverride;

	/** If true, Autogenerated Assets have a short random string generated and appended to their name */
	UPROPERTY(config, EditAnywhere, Category = "Mesh Terrain Mode|Generated Assets")
	bool bAppendRandomStringToName = true;

	//
	// User Interface
	//

	/** If true, the standard UE Editor Gizmo Mode (ie selected via the Level Editor Viewport toggle) will be used to configure the Gizmo, otherwise a Combined Gizmo will always be used. It may be necessary to exit and re-enter Mesh Terrain Mode after changing this setting. */
	UPROPERTY(config, EditAnywhere, Category = "Mesh Terrain Mode|User Interface")
	bool bRespectLevelEditorGizmoMode = false;

	//
	// Selection
	//

	/** Enable/Disable the Mesh Selection System. */
	UPROPERTY(config, EditAnywhere, Category = "Mesh Terrain Mode|Selection", meta = (DisplayName="Enable Mesh Selection UI"))
	bool bEnableMeshSelections = false;

	virtual bool GetMeshSelectionsEnabled() const
	{
		return bEnableMeshSelections;
	}
	
	virtual void SetMeshSelectionsEnabled(const bool bEnabled)
	{
		bEnableMeshSelections = bEnabled;
	}


	DECLARE_MULTICAST_DELEGATE_TwoParams(UMeshTerrainModeSettingsModified, UObject*, FProperty*);
	UMeshTerrainModeSettingsModified OnModified;

	virtual void PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent) override
	{
		OnModified.Broadcast(this, PropertyChangedEvent.Property);

		Super::PostEditChangeProperty(PropertyChangedEvent);
	}

	//
	// Restrictive Mode
	//

	virtual bool InRestrictiveMode() const
	{
		return bRestrictiveMode;
	}

	virtual void SetRestrictiveMode(bool bEnabled)
	{
		bRestrictiveMode = bEnabled;
	}

	virtual FString GetRestrictiveModeAutoGeneratedAssetPath() const
	{
		return bRestrictiveMode ? RestrictiveModeAutoGeneratedAssetPath : FString();
	}

	virtual bool SetRestrictiveModeAutoGeneratedAssetPath(const FString& AssetPath)
	{
		if (bRestrictiveMode)
		{
			RestrictiveModeAutoGeneratedAssetPath = AssetPath;
			return true;
		}
		return false;
	}

protected:

	/** Assets auto-generated by Mesh Terrain are stored at this path when in restrictive mode, relative to the package folder path. */
	UPROPERTY(Config)
	FString RestrictiveModeAutoGeneratedAssetPath = TEXT("Meshes");

	bool bRestrictiveMode = false;




public:
	//
	// Settings that are currently stored during a single session but are not stored in the config file
	// (may promote them to persistent settings in the future)
	//

	/** Toggle Absolute World Grid Position snapping */
	UPROPERTY(Transient)
	bool bEnableAbsoluteWorldSnapping = false;
};




/**
 * Defines a color to be used for a particular Tool Palette Section
 */
USTRUCT()
struct FMeshTerrainModeCustomSectionColor
{
	GENERATED_BODY()

	/** Name of Section in Mesh Terrain Mode Palette */
	UPROPERTY(EditAnywhere, Category = "SectionColor")
	FString SectionName = TEXT("");

	/** Custom Header Color */
	UPROPERTY(EditAnywhere, Category = "SectionColor")
	FLinearColor Color = FLinearColor::Gray;
};


/**
 * Defines a color to be used for a particular Tool Palette Tool
 */
USTRUCT()
struct FMeshTerrainModeCustomToolColor
{
	GENERATED_BODY()

	/**
	 * Name of Section or Tool in Mesh Terrain Mode Palette
	 *
	 * Format:
	 * SectionName        (Specifies a default color for all tools in the section.)
	 * SectionName.ToolName        (Specifies an override color for a specific tool in the given section.)
	 */
	UPROPERTY(EditAnywhere, Category = "ToolColor")
	FString ToolName = TEXT("");

	/** Custom Tool Color */
	UPROPERTY(EditAnywhere, Category = "ToolColor")
	FLinearColor Color = FLinearColor::Gray;
};


/**
 * Defines a Named list/set of content-browser Collection names
 */
USTRUCT()
struct FMeshTerrainModeAssetCollectionSet
{
	GENERATED_BODY()

	/** Name of the set of collections */
	UPROPERTY(EditAnywhere, Category = "CollectionSet")
	FString Name;

	/** List of Collection names */
	UPROPERTY(EditAnywhere, Category = "CollectionSet")
	TArray<FCollectionReference> Collections;
};


UCLASS(config=Editor)
class UMeshTerrainModeCustomizationSettings : public UDeveloperSettings
{
	GENERATED_BODY()

public:
	// UDeveloperSettings overrides

	virtual FName GetContainerName() const { return FName("Editor"); }
	virtual FName GetCategoryName() const { return FName("Plugins"); }
	virtual FName GetSectionName() const { return FName("MeshTerrainMode"); }

	virtual FText GetSectionText() const override;
	virtual FText GetSectionDescription() const override;

public:
	/**
	 * A Brush Alpha Set is a named list of Content Browser Collections, which will be shown as a separate tab in 
	 * the Texture Asset Picker used in various Mesh Terrain Mode when selecting a Brush Alpha (eg in Sculpting)
	 */
	UPROPERTY(config, EditAnywhere, Category = "Mesh Terrain Mode|Tool Assets")
	TArray<FMeshTerrainModeAssetCollectionSet> BrushAlphaSets;


	/**
	 * If true, the category labels will be shown on the toolbar buttons, else they will be hidden
	 */
	UPROPERTY(config, EditAnywhere, Category = "Mesh Terrain Mode|UI Customization")
	bool bShowCategoryButtonLabels = true;
	
	/**
	 * If true, Tool buttons will always be shown when in a Tool. By default they will be hidden.
	 */
	UPROPERTY(config, EditAnywhere, Category = "Mesh Terrain Mode|UI Customization")
	bool bAlwaysShowToolButtons = false;

	/** If true, pressing Escape will Accept the active tool result instead of Cancelling it. Defaults to off. */
	UPROPERTY(config, EditAnywhere, Category = "Mesh Terrain Mode|UI Customization")
	bool bEscapeAcceptsToolResult = false;
	
	/** If true, exiting Mesh Terrain Mode will Accept the active tool result instead of Cancelling it. Defaults to on. */
	UPROPERTY(config, EditAnywhere, Category = "Mesh Terrain Mode|UI Customization")
	bool bExitModeAcceptsToolResult = true;

	/**
	 * Custom color for when geometry is not a part of current selection when using Mesh Element Selection
	 */
	UPROPERTY(Config, EditAnywhere, Category = "Mesh Terrain Mode|Mesh Element Selection Mode")
	FLinearColor UnselectedColor = FLinearColor(.068478f, 0.56478f, 0.708376f,.5f); // default light blue

	/**
	 * Custom color hovering over selected geometry in Mesh Element Selection
	 */
	UPROPERTY(Config, EditAnywhere, Category = "Mesh Terrain Mode|Mesh Element Selection Mode")
	FLinearColor HoverOverSelectedColor = FLinearColor(1.0f, 0.943014f, 0.74f); // default white

	/**
	* Custom color hovering over unselected geometry in Mesh Element Selection
	*/
	UPROPERTY(Config, EditAnywhere, Category = "Mesh Terrain Mode|Mesh Element Selection Mode")
	FLinearColor HoverOverUnselectedColor = FLinearColor(1.0f, 0.25f, 0.712328f, .5f); // default pink

	/**
	* Custom color for when geometry is a part of the current selection when using Mesh Element Selection
	*/
	UPROPERTY(Config, EditAnywhere, Category = "Mesh Terrain Mode|Mesh Element Selection Mode")
	FLinearColor GeometrySelectedColor = FLinearColor(0.982251f, 0.672443f, 0.029557f, .5f); // default yellow

	/**
	* Custom color for when geometry is a part of the soft selection when using Mesh Element Selection
	*/
	UPROPERTY(Config, EditAnywhere, Category = "Mesh Terrain Mode|Mesh Element Selection Mode")
	FLinearColor GeometrySoftSelectedColor = FLinearColor(0.063f, 0.161f, 0.71f, .5f); // default blue
	
	/**
	 * Saved size for the Submode Tool Panel
	 */
	UPROPERTY(config, EditAnywhere, Category = "Mesh Terrain Mode|UI Customization")
	FVector2f SubmodeToolPanelSize = FVector2f::ZeroVector;

	/**
	 * Saved position for the viewport Tool Details panel
	 */
	UPROPERTY(config, EditAnywhere, Category = "Mesh Terrain Mode|UI Customization")
	FToolWidget_DragBoxPosition DetailsOverlayPosition;
	
	/**
	 * Saved size for the viewport Tool Details panel
	 */
	UPROPERTY(Config, EditAnywhere, Category = "Mesh Terrain Mode|UI Customization")
	FVector2f DetailsOverlaySize = FVector2f::ZeroVector;

	/**
	 * Saved position for the viewport Quick Settings bar
	 */
	UPROPERTY(config, EditAnywhere, Category = "Mesh Terrain Mode|UI Customization")
	FToolWidget_DragBoxPosition QuickSettingsOverlayPosition;

public:

	// saved-state for various mode settings that are configured via UI toggles/etc, and not exposed in settings dialog

	UPROPERTY(config)
	int32 LastMeshSelectionDragMode = 0;

	UPROPERTY(config)
	int32 LastMeshSelectionLocalFrameMode = 0;

public:

	// saved-state for various mode settings that does not persist between editor runs

	UPROPERTY()
	int32 LastMeshSelectionElementType = 0;

	UPROPERTY()
	int32 LastMeshSelectionTopologyMode = 0;

	UPROPERTY()
	bool bLastMeshSelectionVolumeToggle = true;

	UPROPERTY()
	bool bLastMeshSelectionStaticMeshToggle = true;


};

#undef UE_API
