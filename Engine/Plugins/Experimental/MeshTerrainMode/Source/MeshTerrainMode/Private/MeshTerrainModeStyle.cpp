// Copyright Epic Games, Inc. All Rights Reserved.

#include "MeshTerrainModeStyle.h"
#include "Brushes/SlateImageBrush.h"
#include "Styling/SlateStyleRegistry.h"
#include "Misc/Paths.h"
#include "Styling/CoreStyle.h"
#include "Interfaces/IPluginManager.h"
#include "SlateOptMacros.h"
#include "Styling/SlateStyle.h"
#include "Styling/SlateStyleMacros.h"
#include "Styling/ToolBarStyle.h"
#include "Styling/StarshipCoreStyle.h"
#include "Styling/StyleColors.h"
#include "Util/ColorConstants.h"

#define IMAGE_PLUGIN_BRUSH( RelativePath, ... ) FSlateImageBrush( FMeshTerrainModeStyle::InContent( RelativePath, ".png" ), __VA_ARGS__ )

// This is to fix the issue that SlateStyleMacros like IMAGE_BRUSH look for RootToContentDir but StyleSet->RootToContentDir is how this style is set up
#define RootToContentDir StyleSet->RootToContentDir


FString FMeshTerrainModeStyle::InContent(const FString& RelativePath, const ANSICHAR* Extension)
{
	static FString ContentDir = IPluginManager::Get().FindPlugin(TEXT("MeshTerrainMode"))->GetContentDir();
	return (ContentDir / RelativePath) + Extension;
}

TSharedPtr< FSlateStyleSet > FMeshTerrainModeStyle::StyleSet = nullptr;
TSharedPtr< class ISlateStyle > FMeshTerrainModeStyle::Get() { return StyleSet; }

FName FMeshTerrainModeStyle::GetStyleSetName()
{
	static FName MeshTerrainModeStyleName(TEXT("MeshTerrainModeStyle"));
	return MeshTerrainModeStyleName;
}

const FSlateBrush* FMeshTerrainModeStyle::GetBrush(FName PropertyName, const ANSICHAR* Specifier)
{
	return Get()->GetBrush(PropertyName, Specifier);
}


BEGIN_SLATE_FUNCTION_BUILD_OPTIMIZATION

void FMeshTerrainModeStyle::Initialize()
{
	using namespace UE::Geometry;
	
	// Const icon sizes
	const FVector2D Icon8x8(8.0f, 8.0f);
	const FVector2D Icon16x16(16.0f, 16.0f);
	const FVector2D Icon20x20(20.0f, 20.0f);
	const FVector2D Icon28x28(28.0f, 28.0f);
	const FVector2D Icon40x40(40.0f, 40.0f);
	const FVector2D Icon120(120.0f, 120.0f);

	// Only register once
	if (StyleSet.IsValid())
	{
		return;
	}

	StyleSet = MakeShareable(new FSlateStyleSet(GetStyleSetName()));

	// If we get asked for something that we don't set, we should default to editor style
	StyleSet->SetParentStyleName("EditorStyle");

	const FString PluginContentDir = FPaths::Combine(IPluginManager::Get().FindPlugin(TEXT("MeshTerrainMode"))->GetBaseDir(), TEXT("Content"));
	StyleSet->SetContentRoot(PluginContentDir);
	StyleSet->SetCoreContentRoot(FPaths::EngineContentDir() / TEXT("Slate"));
	
	const FTextBlockStyle& NormalText = FAppStyle::Get().GetWidgetStyle<FTextBlockStyle>("NormalText");

	// Shared editors
	//{
	//	StyleSet->Set("Paper2D.Common.ViewportZoomTextStyle", FTextBlockStyle(NormalText)
	//		.SetFont(DEFAULT_FONT("BoldCondensed", 16))
	//	);

	//	StyleSet->Set("Paper2D.Common.ViewportTitleTextStyle", FTextBlockStyle(NormalText)
	//		.SetFont(DEFAULT_FONT("Regular", 18))
	//		.SetColorAndOpacity(FLinearColor(1.0, 1.0f, 1.0f, 0.5f))
	//	);

	//	StyleSet->Set("Paper2D.Common.ViewportTitleBackground", new BOX_BRUSH("Old/Graph/GraphTitleBackground", FMargin(0)));
	//}

	// Tool Manager icons
	{
		// Accept/Cancel/Complete active tool

		StyleSet->Set("LevelEditor.MeshTerrainMode", new IMAGE_BRUSH_SVG("Starship/geometry", FVector2D(20.0f, 20.0f)));

		StyleSet->Set("MeshTerrainMode.DefaultSettings", new FSlateImageBrush(StyleSet->RootToCoreContentDir(TEXT("../Editor/Slate/Icons/GeneralTools/Settings_40x.png")), Icon20x20));
		StyleSet->Set("MeshTerrainMode.SubToolArrow", new FSlateImageBrush(StyleSet->RootToCoreContentDir(
			// This arrow is 25x13, but 20x13 looks better
			TEXT("../Editor/Slate/Persona/BlendSpace/arrow_right_12x.png")), FVector2D(20,13)));

		// NOTE:  Old-style, need to be replaced: 
		StyleSet->Set("MeshTerrainModeManagerCommands.CancelActiveTool", new IMAGE_PLUGIN_BRUSH("Icons/icon_ActiveTool_Cancel_40x", Icon20x20));
		StyleSet->Set("MeshTerrainModeManagerCommands.CancelActiveTool.Small", new IMAGE_PLUGIN_BRUSH("Icons/icon_ActiveTool_Cancel_40x", Icon20x20));
		StyleSet->Set("MeshTerrainModeManagerCommands.AcceptActiveTool", new IMAGE_PLUGIN_BRUSH("Icons/icon_ActiveTool_Accept_40x", Icon20x20));
		StyleSet->Set("MeshTerrainModeManagerCommands.AcceptActiveTool.Small", new IMAGE_PLUGIN_BRUSH("Icons/icon_ActiveTool_Accept_40x", Icon20x20));
		StyleSet->Set("MeshTerrainModeManagerCommands.CompleteActiveTool", new IMAGE_PLUGIN_BRUSH("Icons/icon_ActiveTool_Accept_40x", Icon20x20));
		StyleSet->Set("MeshTerrainModeManagerCommands.CompleteActiveTool.Small", new IMAGE_PLUGIN_BRUSH("Icons/icon_ActiveTool_Accept_40x", Icon20x20));

		StyleSet->Set("MeshTerrainModeManagerCommands.LoadFavoritesTools", new IMAGE_BRUSH_SVG( "Icons/LoadFavoritesTools", Icon20x20 ) );
		StyleSet->Set("MeshTerrainModeManagerCommands.LoadSelectionTools", new IMAGE_BRUSH_SVG("Icons/ModSelectionObject_16", Icon20x20 ) );
		StyleSet->Set("MeshTerrainModeManagerCommands.LoadShapesTools", new IMAGE_BRUSH_SVG( "Icons/LoadShapesTools", Icon20x20 ) );
		StyleSet->Set("MeshTerrainModeManagerCommands.LoadCreateTools", new IMAGE_BRUSH_SVG( "Icons/LoadCreateTools", Icon20x20 ) );
		StyleSet->Set("MeshTerrainModeManagerCommands.LoadPolyTools", new IMAGE_BRUSH_SVG( "Icons/LoadPolyTools", Icon20x20 ) );
		StyleSet->Set("MeshTerrainModeManagerCommands.LoadTriTools", new IMAGE_BRUSH_SVG( "Icons/LoadTriTools", Icon20x20 ) );
		StyleSet->Set("MeshTerrainModeManagerCommands.LoadDeformTools", new IMAGE_BRUSH_SVG( "Icons/LoadDeformTools", Icon20x20 ) );
		StyleSet->Set("MeshTerrainModeManagerCommands.LoadTransformTools", new IMAGE_BRUSH_SVG( "Icons/LoadTransformTools", Icon20x20 ) );
		StyleSet->Set("MeshTerrainModeManagerCommands.LoadMeshOpsTools", new IMAGE_BRUSH_SVG( "Icons/LoadMeshOpsTools", Icon20x20 ) );
		StyleSet->Set("MeshTerrainModeManagerCommands.LoadVoxOpsTools", new IMAGE_BRUSH_SVG( "Icons/LoadVoxOpsTools", Icon20x20 ) );
		StyleSet->Set("MeshTerrainModeManagerCommands.LoadAttributesTools", new IMAGE_BRUSH_SVG( "Icons/LoadAttributesTools", Icon20x20 ) );
		StyleSet->Set("MeshTerrainModeManagerCommands.LoadUVsTools", new IMAGE_BRUSH_SVG( "Icons/LoadUVsTools", Icon20x20 ) );
		StyleSet->Set("MeshTerrainModeManagerCommands.LoadBakingTools", new IMAGE_BRUSH_SVG( "Icons/LoadBakingTools", Icon20x20 ) );
		StyleSet->Set("MeshTerrainModeManagerCommands.LoadVolumeTools", new IMAGE_BRUSH_SVG( "Icons/LoadVolumeTools", Icon20x20 ) );
		StyleSet->Set("MeshTerrainModeManagerCommands.LoadLodsTools", new IMAGE_BRUSH_SVG( "Icons/LoadLodsTools", Icon20x20 ) );

		StyleSet->Set("MeshTerrainModeManagerCommands.EnterShapesSubmode", new IMAGE_BRUSH_SVG( "Icons/LoadShapesTools", Icon20x20 ) );
		StyleSet->Set("MeshTerrainModeManagerCommands.EnterCreateSubmode", new IMAGE_BRUSH_SVG( "Icons/LoadCreateTools", Icon20x20 ) );
		StyleSet->Set("MeshTerrainModeManagerCommands.EnterEditSubmode", new IMAGE_BRUSH_SVG( "Icons/LoadTriTools", Icon20x20 ) );
		StyleSet->Set("MeshTerrainModeManagerCommands.EnterModifiersSubmode",	new IMAGE_BRUSH_SVG( "Icons/Modifier_20", Icon20x20));
		StyleSet->Set("MeshTerrainModeManagerCommands.EnterSculptSubmode", new IMAGE_BRUSH_SVG( "Icons/LoadDeformTools", Icon20x20 ) );
		StyleSet->Set("MeshTerrainModeManagerCommands.EnterPaintSubmode", new IMAGE_BRUSH_SVG( "Icons/GroupPaint", Icon20x20 ) );
		
		StyleSet->Set("MeshTerrainModeManagerCommands.LoadSkinTools", new IMAGE_BRUSH_SVG( "Icons/Skin", Icon20x20 ) );
		StyleSet->Set("MeshTerrainModeManagerCommands.LoadSkeletonTools", new IMAGE_BRUSH_SVG( "Icons/SkeletalEditor_20", Icon20x20 ) );

		StyleSet->Set("MeshTerrainModeManagerCommands.BeginShapeSprayTool", 				new IMAGE_PLUGIN_BRUSH("Icons/ShapeSpray_40x",	Icon20x20));
		StyleSet->Set("MeshTerrainModeManagerCommands.BeginShapeSprayTool.Small", 		new IMAGE_PLUGIN_BRUSH("Icons/ShapeSpray_40x",	Icon20x20));
		//StyleSet->Set("MeshTerrainModeManagerCommands.BeginMeshSpaceDeformerTool", 		new IMAGE_PLUGIN_BRUSH("Icons/icon_Tool_Displace_40x",		Icon20x20));
		//StyleSet->Set("MeshTerrainModeManagerCommands.BeginMeshSpaceDeformerTool.Small", 	new IMAGE_PLUGIN_BRUSH("Icons/icon_Tool_Displace_40x",		Icon20x20));
		StyleSet->Set("MeshTerrainModeManagerCommands.BeginPolygonOnMeshTool", 			new IMAGE_PLUGIN_BRUSH("Icons/icon_Tool_PolygonOnMesh_40x",	Icon20x20));
		StyleSet->Set("MeshTerrainModeManagerCommands.BeginPolygonOnMeshTool.Small", 		new IMAGE_PLUGIN_BRUSH("Icons/icon_Tool_PolygonOnMesh_40x",	Icon20x20));
		StyleSet->Set("MeshTerrainModeManagerCommands.BeginParameterizeMeshTool", 		new IMAGE_PLUGIN_BRUSH("Icons/icon_Tool_UVGenerate_40x",	Icon20x20));
		StyleSet->Set("MeshTerrainModeManagerCommands.BeginParameterizeMeshTool.Small", 	new IMAGE_PLUGIN_BRUSH("Icons/icon_Tool_UVGenerate_40x",	Icon20x20));
		StyleSet->Set("MeshTerrainModeManagerCommands.BeginPolyGroupsTool", 				new IMAGE_PLUGIN_BRUSH("Icons/icon_Tool_PolyGroups_40x",	Icon20x20));
		StyleSet->Set("MeshTerrainModeManagerCommands.BeginPolyGroupsTool.Small", 		new IMAGE_PLUGIN_BRUSH("Icons/icon_Tool_PolyGroups_40x",	Icon20x20));


		// Modes Palette Toolbar Icons
		StyleSet->Set("MeshTerrainModeManagerCommands.BeginAddBoxPrimitiveTool", 			new IMAGE_BRUSH_SVG("Icons/ModelingBox", 		Icon20x20));
		StyleSet->Set("MeshTerrainModeManagerCommands.BeginAddBoxPrimitiveTool.Small", 		new IMAGE_BRUSH_SVG("Icons/ModelingBox",		Icon20x20));
		StyleSet->Set("MeshTerrainModeManagerCommands.BeginAddCapsulePrimitiveTool",			new IMAGE_BRUSH_SVG("Icons/ModelingCapsule", Icon20x20));
		StyleSet->Set("MeshTerrainModeManagerCommands.BeginAddCapsulePrimitiveTool.Small",	new IMAGE_BRUSH_SVG("Icons/ModelingCapsule", Icon20x20));
		StyleSet->Set("MeshTerrainModeManagerCommands.BeginAddCylinderPrimitiveTool", 			new IMAGE_BRUSH_SVG("Icons/ModelingCylinder", 		Icon20x20));
		StyleSet->Set("MeshTerrainModeManagerCommands.BeginAddCylinderPrimitiveTool.Small", 		new IMAGE_BRUSH_SVG("Icons/ModelingCylinder",		Icon20x20));
		StyleSet->Set("MeshTerrainModeManagerCommands.BeginAddConePrimitiveTool", 			new IMAGE_BRUSH_SVG("Icons/ModelingCone", 		Icon20x20));
		StyleSet->Set("MeshTerrainModeManagerCommands.BeginAddConePrimitiveTool.Small", 		new IMAGE_BRUSH_SVG("Icons/ModelingCone",		Icon20x20));
		StyleSet->Set("MeshTerrainModeManagerCommands.BeginAddArrowPrimitiveTool", 			new IMAGE_BRUSH_SVG("Icons/ModelingArrow", 		Icon20x20));
		StyleSet->Set("MeshTerrainModeManagerCommands.BeginAddArrowPrimitiveTool.Small", 		new IMAGE_BRUSH_SVG("Icons/ModelingArrow",		Icon20x20));
		StyleSet->Set("MeshTerrainModeManagerCommands.BeginAddRectanglePrimitiveTool", 			new IMAGE_PLUGIN_BRUSH("Icons/ModelingRectangle_x20", 		Icon20x20));
		StyleSet->Set("MeshTerrainModeManagerCommands.BeginAddRectanglePrimitiveTool.Small", 		new IMAGE_PLUGIN_BRUSH("Icons/ModelingRectangle_x40",		Icon20x20));
		StyleSet->Set("MeshTerrainModeManagerCommands.BeginAddDiscPrimitiveTool", 			new IMAGE_PLUGIN_BRUSH("Icons/ModelingDisc_x20", 		Icon20x20));
		StyleSet->Set("MeshTerrainModeManagerCommands.BeginAddDiscPrimitiveTool.Small", 		new IMAGE_PLUGIN_BRUSH("Icons/ModelingDisc_x40",		Icon20x20));
		StyleSet->Set("MeshTerrainModeManagerCommands.BeginAddTorusPrimitiveTool", 			new IMAGE_BRUSH_SVG("Icons/ModelingTorus", 		Icon20x20));
		StyleSet->Set("MeshTerrainModeManagerCommands.BeginAddTorusPrimitiveTool.Small", 		new IMAGE_BRUSH_SVG("Icons/ModelingTorus",		Icon20x20));
		StyleSet->Set("MeshTerrainModeManagerCommands.BeginAddSpherePrimitiveTool", 			new IMAGE_BRUSH_SVG("Icons/ModelingSphere", 		Icon20x20));
		StyleSet->Set("MeshTerrainModeManagerCommands.BeginAddSpherePrimitiveTool.Small", 		new IMAGE_BRUSH_SVG("Icons/ModelingSphere",		Icon20x20));
		StyleSet->Set("MeshTerrainModeManagerCommands.BeginAddStairsPrimitiveTool", 			new IMAGE_BRUSH_SVG("Icons/Staircase", Icon20x20));
		StyleSet->Set("MeshTerrainModeManagerCommands.BeginAddStairsPrimitiveTool.Small",		new IMAGE_BRUSH_SVG("Icons/Staircase", Icon20x20));
		StyleSet->Set("MeshTerrainModeManagerCommands.BeginDrawSplineTool",	     	        new IMAGE_BRUSH_SVG("Icons/GeometryDrawSpline", Icon20x20));
		StyleSet->Set("MeshTerrainModeManagerCommands.BeginDrawSplineTool.Small",		        new IMAGE_BRUSH_SVG("Icons/GeometryDrawSpline", Icon20x20));

		StyleSet->Set("MeshTerrainModeManagerCommands.BeginDrawPolygonTool", 				new IMAGE_PLUGIN_BRUSH("Icons/DrawPolygon_40x",		Icon20x20));
		StyleSet->Set("MeshTerrainModeManagerCommands.BeginDrawPolygonTool.Small", 		new IMAGE_PLUGIN_BRUSH("Icons/DrawPolygon_40x", 	Icon20x20));
		StyleSet->Set("MeshTerrainModeManagerCommands.BeginAddPatchTool",					new IMAGE_PLUGIN_BRUSH("Icons/Patch_40x",			Icon20x20));
		StyleSet->Set("MeshTerrainModeManagerCommands.BeginAddPatchTool.Small",			new IMAGE_PLUGIN_BRUSH("Icons/Patch_40x",			Icon20x20));

		StyleSet->Set("MeshTerrainModeManagerCommands.BeginSmoothMeshTool", 				new IMAGE_PLUGIN_BRUSH("Icons/ModelingSmooth_x40", 			Icon20x20));
		StyleSet->Set("MeshTerrainModeManagerCommands.BeginSmoothMeshTool.Small", 		new IMAGE_PLUGIN_BRUSH("Icons/ModelingSmooth_x40", 			Icon20x20));
		StyleSet->Set("MeshTerrainModeManagerCommands.BeginSculptMeshTool", 				new IMAGE_PLUGIN_BRUSH("Icons/Sculpt_40x", 			Icon20x20));
		StyleSet->Set("MeshTerrainModeManagerCommands.BeginSculptMeshTool.Small", 		new IMAGE_PLUGIN_BRUSH("Icons/Sculpt_40x", 			Icon20x20));
		StyleSet->Set("MeshTerrainModeManagerCommands.BeginPolyEditTool", 				new IMAGE_PLUGIN_BRUSH("Icons/PolyEdit_40x", 		Icon20x20));
		StyleSet->Set("MeshTerrainModeManagerCommands.BeginPolyEditTool.Small", 			new IMAGE_PLUGIN_BRUSH("Icons/PolyEdit_40x", 		Icon20x20));
		StyleSet->Set("MeshTerrainModeManagerCommands.BeginSubdividePolyTool",			new IMAGE_BRUSH_SVG("Icons/ModelingSubD",		Icon20x20));
		StyleSet->Set("MeshTerrainModeManagerCommands.BeginSubdividePolyTool.Small",		new IMAGE_BRUSH_SVG("Icons/ModelingSubD",		Icon20x20));
		StyleSet->Set("MeshTerrainModeManagerCommands.BeginTriEditTool", 				new IMAGE_PLUGIN_BRUSH("Icons/TriEdit_40x", 		Icon20x20));
		StyleSet->Set("MeshTerrainModeManagerCommands.BeginTriEditTool.Small", 			new IMAGE_PLUGIN_BRUSH("Icons/TriEdit_40x", 		Icon20x20));
		//StyleSet->Set("MeshTerrainModeManagerCommands.BeginPolyDeformTool", 			new IMAGE_PLUGIN_BRUSH("Icons/PolyEdit_40x", 		Icon20x20));
		//StyleSet->Set("MeshTerrainModeManagerCommands.BeginPolyDeformTool.Small", 		new IMAGE_PLUGIN_BRUSH("Icons/PolyEdit_40x", 		Icon20x20));
		StyleSet->Set("MeshTerrainModeManagerCommands.BeginDisplaceMeshTool", 			new IMAGE_PLUGIN_BRUSH("Icons/Displace_40x", 		Icon20x20));
		StyleSet->Set("MeshTerrainModeManagerCommands.BeginDisplaceMeshTool.Small", 		new IMAGE_PLUGIN_BRUSH("Icons/Displace_40x", 		Icon20x20));
		StyleSet->Set("MeshTerrainModeManagerCommands.BeginTransformMeshesTool", 			new IMAGE_PLUGIN_BRUSH("Icons/Transform_40x", 		Icon20x20));
		StyleSet->Set("MeshTerrainModeManagerCommands.BeginTransformMeshesTool.Small", 	new IMAGE_PLUGIN_BRUSH("Icons/Transform_40x", 		Icon20x20));

		constexpr int VertexSculptColorIdx = 0;
		const FLinearColor VertexSculptColor = LinearColors::SelectColor<FLinearColor>(VertexSculptColorIdx);
		StyleSet->Set("MeshTerrainModeManagerCommands.BeginSculptMeshMoveBrushTool", 					new IMAGE_BRUSH_SVG("Icons/BrushToolIcons/BrushMove", 	Icon40x40, VertexSculptColor));
		StyleSet->Set("MeshTerrainModeManagerCommands.BeginSculptMeshMoveBrushTool.Small", 				new IMAGE_BRUSH_SVG("Icons/BrushToolIcons/BrushMove", 	Icon20x20, VertexSculptColor));
		StyleSet->Set("MeshTerrainModeManagerCommands.BeginSculptMeshPullKelvinBrushTool", 				new IMAGE_BRUSH_SVG("Icons/BrushToolIcons/BrushGrab", 	Icon40x40, VertexSculptColor));
		StyleSet->Set("MeshTerrainModeManagerCommands.BeginSculptMeshPullKelvinBrushTool.Small",			new IMAGE_BRUSH_SVG("Icons/BrushToolIcons/BrushGrab", 	Icon20x20, VertexSculptColor));
		StyleSet->Set("MeshTerrainModeManagerCommands.BeginSculptMeshPullSharpKelvinBrushTool", 			new IMAGE_BRUSH_SVG("Icons/BrushToolIcons/BrushGrabSharp", 	Icon40x40, VertexSculptColor));
		StyleSet->Set("MeshTerrainModeManagerCommands.BeginSculptMeshPullSharpKelvinBrushTool.Small",	new IMAGE_BRUSH_SVG("Icons/BrushToolIcons/BrushGrabSharp", 	Icon20x20, VertexSculptColor));
		StyleSet->Set("MeshTerrainModeManagerCommands.BeginSculptMeshSmoothBrushTool", 					new IMAGE_BRUSH_SVG("Icons/BrushToolIcons/BrushSmooth", 	Icon40x40, VertexSculptColor));
		StyleSet->Set("MeshTerrainModeManagerCommands.BeginSculptMeshSmoothBrushTool.Small", 			new IMAGE_BRUSH_SVG("Icons/BrushToolIcons/BrushSmooth", 	Icon20x20, VertexSculptColor));
		StyleSet->Set("MeshTerrainModeManagerCommands.BeginSculptMeshSmoothFillBrushTool", 				new IMAGE_BRUSH_SVG("Icons/BrushToolIcons/BrushSmoothFill", 	Icon40x40, VertexSculptColor));
		StyleSet->Set("MeshTerrainModeManagerCommands.BeginSculptMeshSmoothFillBrushTool.Small", 		new IMAGE_BRUSH_SVG("Icons/BrushToolIcons/BrushSmoothFill", 	Icon20x20, VertexSculptColor));
		StyleSet->Set("MeshTerrainModeManagerCommands.BeginSculptMeshOffsetBrushTool", 					new IMAGE_BRUSH_SVG("Icons/BrushToolIcons/BrushSculptN", 	Icon40x40, VertexSculptColor));
		StyleSet->Set("MeshTerrainModeManagerCommands.BeginSculptMeshOffsetBrushTool.Small", 			new IMAGE_BRUSH_SVG("Icons/BrushToolIcons/BrushSculptN", 	Icon20x20, VertexSculptColor));
		StyleSet->Set("MeshTerrainModeManagerCommands.BeginSculptMeshSculptViewBrushTool", 				new IMAGE_BRUSH_SVG("Icons/BrushToolIcons/BrushSculptV", 	Icon40x40, VertexSculptColor));
		StyleSet->Set("MeshTerrainModeManagerCommands.BeginSculptMeshSculptViewBrushTool.Small", 		new IMAGE_BRUSH_SVG("Icons/BrushToolIcons/BrushSculptV", 	Icon20x20, VertexSculptColor));
		StyleSet->Set("MeshTerrainModeManagerCommands.BeginSculptMeshSculptMaxBrushTool", 				new IMAGE_BRUSH_SVG("Icons/BrushToolIcons/BrushSculptMX", 	Icon40x40, VertexSculptColor));
		StyleSet->Set("MeshTerrainModeManagerCommands.BeginSculptMeshSculptMaxBrushTool.Small", 			new IMAGE_BRUSH_SVG("Icons/BrushToolIcons/BrushSculptMX", 	Icon20x20, VertexSculptColor));
		StyleSet->Set("MeshTerrainModeManagerCommands.BeginSculptMeshInflateBrushTool", 					new IMAGE_BRUSH_SVG("Icons/BrushToolIcons/BrushInflate", 	Icon40x40, VertexSculptColor));
		StyleSet->Set("MeshTerrainModeManagerCommands.BeginSculptMeshInflateBrushTool.Small", 			new IMAGE_BRUSH_SVG("Icons/BrushToolIcons/BrushInflate", 	Icon20x20, VertexSculptColor));
		StyleSet->Set("MeshTerrainModeManagerCommands.BeginSculptMeshScaleKelvinBrushTool", 				new IMAGE_BRUSH_SVG("Icons/BrushToolIcons/BrushScale", 	Icon40x40, VertexSculptColor));
		StyleSet->Set("MeshTerrainModeManagerCommands.BeginSculptMeshScaleKelvinBrushTool.Small", 		new IMAGE_BRUSH_SVG("Icons/BrushToolIcons/BrushScale", 	Icon20x20, VertexSculptColor));
		StyleSet->Set("MeshTerrainModeManagerCommands.BeginSculptMeshPinchBrushTool", 					new IMAGE_BRUSH_SVG("Icons/BrushToolIcons/BrushPinch", 	Icon40x40, VertexSculptColor));
		StyleSet->Set("MeshTerrainModeManagerCommands.BeginSculptMeshPinchBrushTool.Small", 				new IMAGE_BRUSH_SVG("Icons/BrushToolIcons/BrushPinch", 	Icon20x20, VertexSculptColor));
		StyleSet->Set("MeshTerrainModeManagerCommands.BeginSculptMeshTwistKelvinBrushTool", 				new IMAGE_BRUSH_SVG("Icons/BrushToolIcons/BrushTwist", 	Icon40x40, VertexSculptColor));
		StyleSet->Set("MeshTerrainModeManagerCommands.BeginSculptMeshTwistKelvinBrushTool.Small", 		new IMAGE_BRUSH_SVG("Icons/BrushToolIcons/BrushTwist", 	Icon20x20, VertexSculptColor));
		StyleSet->Set("MeshTerrainModeManagerCommands.BeginSculptMeshFlattenBrushTool", 					new IMAGE_BRUSH_SVG("Icons/BrushToolIcons/BrushFlatten", 	Icon40x40, VertexSculptColor));
		StyleSet->Set("MeshTerrainModeManagerCommands.BeginSculptMeshFlattenBrushTool.Small", 			new IMAGE_BRUSH_SVG("Icons/BrushToolIcons/BrushFlatten", 	Icon20x20, VertexSculptColor));
		StyleSet->Set("MeshTerrainModeManagerCommands.BeginSculptMeshPlaneBrushTool", 					new IMAGE_BRUSH_SVG("Icons/BrushToolIcons/BrushPlaneN", 	Icon40x40, VertexSculptColor));
		StyleSet->Set("MeshTerrainModeManagerCommands.BeginSculptMeshPlaneBrushTool.Small", 				new IMAGE_BRUSH_SVG("Icons/BrushToolIcons/BrushPlaneN", 	Icon20x20, VertexSculptColor));
		StyleSet->Set("MeshTerrainModeManagerCommands.BeginSculptMeshPlaneViewAlignedBrushTool", 		new IMAGE_BRUSH_SVG("Icons/BrushToolIcons/BrushPlaneV", 	Icon40x40, VertexSculptColor));
		StyleSet->Set("MeshTerrainModeManagerCommands.BeginSculptMeshPlaneViewAlignedBrushTool.Small", 	new IMAGE_BRUSH_SVG("Icons/BrushToolIcons/BrushPlaneV", 	Icon20x20, VertexSculptColor));
		StyleSet->Set("MeshTerrainModeManagerCommands.BeginSculptMeshFixedPlaneBrushTool", 				new IMAGE_BRUSH_SVG("Icons/BrushToolIcons/BrushPlaneW", 	Icon40x40, VertexSculptColor));
		StyleSet->Set("MeshTerrainModeManagerCommands.BeginSculptMeshFixedPlaneBrushTool.Small", 		new IMAGE_BRUSH_SVG("Icons/BrushToolIcons/BrushPlaneW", 	Icon20x20, VertexSculptColor));
		StyleSet->Set("MeshTerrainModeManagerCommands.BeginSculptMeshEraseLayerTool",				new IMAGE_BRUSH_SVG("Icons/BrushToolIcons/BrushErase",		Icon40x40, VertexSculptColor));
		StyleSet->Set("MeshTerrainModeManagerCommands.BeginSculptMeshEraseLayerTool.Small",		new IMAGE_BRUSH_SVG("Icons/BrushToolIcons/BrushErase",		Icon20x20, VertexSculptColor));
		
		StyleSet->Set("MeshTerrainModeManagerCommands.BeginRemeshSculptMeshTool", 		new IMAGE_PLUGIN_BRUSH("Icons/DynaSculpt_40x", 		Icon20x20));
		StyleSet->Set("MeshTerrainModeManagerCommands.BeginRemeshSculptMeshTool.Small", 	new IMAGE_PLUGIN_BRUSH("Icons/DynaSculpt_40x",		Icon20x20));
		StyleSet->Set("MeshTerrainModeManagerCommands.BeginRemeshMeshTool", 				new IMAGE_PLUGIN_BRUSH("Icons/Remesh_40x", 			Icon20x20));
		StyleSet->Set("MeshTerrainModeManagerCommands.BeginRemeshMeshTool.Small", 		new IMAGE_PLUGIN_BRUSH("Icons/Remesh_40x", 			Icon20x20));
		StyleSet->Set("MeshTerrainModeManagerCommands.BeginProjectToTargetTool", 		new IMAGE_PLUGIN_BRUSH("Icons/ModelingRemeshToTarget_x40",	Icon20x20));
		StyleSet->Set("MeshTerrainModeManagerCommands.BeginProjectToTargetTool.Small", 	new IMAGE_PLUGIN_BRUSH("Icons/ModelingRemeshToTarget_x40",	Icon20x20));
		//StyleSet->Set("MeshTerrainModeManagerCommands.BeginProjectToTargetTool",			new IMAGE_PLUGIN_BRUSH("",			Icon20x20));
		//StyleSet->Set("MeshTerrainModeManagerCommands.BeginProjectToTargetTool.Small",	new IMAGE_PLUGIN_BRUSH("",			Icon20x20));
		StyleSet->Set("MeshTerrainModeManagerCommands.BeginSimplifyMeshTool", 			new IMAGE_PLUGIN_BRUSH("Icons/Simplify_40x", 		Icon20x20));
		StyleSet->Set("MeshTerrainModeManagerCommands.BeginSimplifyMeshTool.Small", 		new IMAGE_PLUGIN_BRUSH("Icons/Simplify_40x", 		Icon20x20));
		StyleSet->Set("MeshTerrainModeManagerCommands.BeginEditNormalsTool", 			new IMAGE_PLUGIN_BRUSH("Icons/Normals_40x",			Icon20x20));
		StyleSet->Set("MeshTerrainModeManagerCommands.BeginEditNormalsTool.Small", 		new IMAGE_PLUGIN_BRUSH("Icons/Normals_40x",			Icon20x20));
		StyleSet->Set("MeshTerrainModeManagerCommands.BeginEditTangentsTool", 			new IMAGE_PLUGIN_BRUSH("Icons/ModelingTangents_x40",			Icon20x20));
		StyleSet->Set("MeshTerrainModeManagerCommands.BeginEditTangentsTool.Small", 		new IMAGE_PLUGIN_BRUSH("Icons/ModelingTangents_x40",			Icon20x20));
		StyleSet->Set("MeshTerrainModeManagerCommands.BeginUVSeamEditTool", 			new IMAGE_PLUGIN_BRUSH("Icons/ModelingUVSeamEdit_x40",			Icon20x20));
		StyleSet->Set("MeshTerrainModeManagerCommands.BeginUVSeamEditTool.Small", 		new IMAGE_PLUGIN_BRUSH("Icons/ModelingUVSeamEdit_x40",			Icon20x20));
		StyleSet->Set("MeshTerrainModeManagerCommands.BeginBakeMeshAttributeMapsTool", 			new IMAGE_BRUSH_SVG("Icons/BakeTexture",			Icon20x20));
		StyleSet->Set("MeshTerrainModeManagerCommands.BeginBakeMeshAttributeMapsTool.Small", 		new IMAGE_BRUSH_SVG("Icons/BakeTexture",			Icon20x20));
		StyleSet->Set("MeshTerrainModeManagerCommands.BeginBakeMultiMeshAttributeMapsTool", 			new IMAGE_BRUSH_SVG("Icons/BakeAll",			Icon20x20));
		StyleSet->Set("MeshTerrainModeManagerCommands.BeginBakeMultiMeshAttributeMapsTool.Small", 		new IMAGE_BRUSH_SVG("Icons/BakeAll",			Icon20x20));
		StyleSet->Set("MeshTerrainModeManagerCommands.BeginBakeRenderCaptureTool", 			new IMAGE_BRUSH_SVG("Icons/BakeRenderCapture",			Icon20x20));
		StyleSet->Set("MeshTerrainModeManagerCommands.BeginBakeRenderCaptureTool.Small", 		new IMAGE_BRUSH_SVG("Icons/BakeRenderCapture",			Icon20x20));
		StyleSet->Set("MeshTerrainModeManagerCommands.BeginBakeMeshAttributeVertexTool", new IMAGE_BRUSH_SVG("Icons/BakeVertex", Icon20x20));
		StyleSet->Set("MeshTerrainModeManagerCommands.BeginBakeMeshAttributeVertexTool.Small", new IMAGE_BRUSH_SVG("Icons/BakeVertex", Icon20x20));
		//StyleSet->Set("MeshTerrainModeManagerCommands.BeginRemoveOccludedTrianglesTool", 				new IMAGE_PLUGIN_BRUSH("Icons/Jacket_40x",			Icon20x20));
		//StyleSet->Set("MeshTerrainModeManagerCommands.BeginRemoveOccludedTrianglesTool.Small", 		new IMAGE_PLUGIN_BRUSH("Icons/Jacket_40x",			Icon20x20));
		StyleSet->Set("MeshTerrainModeManagerCommands.BeginHoleFillTool", 				new IMAGE_PLUGIN_BRUSH("Icons/ModelingHoleFill_x40",			Icon20x20));
		StyleSet->Set("MeshTerrainModeManagerCommands.BeginHoleFillTool.Small", 		new IMAGE_PLUGIN_BRUSH("Icons/ModelingHoleFill_x40",			Icon20x20));
		StyleSet->Set("MeshTerrainModeManagerCommands.BeginUVProjectionTool", 			new IMAGE_PLUGIN_BRUSH("Icons/UVProjection_40x", 	Icon20x20));
		StyleSet->Set("MeshTerrainModeManagerCommands.BeginUVProjectionTool.Small", 		new IMAGE_PLUGIN_BRUSH("Icons/UVProjection_40x", 	Icon20x20));
		//StyleSet->Set("MeshTerrainModeManagerCommands.BeginUVLayoutTool", 			new IMAGE_PLUGIN_BRUSH("Icons/UVLayout_40x", 	Icon20x20));
		//StyleSet->Set("MeshTerrainModeManagerCommands.BeginUVLayoutTool.Small", 		new IMAGE_PLUGIN_BRUSH("Icons/UVLayout_40x", 	Icon20x20));
		StyleSet->Set("MeshTerrainModeManagerCommands.BeginVoxelMergeTool", 				new IMAGE_PLUGIN_BRUSH("Icons/VoxMerge_40x", 		Icon20x20));
		StyleSet->Set("MeshTerrainModeManagerCommands.BeginVoxelMergeTool.Small", 		new IMAGE_PLUGIN_BRUSH("Icons/VoxMerge_40x", 		Icon20x20));
		StyleSet->Set("MeshTerrainModeManagerCommands.BeginVoxelBooleanTool", 			new IMAGE_PLUGIN_BRUSH("Icons/VoxBoolean_40x", 		Icon20x20));
		StyleSet->Set("MeshTerrainModeManagerCommands.BeginVoxelBooleanTool.Small", 		new IMAGE_PLUGIN_BRUSH("Icons/VoxBoolean_40x", 		Icon20x20));
		//StyleSet->Set("MeshTerrainModeManagerCommands.BeginSelfUnionTool",				new IMAGE_PLUGIN_BRUSH("Icons/MeshMerge_40x",		Icon20x20));
		//StyleSet->Set("MeshTerrainModeManagerCommands.BeginSelfUnionTool.Small",			new IMAGE_PLUGIN_BRUSH("Icons/MeshMerge_40x",		Icon20x20));
		//StyleSet->Set("MeshTerrainModeManagerCommands.BeginMeshBooleanTool",				new IMAGE_PLUGIN_BRUSH("Icons/Boolean_40x",			Icon20x20));
		//StyleSet->Set("MeshTerrainModeManagerCommands.BeginMeshBooleanTool.Small",		new IMAGE_PLUGIN_BRUSH("Icons/Boolean_40x",			Icon20x20));
		StyleSet->Set("MeshTerrainModeManagerCommands.BeginPlaneCutTool", 				new IMAGE_PLUGIN_BRUSH("Icons/PlaneCut_40x", 		Icon20x20));
		StyleSet->Set("MeshTerrainModeManagerCommands.BeginPlaneCutTool.Small", 			new IMAGE_PLUGIN_BRUSH("Icons/PlaneCut_40x", 		Icon20x20));
		StyleSet->Set("MeshTerrainModeManagerCommands.BeginMirrorTool", 				    new IMAGE_PLUGIN_BRUSH("Icons/ModelingMirror_x40", 		Icon20x20));
		StyleSet->Set("MeshTerrainModeManagerCommands.BeginMirrorTool.Small", 			new IMAGE_PLUGIN_BRUSH("Icons/ModelingMirror_x40", 		Icon20x20));
		StyleSet->Set("MeshTerrainModeManagerCommands.BeginOffsetMeshTool", 				new IMAGE_PLUGIN_BRUSH("Icons/ModelingOffset_x40", 		Icon20x20));
		StyleSet->Set("MeshTerrainModeManagerCommands.BeginOffsetMeshTool.Small", 		new IMAGE_PLUGIN_BRUSH("Icons/ModelingOffset_x40", 		Icon20x20));
		StyleSet->Set("MeshTerrainModeManagerCommands.BeginDisplaceMeshTool", 			new IMAGE_PLUGIN_BRUSH("Icons/ModelingDisplace_x40", 		Icon20x20));
		StyleSet->Set("MeshTerrainModeManagerCommands.BeginDisplaceMeshTool.Small", 		new IMAGE_PLUGIN_BRUSH("Icons/ModelingDisplace_x40", 		Icon20x20));
		StyleSet->Set("MeshTerrainModeManagerCommands.BeginMeshSelectionTool", 			new IMAGE_PLUGIN_BRUSH("Icons/MeshSelect_40x",		Icon20x20));
		StyleSet->Set("MeshTerrainModeManagerCommands.BeginMeshSelectionTool.Small", 	new IMAGE_PLUGIN_BRUSH("Icons/MeshSelect_40x",		Icon20x20));

		StyleSet->Set("MeshTerrainModeManagerCommands.BeginMeshInspectorTool", 			new IMAGE_PLUGIN_BRUSH("Icons/Inspector_40x",		Icon20x20));
		StyleSet->Set("MeshTerrainModeManagerCommands.BeginMeshInspectorTool.Small", 		new IMAGE_PLUGIN_BRUSH("Icons/Inspector_40x",		Icon20x20));
		StyleSet->Set("MeshTerrainModeManagerCommands.BeginWeldEdgesTool", 				new IMAGE_PLUGIN_BRUSH("Icons/WeldEdges_40x", 		Icon20x20));
		StyleSet->Set("MeshTerrainModeManagerCommands.BeginWeldEdgesTool.Small", 			new IMAGE_PLUGIN_BRUSH("Icons/WeldEdges_40x", 		Icon20x20));
		StyleSet->Set("MeshTerrainModeManagerCommands.BeginAttributeEditorTool", 			new IMAGE_PLUGIN_BRUSH("Icons/AttributeEditor_40x", Icon20x20));
		StyleSet->Set("MeshTerrainModeManagerCommands.BeginAttributeEditorTool.Small", 	new IMAGE_PLUGIN_BRUSH("Icons/AttributeEditor_40x", Icon20x20));

		StyleSet->Set("MeshTerrainModeManagerCommands.BeginAlignObjectsTool",                  new FSlateImageBrush(StyleSet->RootToCoreContentDir(TEXT("../Editor/Slate/Icons/GeneralTools/Align_40x.png")), Icon20x20));
		StyleSet->Set("MeshTerrainModeManagerCommands.BeginAlignObjectsTool.Small",            new FSlateImageBrush(StyleSet->RootToCoreContentDir(TEXT("../Editor/Slate/Icons/GeneralTools/Align_40x.png")), Icon20x20));
		StyleSet->Set("MeshTerrainModeManagerCommands.BeginTransferMeshTool",                  new FSlateImageBrush(StyleSet->RootToCoreContentDir(TEXT("../Editor/Slate/Icons/GeneralTools/Next_40x.png")), Icon20x20));
		StyleSet->Set("MeshTerrainModeManagerCommands.BeginTransferMeshTool.Small",            new FSlateImageBrush(StyleSet->RootToCoreContentDir(TEXT("../Editor/Slate/Icons/GeneralTools/Next_40x.png")), Icon20x20));
		StyleSet->Set("MeshTerrainModeManagerCommands.BeginGlobalUVGenerateTool",              new IMAGE_PLUGIN_BRUSH("Icons/AutoUnwrap_40x",       Icon20x20));      
		StyleSet->Set("MeshTerrainModeManagerCommands.BeginGlobalUVGenerateTool.Small",        new IMAGE_PLUGIN_BRUSH("Icons/AutoUnwrap_40x",       Icon20x20));      
		StyleSet->Set("MeshTerrainModeManagerCommands.BeginBakeTransformTool",                 new IMAGE_BRUSH_SVG("Icons/GeometryBakeXForm",        Icon20x20));      
		StyleSet->Set("MeshTerrainModeManagerCommands.BeginBakeTransformTool.Small",           new IMAGE_BRUSH_SVG("Icons/GeometryBakeXForm",        Icon20x20));      
		StyleSet->Set("MeshTerrainModeManagerCommands.BeginCombineMeshesTool",                 new IMAGE_BRUSH_SVG("Icons/GeometryCombine",          Icon20x20));
		StyleSet->Set("MeshTerrainModeManagerCommands.BeginCombineMeshesTool.Small",           new IMAGE_BRUSH_SVG("Icons/GeometryCombine",          Icon20x20));
		StyleSet->Set("MeshTerrainModeManagerCommands.BeginDuplicateMeshesTool",               new IMAGE_PLUGIN_BRUSH("Icons/Duplicate_40x",        Icon20x20));   
		StyleSet->Set("MeshTerrainModeManagerCommands.BeginDuplicateMeshesTool.Small",         new IMAGE_PLUGIN_BRUSH("Icons/Duplicate_40x",        Icon20x20));   
		StyleSet->Set("MeshTerrainModeManagerCommands.BeginEditMeshMaterialsTool",             new IMAGE_PLUGIN_BRUSH("Icons/EditMats_40x",         Icon20x20));     
		StyleSet->Set("MeshTerrainModeManagerCommands.BeginEditMeshMaterialsTool.Small",       new IMAGE_PLUGIN_BRUSH("Icons/EditMats_40x",         Icon20x20));     
		StyleSet->Set("MeshTerrainModeManagerCommands.BeginEditPivotTool",                     new IMAGE_PLUGIN_BRUSH("Icons/EditPivot_40x",        Icon20x20));      
		StyleSet->Set("MeshTerrainModeManagerCommands.BeginEditPivotTool.Small",               new IMAGE_PLUGIN_BRUSH("Icons/EditPivot_40x",        Icon20x20));
		StyleSet->Set("MeshTerrainModeManagerCommands.BeginAddPivotActorTool",                 new IMAGE_BRUSH_SVG("Icons/Pivot",                   Icon20x20));
		StyleSet->Set("MeshTerrainModeManagerCommands.BeginAddPivotActorTool.Small",           new IMAGE_BRUSH_SVG("Icons/Pivot",                   Icon20x20));
		StyleSet->Set("MeshTerrainModeManagerCommands.BeginGroupUVGenerateTool",               new IMAGE_PLUGIN_BRUSH("Icons/GroupUnwrap_40x",      Icon20x20));       
		StyleSet->Set("MeshTerrainModeManagerCommands.BeginGroupUVGenerateTool.Small",         new IMAGE_PLUGIN_BRUSH("Icons/GroupUnwrap_40x",      Icon20x20));       
		StyleSet->Set("MeshTerrainModeManagerCommands.BeginRemoveOccludedTrianglesTool",       new IMAGE_PLUGIN_BRUSH("Icons/Jacketing_40x",        Icon20x20));     
		StyleSet->Set("MeshTerrainModeManagerCommands.BeginRemoveOccludedTrianglesTool.Small", new IMAGE_PLUGIN_BRUSH("Icons/Jacketing_40x",        Icon20x20));     
		StyleSet->Set("MeshTerrainModeManagerCommands.BeginPolygonCutTool",                    new IMAGE_PLUGIN_BRUSH("Icons/PolyCut_40x",          Icon20x20));   
		StyleSet->Set("MeshTerrainModeManagerCommands.BeginPolygonCutTool.Small",              new IMAGE_PLUGIN_BRUSH("Icons/PolyCut_40x",          Icon20x20));   
		StyleSet->Set("MeshTerrainModeManagerCommands.BeginPolyDeformTool",                    new IMAGE_PLUGIN_BRUSH("Icons/PolyDeform_40x",       Icon20x20));      
		StyleSet->Set("MeshTerrainModeManagerCommands.BeginPolyDeformTool.Small",              new IMAGE_PLUGIN_BRUSH("Icons/PolyDeform_40x",       Icon20x20));      
		StyleSet->Set("MeshTerrainModeManagerCommands.BeginPolyGroupsTool",                    new IMAGE_PLUGIN_BRUSH("Icons/PolyGroups_40x",       Icon20x20));      
		StyleSet->Set("MeshTerrainModeManagerCommands.BeginPolyGroupsTool.Small",              new IMAGE_PLUGIN_BRUSH("Icons/PolyGroups_40x",       Icon20x20));      
		StyleSet->Set("MeshTerrainModeManagerCommands.BeginDrawPolyPathTool",                  new IMAGE_PLUGIN_BRUSH("Icons/PolyPath_40x",         Icon20x20));    
		StyleSet->Set("MeshTerrainModeManagerCommands.BeginDrawPolyPathTool.Small",            new IMAGE_PLUGIN_BRUSH("Icons/PolyPath_40x",         Icon20x20));    
		StyleSet->Set("MeshTerrainModeManagerCommands.BeginDrawAndRevolveTool",                new IMAGE_BRUSH_SVG("Icons/ModelingDrawAndRevolve",  Icon20x20));
		StyleSet->Set("MeshTerrainModeManagerCommands.BeginDrawAndRevolveTool.Small",          new IMAGE_BRUSH_SVG("Icons/ModelingDrawAndRevolve",  Icon20x20));
		StyleSet->Set("MeshTerrainModeManagerCommands.BeginRevolveBoundaryTool",               new IMAGE_BRUSH_SVG("Icons/ModelingRevolveBoundary", Icon20x20));
		StyleSet->Set("MeshTerrainModeManagerCommands.BeginRevolveBoundaryTool.Small",         new IMAGE_BRUSH_SVG("Icons/ModelingRevolveBoundary", Icon20x20));
		StyleSet->Set("MeshTerrainModeManagerCommands.BeginRevolveSplineTool",                 new IMAGE_BRUSH_SVG("Icons/ModelingRevolveSpline",   Icon20x20));
		StyleSet->Set("MeshTerrainModeManagerCommands.BeginRevolveSplineTool.Small",           new IMAGE_BRUSH_SVG("Icons/ModelingRevolveSpline",   Icon20x20));
		StyleSet->Set("MeshTerrainModeManagerCommands.BeginTriangulateSplinesTool",            new IMAGE_BRUSH_SVG("Icons/ModelingTriangulateSpline",   Icon20x20));
		StyleSet->Set("MeshTerrainModeManagerCommands.BeginTriangulateSplinesTool.Small",      new IMAGE_BRUSH_SVG("Icons/ModelingTriangulateSpline",   Icon20x20));
		StyleSet->Set("MeshTerrainModeManagerCommands.BeginISMEditorTool",						new IMAGE_BRUSH_SVG("Icons/ModelingISMEditor",       Icon20x20));
		StyleSet->Set("MeshTerrainModeManagerCommands.BeginISMEditorTool.Small",				new IMAGE_BRUSH_SVG("Icons/ModelingISMEditor",       Icon20x20));
		StyleSet->Set("MeshTerrainModeManagerCommands.BeginCubeGridTool",                      new IMAGE_BRUSH_SVG("Icons/CubeGrid",                Icon20x20));
		StyleSet->Set("MeshTerrainModeManagerCommands.BeginCubeGridTool.Small",                new IMAGE_BRUSH_SVG("Icons/CubeGrid",                Icon20x20));
		StyleSet->Set("MeshTerrainModeManagerCommands.BeginMeshBooleanTool",                   new IMAGE_PLUGIN_BRUSH("Icons/ModelingMeshBoolean_x40", Icon20x20));
		StyleSet->Set("MeshTerrainModeManagerCommands.BeginMeshBooleanTool.Small",             new IMAGE_PLUGIN_BRUSH("Icons/ModelingMeshBoolean_x20", Icon20x20));
		StyleSet->Set("MeshTerrainModeManagerCommands.BeginMeshTrimTool",				 new IMAGE_BRUSH_SVG("Icons/ModelingTrim", Icon20x20));
		StyleSet->Set("MeshTerrainModeManagerCommands.BeginMeshTrimTool.Small",			 new IMAGE_BRUSH_SVG("Icons/ModelingTrim", Icon20x20));
		StyleSet->Set("MeshTerrainModeManagerCommands.BeginCutMeshWithMeshTool",               new IMAGE_BRUSH_SVG("Icons/ModelingMeshCut", Icon20x20));
		StyleSet->Set("MeshTerrainModeManagerCommands.BeginCutMeshWithMeshTool.Small",         new IMAGE_BRUSH_SVG("Icons/ModelingMeshCut", Icon20x20));
		StyleSet->Set("MeshTerrainModeManagerCommands.BeginSelfUnionTool",                     new IMAGE_PLUGIN_BRUSH("Icons/ModelingSelfUnion_x40", Icon20x20));
		StyleSet->Set("MeshTerrainModeManagerCommands.BeginSelfUnionTool.Small",               new IMAGE_PLUGIN_BRUSH("Icons/ModelingSelfUnion_x20", Icon20x20));
		StyleSet->Set("MeshTerrainModeManagerCommands.BeginVoxelSolidifyTool",                 new IMAGE_PLUGIN_BRUSH("Icons/ModelingVoxSolidify_x40", Icon20x20));
		StyleSet->Set("MeshTerrainModeManagerCommands.BeginVoxelSolidifyTool.Small",           new IMAGE_PLUGIN_BRUSH("Icons/ModelingVoxSolidify_x20", Icon20x20));
		StyleSet->Set("MeshTerrainModeManagerCommands.BeginVoxelBlendTool",                    new IMAGE_PLUGIN_BRUSH("Icons/ModelingVoxBlend_x40", Icon20x20));
		StyleSet->Set("MeshTerrainModeManagerCommands.BeginVoxelBlendTool.Small",              new IMAGE_PLUGIN_BRUSH("Icons/ModelingVoxBlend_x20", Icon20x20));
		StyleSet->Set("MeshTerrainModeManagerCommands.BeginVoxelMorphologyTool",               new IMAGE_PLUGIN_BRUSH("Icons/ModelingVoxMorphology_x40", Icon20x20));
		StyleSet->Set("MeshTerrainModeManagerCommands.BeginVoxelMorphologyTool.Small",         new IMAGE_PLUGIN_BRUSH("Icons/ModelingVoxMorphology_x20", Icon20x20));
		StyleSet->Set("MeshTerrainModeManagerCommands.BeginMeshSpaceDeformerTool",             new IMAGE_PLUGIN_BRUSH("Icons/SpaceDeform_40x",      Icon20x20));       
		StyleSet->Set("MeshTerrainModeManagerCommands.BeginMeshSpaceDeformerTool.Small",       new IMAGE_PLUGIN_BRUSH("Icons/SpaceDeform_40x",      Icon20x20));       
		StyleSet->Set("MeshTerrainModeManagerCommands.BeginMeshAttributePaintTool",             new IMAGE_PLUGIN_BRUSH("Icons/ModelingAttributePaint_x40",      Icon20x20));
		StyleSet->Set("MeshTerrainModeManagerCommands.BeginMeshAttributePaintTool.Small",       new IMAGE_PLUGIN_BRUSH("Icons/ModelingAttributePaint_x40",      Icon20x20));
		StyleSet->Set("MeshTerrainModeManagerCommands.BeginTransformUVIslandsTool",            new IMAGE_PLUGIN_BRUSH("Icons/TransformUVs_40x",     Icon20x20));         
		StyleSet->Set("MeshTerrainModeManagerCommands.BeginTransformUVIslandsTool.Small",      new IMAGE_PLUGIN_BRUSH("Icons/TransformUVs_40x",     Icon20x20));         
		StyleSet->Set("MeshTerrainModeManagerCommands.BeginUVLayoutTool",                      new IMAGE_PLUGIN_BRUSH("Icons/UVLayout_40x",         Icon20x20));    
		StyleSet->Set("MeshTerrainModeManagerCommands.BeginUVLayoutTool.Small",                new IMAGE_PLUGIN_BRUSH("Icons/UVLayout_40x",         Icon20x20));    
		StyleSet->Set("MeshTerrainModeManagerCommands.LaunchUVEditor",                         new IMAGE_BRUSH_SVG("Icons/UVEditor", Icon20x20));
		StyleSet->Set("MeshTerrainModeManagerCommands.LaunchUVEditor.Small",                   new IMAGE_BRUSH_SVG("Icons/UVEditor", Icon20x20));

		StyleSet->Set("MeshTerrainModeManagerCommands.BeginMeshGroupPaintTool", new IMAGE_BRUSH_SVG("Icons/GroupPaint", Icon20x20));
		StyleSet->Set("MeshTerrainModeManagerCommands.BeginMeshGroupPaintTool.Small", new IMAGE_BRUSH_SVG("Icons/GroupPaint", Icon20x20));
		StyleSet->Set("MeshTerrainModeManagerCommands.BeginLatticeDeformerTool", new IMAGE_BRUSH_SVG("Icons/LatticeDeformation", Icon20x20));
		StyleSet->Set("MeshTerrainModeManagerCommands.BeginLatticeDeformerTool.Small", new IMAGE_BRUSH_SVG("Icons/LatticeDeformation", Icon20x20));
		StyleSet->Set("MeshTerrainModeManagerCommands.BeginConvertMeshesTool", new IMAGE_BRUSH_SVG("Icons/Convert_20", Icon20x20));
		StyleSet->Set("MeshTerrainModeManagerCommands.BeginConvertMeshesTool.Small", new IMAGE_BRUSH_SVG("Icons/Convert_20", Icon20x20));
		StyleSet->Set("MeshTerrainModeManagerCommands.BeginSplitMeshesTool", new IMAGE_BRUSH_SVG("Icons/GeometrySplit", Icon20x20));
		StyleSet->Set("MeshTerrainModeManagerCommands.BeginSplitMeshesTool.Small", new IMAGE_BRUSH_SVG("Icons/GeometrySplit", Icon20x20));
		StyleSet->Set("MeshTerrainModeManagerCommands.BeginPatternTool", new IMAGE_BRUSH_SVG("Icons/ModelingPattern", Icon20x20));
		StyleSet->Set("MeshTerrainModeManagerCommands.BeginPatternTool.Small", new IMAGE_BRUSH_SVG("Icons/ModelingPattern", Icon20x20));
		StyleSet->Set("MeshTerrainModeManagerCommands.BeginHarvestInstancesTool", new IMAGE_BRUSH_SVG("Icons/HarvestInstances", Icon20x20));
		StyleSet->Set("MeshTerrainModeManagerCommands.BeginHarvestInstancesTool.Small", new IMAGE_BRUSH_SVG("Icons/HarvestInstances", Icon20x20));
		StyleSet->Set("MeshTerrainModeManagerCommands.BeginMeshVertexPaintTool", new IMAGE_BRUSH_SVG("Icons/PaintVertexColors", Icon20x20));
		StyleSet->Set("MeshTerrainModeManagerCommands.BeginMeshVertexPaintTool.Small", new IMAGE_BRUSH_SVG("Icons/PaintVertexColors", Icon20x20));


		StyleSet->Set("MeshTerrainModeManagerCommands.BeginVolumeToMeshTool",                  new IMAGE_PLUGIN_BRUSH("Icons/ModelingVol2Mesh_x40",         Icon20x20));
		StyleSet->Set("MeshTerrainModeManagerCommands.BeginVolumeToMeshTool.Small",            new IMAGE_PLUGIN_BRUSH("Icons/ModelingVol2Mesh_x40",         Icon20x20));
		StyleSet->Set("MeshTerrainModeManagerCommands.BeginMeshToVolumeTool",                  new IMAGE_PLUGIN_BRUSH("Icons/ModelingMesh2Vol_x40",         Icon20x20));
		StyleSet->Set("MeshTerrainModeManagerCommands.BeginMeshToVolumeTool.Small",            new IMAGE_PLUGIN_BRUSH("Icons/ModelingMesh2Vol_x40",         Icon20x20));
		StyleSet->Set("MeshTerrainModeManagerCommands.BeginBspConversionTool",                 new IMAGE_PLUGIN_BRUSH("Icons/ModelingBSPConversion_x40",         Icon20x20));
		StyleSet->Set("MeshTerrainModeManagerCommands.BeginBspConversionTool.Small",           new IMAGE_PLUGIN_BRUSH("Icons/ModelingBSPConversion_x40",         Icon20x20));
		StyleSet->Set("MeshTerrainModeManagerCommands.BeginPhysicsInspectorTool",              new IMAGE_BRUSH_SVG("Icons/InspectCollision",                     Icon20x20));
		StyleSet->Set("MeshTerrainModeManagerCommands.BeginPhysicsInspectorTool.Small",        new IMAGE_BRUSH_SVG("Icons/InspectCollision",                     Icon20x20));
		StyleSet->Set("MeshTerrainModeManagerCommands.BeginSetCollisionGeometryTool",          new IMAGE_PLUGIN_BRUSH("Icons/ModelingMeshToCollision_x40",         Icon20x20));
		StyleSet->Set("MeshTerrainModeManagerCommands.BeginSetCollisionGeometryTool.Small",    new IMAGE_PLUGIN_BRUSH("Icons/ModelingMeshToCollision_x40",         Icon20x20));
		StyleSet->Set("MeshTerrainModeManagerCommands.BeginExtractCollisionGeometryTool",      new IMAGE_PLUGIN_BRUSH("Icons/ModelingCollisionToMesh_x40",         Icon20x20));
		StyleSet->Set("MeshTerrainModeManagerCommands.BeginExtractCollisionGeometryTool.Small",new IMAGE_PLUGIN_BRUSH("Icons/ModelingCollisionToMesh_x40",         Icon20x20));
		StyleSet->Set("MeshTerrainModeManagerCommands.BeginSimpleCollisionEditorTool",         new IMAGE_BRUSH_SVG("Icons/SimpleCollisionEditor",                  Icon20x20));
		StyleSet->Set("MeshTerrainModeManagerCommands.BeginSimpleCollisionEditorTool.Small",   new IMAGE_BRUSH_SVG("Icons/SimpleCollisionEditor",                  Icon20x20));

		StyleSet->Set("MeshTerrainModeManagerCommands.BeginGenerateStaticMeshLODAssetTool", new IMAGE_BRUSH_SVG("Icons/AutoLOD", Icon20x20));
		StyleSet->Set("MeshTerrainModeManagerCommands.BeginGenerateStaticMeshLODAssetTool.Small", new IMAGE_BRUSH_SVG("Icons/AutoLOD", Icon20x20));
		StyleSet->Set("MeshTerrainModeManagerCommands.BeginLODManagerTool", new IMAGE_BRUSH_SVG("Icons/ModelingLODManager", Icon20x20));
		StyleSet->Set("MeshTerrainModeManagerCommands.BeginLODManagerTool.Small", new IMAGE_BRUSH_SVG("Icons/ModelingLODManager", Icon20x20));

		StyleSet->Set("MeshTerrainModeManagerCommands.BeginGroomCardsEditorTool", new IMAGE_BRUSH_SVG("Icons/CardsEditor", Icon20x20));
		StyleSet->Set("MeshTerrainModeManagerCommands.BeginGroomCardsEditorTool.Small", new IMAGE_BRUSH_SVG("Icons/CardsEditor", Icon20x20));
		StyleSet->Set("MeshTerrainModeManagerCommands.BeginGenerateLODMeshesTool", new IMAGE_BRUSH_SVG("Icons/GenLODs", Icon20x20));
		StyleSet->Set("MeshTerrainModeManagerCommands.BeginGenerateLODMeshesTool.Small", new IMAGE_BRUSH_SVG("Icons/GenLODs", Icon20x20));
		StyleSet->Set("MeshTerrainModeManagerCommands.BeginGroomToMeshTool", new IMAGE_BRUSH_SVG("Icons/HairHelmet", Icon20x20));
		StyleSet->Set("MeshTerrainModeManagerCommands.BeginGroomToMeshTool.Small", new IMAGE_BRUSH_SVG("Icons/HairHelmet", Icon20x20));
		
		StyleSet->Set("MeshTerrainModeManagerCommands.BeginSkeletonEditingTool", new IMAGE_BRUSH_SVG("Icons/SkeletalEditor_20", Icon20x20));
		StyleSet->Set("MeshTerrainModeManagerCommands.BeginSkinWeightsBindingTool", new IMAGE_BRUSH_SVG("Icons/BindSkin", Icon20x20));
		StyleSet->Set("MeshTerrainModeManagerCommands.BeginSkinWeightsPaintTool", new IMAGE_BRUSH_SVG("Icons/EditWeights", Icon20x20));
		StyleSet->Set("MeshTerrainModeManagerCommands.BeginSkinWeightsPaintTool.Small", new IMAGE_BRUSH_SVG("Icons/EditWeights", Icon20x20));

		StyleSet->Set("MeshTerrainModeManagerCommands.MeshSelectionModeAction_NoSelection", new IMAGE_BRUSH_SVG("Icons/ModSelectionObject_16", Icon20x20));
		StyleSet->Set("MeshTerrainModeManagerCommands.MeshSelectionModeAction_MeshTriangles", new IMAGE_BRUSH_SVG("Icons/ModSelectionPolys_16", Icon20x20));
		StyleSet->Set("MeshTerrainModeManagerCommands.MeshSelectionModeAction_MeshVertices", new IMAGE_BRUSH_SVG("Icons/ModSelectionVerts_16", Icon20x20));
		StyleSet->Set("MeshTerrainModeManagerCommands.MeshSelectionModeAction_MeshEdges", new IMAGE_BRUSH_SVG("Icons/ModSelectionEdges_16", Icon20x20));
		StyleSet->Set("MeshTerrainModeManagerCommands.MeshSelectionModeAction_GroupFaces", new IMAGE_BRUSH_SVG("Icons/ModSelectionPolygroupFaces_16", Icon20x20));
		StyleSet->Set("MeshTerrainModeManagerCommands.MeshSelectionModeAction_GroupCorners", new IMAGE_BRUSH_SVG("Icons/ModSelectionPolygroupVerts_16", Icon20x20));
		StyleSet->Set("MeshTerrainModeManagerCommands.MeshSelectionModeAction_GroupEdges", new IMAGE_BRUSH_SVG("Icons/ModSelectionPolygroupEdges_16", Icon20x20));

		StyleSet->Set("MeshTerrainModeManagerCommands.MeshElementSelection", new IMAGE_BRUSH_SVG("Icons/MeshSelect_16", Icon20x20));
		
		StyleSet->Set("MeshTerrainModeManagerCommands.BeginSelectionAction_SelectAll", new IMAGE_BRUSH_SVG("Icons/ModSelectionAll_16", Icon20x20));
		StyleSet->Set("MeshTerrainModeManagerCommands.BeginSelectionAction_Invert", new IMAGE_BRUSH_SVG("Icons/ModSelectionInverse_16", Icon20x20));
		StyleSet->Set("MeshTerrainModeManagerCommands.BeginSelectionAction_ExpandToConnected", new IMAGE_BRUSH_SVG("Icons/ModSelectionConnected_16", Icon20x20));
		StyleSet->Set("MeshTerrainModeManagerCommands.BeginSelectionAction_InvertConnected", new IMAGE_BRUSH_SVG("Icons/ModSelectionConnectedInverse_16", Icon20x20));
		StyleSet->Set("MeshTerrainModeManagerCommands.BeginSelectionAction_Expand", new IMAGE_BRUSH_SVG("Icons/ModSelectionExpand_16", Icon20x20));
		StyleSet->Set("MeshTerrainModeManagerCommands.BeginSelectionAction_Contract", new IMAGE_BRUSH_SVG("Icons/ModSelectionShrink_16", Icon20x20));

		StyleSet->Set("MeshTerrainModeManagerCommands.BeginSelectionAction_Delete", new IMAGE_BRUSH_SVG("Icons/Delete", Icon20x20));
		StyleSet->Set("MeshTerrainModeManagerCommands.BeginSelectionAction_Delete.Small", new IMAGE_BRUSH_SVG("Icons/Delete", Icon20x20));
		StyleSet->Set("MeshTerrainModeManagerCommands.BeginSelectionAction_Extrude", new IMAGE_BRUSH_SVG("Icons/Extrude", Icon20x20));
		StyleSet->Set("MeshTerrainModeManagerCommands.BeginSelectionAction_Extrude.Small", new IMAGE_BRUSH_SVG("Icons/Extrude", Icon20x20));
		StyleSet->Set("MeshTerrainModeManagerCommands.BeginSelectionAction_Offset", new IMAGE_BRUSH_SVG("Icons/Offset", Icon20x20));
		StyleSet->Set("MeshTerrainModeManagerCommands.BeginSelectionAction_Offset.Small", new IMAGE_BRUSH_SVG("Icons/Offset", Icon20x20));
		StyleSet->Set("MeshTerrainModeManagerCommands.BeginPolyModelTool_PolyEd", new IMAGE_BRUSH_SVG("Icons/PolyEdit", Icon20x20));
		StyleSet->Set("MeshTerrainModeManagerCommands.BeginPolyModelTool_PolyEd.Small", new IMAGE_BRUSH_SVG("Icons/PolyEdit",	Icon20x20));
		StyleSet->Set("MeshTerrainModeManagerCommands.BeginPolyModelTool_TriSel", new IMAGE_BRUSH_SVG("Icons/TriSelect", Icon20x20));
		StyleSet->Set("MeshTerrainModeManagerCommands.BeginPolyModelTool_TriSel.Small", new IMAGE_BRUSH_SVG("Icons/TriSelect", Icon20x20));
		StyleSet->Set("MeshTerrainModeManagerCommands.BeginPolyModelTool_Inset", new IMAGE_BRUSH_SVG("Icons/Inset", Icon20x20));
		StyleSet->Set("MeshTerrainModeManagerCommands.BeginPolyModelTool_Inset.Small", new IMAGE_BRUSH_SVG("Icons/Inset", Icon20x20));
		StyleSet->Set("MeshTerrainModeManagerCommands.BeginPolyModelTool_Outset", new IMAGE_BRUSH_SVG("Icons/Outset", Icon20x20));
		StyleSet->Set("MeshTerrainModeManagerCommands.BeginPolyModelTool_Outset.Small", new IMAGE_BRUSH_SVG("Icons/Outset", Icon20x20));
		StyleSet->Set("MeshTerrainModeManagerCommands.BeginPolyModelTool_CutFaces", new IMAGE_BRUSH_SVG("Icons/CutFaces", Icon20x20));
		StyleSet->Set("MeshTerrainModeManagerCommands.BeginPolyModelTool_CutFaces.Small", new IMAGE_BRUSH_SVG("Icons/CutFaces", Icon20x20));
		StyleSet->Set("MeshTerrainModeManagerCommands.BeginPolyModelTool_InsertEdgeLoop", new IMAGE_BRUSH_SVG("Icons/EdgeLoop", Icon20x20));
		StyleSet->Set("MeshTerrainModeManagerCommands.BeginPolyModelTool_InsertEdgeLoop.Small", new IMAGE_BRUSH_SVG("Icons/EdgeLoop", Icon20x20));
		StyleSet->Set("MeshTerrainModeManagerCommands.BeginPolyModelTool_ExtrudeEdges", new IMAGE_BRUSH_SVG("Icons/ExtrudeEdge", Icon20x20));
		StyleSet->Set("MeshTerrainModeManagerCommands.BeginPolyModelTool_ExtrudeEdges.Small", new IMAGE_BRUSH_SVG("Icons/ExtrudeEdge", Icon20x20));
		StyleSet->Set("MeshTerrainModeManagerCommands.BeginPolyModelTool_PushPull", new IMAGE_BRUSH_SVG("Icons/PushPull", Icon20x20));
		StyleSet->Set("MeshTerrainModeManagerCommands.BeginPolyModelTool_PushPull.Small", new IMAGE_BRUSH_SVG("Icons/PushPull", Icon20x20));
		StyleSet->Set("MeshTerrainModeManagerCommands.BeginPolyModelTool_Bevel", new IMAGE_BRUSH_SVG("Icons/Bevel", Icon20x20));
		StyleSet->Set("MeshTerrainModeManagerCommands.BeginPolyModelTool_Bevel.Small", new IMAGE_BRUSH_SVG("Icons/Bevel", Icon20x20));
		StyleSet->Set("MeshTerrainModeManagerCommands.BeginSelectionAction_Retriangulate", new IMAGE_BRUSH_SVG("Icons/Clean", Icon20x20));
		StyleSet->Set("MeshTerrainModeManagerCommands.BeginSelectionAction_Retriangulate.Small", new IMAGE_BRUSH_SVG("Icons/Clean", Icon20x20));

		StyleSet->Set("MeshTerrainModeSelection.More_Right",  new IMAGE_BRUSH_SVG("Icons/SelectionToolbar_More", Icon20x20));
		StyleSet->Set("MeshTerrainModeSelection.Edits_Right",  new IMAGE_BRUSH_SVG("Icons/SelectionToolbar_Edits", Icon20x20));

		//
		// icons for MegaMesh tools
		//
		StyleSet->Set("MeshTerrainModeManagerCommands.BeginCreateMegaMeshRectangleTool",	new IMAGE_PLUGIN_BRUSH( "Icons/ModelingRectangle_x40", Icon40x40));
		StyleSet->Set("MeshTerrainModeManagerCommands.BeginCreateMegaMeshRectangleTool.Small",	new IMAGE_PLUGIN_BRUSH( "Icons/ModelingRectangle_x40", Icon20x20));
		StyleSet->Set("MeshTerrainModeManagerCommands.BeginConvertMegaMeshTool",				new IMAGE_BRUSH_SVG("Icons/ConvertMesh", Icon40x40));
		StyleSet->Set("MeshTerrainModeManagerCommands.BeginConvertMegaMeshTool.Small",		new IMAGE_BRUSH_SVG("Icons/ConvertMesh", Icon20x20));
		StyleSet->Set("MeshTerrainModeManagerCommands.BeginSplitMegaMeshTool",				new IMAGE_PLUGIN_BRUSH("Icons/PlaneCut_40x", Icon40x40));
		StyleSet->Set("MeshTerrainModeManagerCommands.BeginSplitMegaMeshTool.Small",			new IMAGE_PLUGIN_BRUSH("Icons/PlaneCut_40x", Icon20x20));
		StyleSet->Set("MeshTerrainModeManagerCommands.BeginMergeMegaMeshTool",				new IMAGE_PLUGIN_BRUSH("Icons/VoxMerge_40x", Icon40x40));
		StyleSet->Set("MeshTerrainModeManagerCommands.BeginMergeMegaMeshTool.Small",			new IMAGE_PLUGIN_BRUSH("Icons/VoxMerge_40x", Icon20x20));
		StyleSet->Set("MeshTerrainModeManagerCommands.BeginResectionMeshTool",				new IMAGE_PLUGIN_BRUSH("Icons/VoxMerge_40x", Icon40x40));
		StyleSet->Set("MeshTerrainModeManagerCommands.BeginResectionMeshTool.Small",			new IMAGE_PLUGIN_BRUSH("Icons/VoxMerge_40x", Icon20x20));
		StyleSet->Set("MeshTerrainModeManagerCommands.BeginExpandMegaMeshTool",				new IMAGE_BRUSH_SVG("Icons/ExtrudeEdge", Icon40x40));
		StyleSet->Set("MeshTerrainModeManagerCommands.BeginExpandMegaMeshTool.Small",         new IMAGE_BRUSH_SVG("Icons/ExtrudeEdge", Icon20x20));
		StyleSet->Set("MeshTerrainModeManagerCommands.BeginStitchMegaMeshTool",				new IMAGE_PLUGIN_BRUSH("Icons/StitchMesh_40x", Icon40x40));
		StyleSet->Set("MeshTerrainModeManagerCommands.BeginStitchMegaMeshTool.Small",			new IMAGE_PLUGIN_BRUSH("Icons/StitchMesh_40x", Icon20x20));
		StyleSet->Set("MeshTerrainModeManagerCommands.BeginHeightmapImport",				new IMAGE_BRUSH_SVG("Icons/HeightmapImport", Icon40x40));
		StyleSet->Set("MeshTerrainModeManagerCommands.BeginHeightmapImport.Small",		new IMAGE_BRUSH_SVG("Icons/HeightmapImport", Icon20x20));
		StyleSet->Set("MeshTerrainModeManagerCommands.BeginHeightSculptTool",		new IMAGE_PLUGIN_BRUSH("Icons/HeightSculpt_40x", Icon40x40));
		StyleSet->Set("MeshTerrainModeManagerCommands.BeginHeightSculptTool.Small",	new IMAGE_PLUGIN_BRUSH("Icons/HeightSculpt_40x", Icon20x20));
		StyleSet->Set("MeshTerrainModeManagerCommands.BeginAddModifierTool",				new IMAGE_PLUGIN_BRUSH("Icons/Displace_40x", Icon40x40));
		StyleSet->Set("MeshTerrainModeManagerCommands.BeginAddModifierTool.Small",		new IMAGE_PLUGIN_BRUSH("Icons/Displace_40x", Icon20x20));

		/** Icons for the Details View Tabs */
		
		StyleSet->Set("SectionHeader.Position",		new IMAGE_BRUSH_SVG("Icons/DetailsViewTabIcons/PagingPosition_16", Icon16x16));
		StyleSet->Set("SectionHeader.Shape",			new IMAGE_BRUSH_SVG("Icons/DetailsViewTabIcons/PagingShape_16", Icon16x16));
		StyleSet->Set("SectionHeader.Materials",		new IMAGE_BRUSH_SVG("Icons/DetailsViewTabIcons/PagingMaterial_16", Icon16x16));
		StyleSet->Set("SectionHeader.GeneralOptions",	new IMAGE_BRUSH_SVG("Icons/DetailsViewTabIcons/PagingGeneralOptions_16", Icon16x16));
		StyleSet->Set("SectionHeader.ALL_SECTIONS",			new IMAGE_BRUSH_SVG("Icons/DetailsViewTabIcons/PagingAll_16", Icon16x16));
		StyleSet->Set("SectionHeader.Pattern",		new IMAGE_BRUSH_SVG("Icons/DetailsViewTabIcons/PagingPattern_16", Icon16x16));
		StyleSet->Set("SectionHeader.Spline",		new IMAGE_BRUSH_SVG("Icons/DetailsViewTabIcons/PagingSpline_16", Icon16x16));
		StyleSet->Set("SectionHeader.Output",		new IMAGE_BRUSH_SVG("Icons/DetailsViewTabIcons/PagingOutput_16", Icon16x16));
		StyleSet->Set("SectionHeader.Raycast",		new IMAGE_BRUSH_SVG("Icons/DetailsViewTabIcons/PagingRaycast_16", Icon16x16));
		StyleSet->Set("SectionHeader.Inspect",		new IMAGE_BRUSH_SVG("Icons/DetailsViewTabIcons/PagingInspect_16", Icon16x16));
		StyleSet->Set("SectionHeader.Normals",		new IMAGE_BRUSH_SVG("Icons/DetailsViewTabIcons/PagingNormals_16", Icon16x16));
		StyleSet->Set("SectionHeader.UVs",			new IMAGE_BRUSH_SVG("Icons/DetailsViewTabIcons/PagingUVs_16", Icon16x16));
		StyleSet->Set("SectionHeader.Snapping",		new IMAGE_BRUSH_SVG("Icons/DetailsViewTabIcons/PagingSnapping_16", Icon16x16));
		StyleSet->Set("SectionHeader.Brush",			new IMAGE_BRUSH_SVG("Icons/DetailsViewTabIcons/PagingBrush_16", Icon16x16));
		StyleSet->Set("SectionHeader.Rendering",		new IMAGE_BRUSH_SVG("Icons/DetailsViewTabIcons/PagingRendering_16", Icon16x16));
		StyleSet->Set("SectionHeader.Sculpting",		new IMAGE_BRUSH_SVG("Icons/DetailsViewTabIcons/PagingSculpting_16", Icon16x16));
		StyleSet->Set("SectionHeader.Filters",		new IMAGE_BRUSH_SVG("Icons/DetailsViewTabIcons/PagingFilters_16", Icon16x16));
		StyleSet->Set("SectionHeader.Operations",	new IMAGE_BRUSH_SVG("Icons/DetailsViewTabIcons/PagingActions_16", Icon16x16));
		StyleSet->Set("SectionHeader.UtilityOps",	new IMAGE_BRUSH_SVG("Icons/DetailsViewTabIcons/PagingUtilityOps_16", Icon16x16));
		StyleSet->Set("SectionHeader.Extrude",		new IMAGE_BRUSH_SVG("Icons/DetailsViewTabIcons/PagingExtrude_16", Icon16x16));
		StyleSet->Set("SectionHeader.Visualization",	new IMAGE_BRUSH_SVG("Icons/DetailsViewTabIcons/PagingVisualization_16", Icon16x16));
		StyleSet->Set("SectionHeader.Import",		new IMAGE_BRUSH_SVG("Icons/DetailsViewTabIcons/PagingImport_16", Icon16x16));
		StyleSet->Set("SectionHeader.Attributes",	new IMAGE_BRUSH_SVG("Icons/DetailsViewTabIcons/PagingAttribute_16", Icon16x16));
		StyleSet->Set("SectionHeader.AttributeInspector",	new IMAGE_BRUSH_SVG("Icons/DetailsViewTabIcons/PagingAttributeInspector_16", Icon16x16));
		StyleSet->Set("SectionHeader.Polygroups",	new IMAGE_BRUSH_SVG("Icons/DetailsViewTabIcons/PagingPolygroups_16", Icon16x16));
		StyleSet->Set("SectionHeader.Lattice",		new IMAGE_BRUSH_SVG("Icons/DetailsViewTabIcons/PagingLattice_16", Icon16x16));
		StyleSet->Set("SectionHeader.Tessellate",	new IMAGE_BRUSH_SVG("Icons/DetailsViewTabIcons/PagingTessellation_16", Icon16x16));

		/** Tool Shutdown Icons */
		StyleSet->Set("ToolShutdown.Accept", new IMAGE_BRUSH_SVG("Icons/Check_20", Icon20x20, FStyleColors::Success));
		StyleSet->Set("ToolShutdown.AcceptDisabled", new IMAGE_BRUSH_SVG("Icons/CheckDisabled_20", Icon20x20, FStyleColors::White));
		StyleSet->Set("ToolShutdown.Cancel", new IMAGE_BRUSH_SVG("Icons/Close_20", Icon20x20, FStyleColors::Error));
		StyleSet->Set("ToolShutdown.Close", new IMAGE_BRUSH_SVG("Icons/Close_20", Icon20x20));
		
		/** Add Modifier Tool - Per Modifier */
		StyleSet->Set("MeshTerrainModeManagerCommands.BeginAddRemeshModifierTool",					new IMAGE_BRUSH_SVG("Icons/ModifierRemesh_16", Icon40x40));
		StyleSet->Set("MeshTerrainModeManagerCommands.BeginAddRemeshModifierTool.Small",			new IMAGE_BRUSH_SVG("Icons/ModifierRemesh_16", Icon20x20));
		StyleSet->Set("MeshTerrainModeManagerCommands.BeginAddPatchModifierTool",					new IMAGE_BRUSH_SVG("Icons/ModifierPatch_16", Icon40x40));
		StyleSet->Set("MeshTerrainModeManagerCommands.BeginAddPatchModifierTool.Small",			new IMAGE_BRUSH_SVG("Icons/ModifierPatch_16", Icon20x20));
		StyleSet->Set("MeshTerrainModeManagerCommands.BeginAddProjectModifierTool",				new IMAGE_BRUSH_SVG("Icons/ModifierProjection_16", Icon40x40));
		StyleSet->Set("MeshTerrainModeManagerCommands.BeginAddProjectModifierTool.Small",			new IMAGE_BRUSH_SVG("Icons/ModifierProjection_16", Icon20x20));
		StyleSet->Set("MeshTerrainModeManagerCommands.BeginAddInstancedPatchModifierTool",			new IMAGE_BRUSH_SVG("Icons/ModifierPatchInstance_16", Icon40x40));
		StyleSet->Set("MeshTerrainModeManagerCommands.BeginAddInstancedPatchModifierTool.Small",	new IMAGE_BRUSH_SVG("Icons/ModifierPatchInstance_16", Icon20x20));
		StyleSet->Set("MeshTerrainModeManagerCommands.BeginAddTexturePatchModifierTool",			new IMAGE_BRUSH_SVG("Icons/ModifierTexturePatch_16", Icon40x40));
		StyleSet->Set("MeshTerrainModeManagerCommands.BeginAddTexturePatchModifierTool.Small",		new IMAGE_BRUSH_SVG("Icons/ModifierTexturePatch_16", Icon20x20));
		StyleSet->Set("MeshTerrainModeManagerCommands.BeginAddSplineModifierTool",					new IMAGE_BRUSH_SVG("Icons/ModifierPath_16", Icon40x40));
		StyleSet->Set("MeshTerrainModeManagerCommands.BeginAddSplineModifierTool.Small",			new IMAGE_BRUSH_SVG("Icons/ModifierPath_16", Icon20x20));
		StyleSet->Set("MeshTerrainModeManagerCommands.BeginAddMeshLayerModifierTool",				new IMAGE_BRUSH_SVG("Icons/ModifierSculpt_16", Icon40x40));
		StyleSet->Set("MeshTerrainModeManagerCommands.BeginAddMeshLayerModifierTool.Small",		new IMAGE_BRUSH_SVG("Icons/ModifierSculpt_16", Icon20x20));
		StyleSet->Set("MeshTerrainModeManagerCommands.BeginAddNoiseModifierTool",					new IMAGE_BRUSH_SVG("Icons/ModifierNoise_16", Icon40x40));
		StyleSet->Set("MeshTerrainModeManagerCommands.BeginAddNoiseModifierTool.Small",			new IMAGE_BRUSH_SVG("Icons/ModifierNoise_16", Icon20x20));
		StyleSet->Set("MeshTerrainModeManagerCommands.BeginAddBooleanModifierTool",				new IMAGE_BRUSH_SVG("Icons/ModifierBoolean_16", Icon40x40));
		StyleSet->Set("MeshTerrainModeManagerCommands.BeginAddBooleanModifierTool.Small",			new IMAGE_BRUSH_SVG("Icons/ModifierBoolean_16", Icon20x20));
		StyleSet->Set("MeshTerrainModeManagerCommands.BeginAddLatticeModifierTool",				new IMAGE_BRUSH_SVG("Icons/ModifierLattice_16", Icon40x40));
		StyleSet->Set("MeshTerrainModeManagerCommands.BeginAddLatticeModifierTool.Small",			new IMAGE_BRUSH_SVG("Icons/ModifierLattice_16", Icon20x20));
		StyleSet->Set("MeshTerrainModeManagerCommands.BeginAddSplineRemeshModifierTool",			new IMAGE_BRUSH_SVG("Icons/ModifierPath_16", Icon40x40));
		StyleSet->Set("MeshTerrainModeManagerCommands.BeginAddSplineRemeshModifierTool.Small",		new IMAGE_BRUSH_SVG("Icons/ModifierPath_16", Icon20x20));

		constexpr int HeightSculptColorIdx = 0;
		const FLinearColor HeightSculptColor = LinearColors::SelectColor<FLinearColor>(HeightSculptColorIdx);
		StyleSet->Set("MeshTerrainModeManagerCommands.BeginHeightSculptBrushTool",			new IMAGE_BRUSH_SVG( "Icons/HeightSculptBrush", Icon40x40, HeightSculptColor));
		StyleSet->Set("MeshTerrainModeManagerCommands.BeginHeightSculptBrushTool.Small",		new IMAGE_BRUSH_SVG( "Icons/HeightSculptBrush", Icon20x20, HeightSculptColor));
		StyleSet->Set("MeshTerrainModeManagerCommands.BeginHeightSmoothBrushTool",			new IMAGE_BRUSH_SVG( "Icons/HeightSmoothBrush", Icon40x40, HeightSculptColor));
		StyleSet->Set("MeshTerrainModeManagerCommands.BeginHeightSmoothBrushTool.Small",		new IMAGE_BRUSH_SVG( "Icons/HeightSmoothBrush", Icon20x20, HeightSculptColor));
		StyleSet->Set("MeshTerrainModeManagerCommands.BeginHeightFlattenBrushTool",			new IMAGE_BRUSH_SVG( "Icons/HeightFlattenBrush", Icon40x40, HeightSculptColor));
		StyleSet->Set("MeshTerrainModeManagerCommands.BeginHeightFlattenBrushTool.Small",	new IMAGE_BRUSH_SVG( "Icons/HeightFlattenBrush", Icon20x20, HeightSculptColor));
		StyleSet->Set("MeshTerrainModeManagerCommands.BeginSlopeErodeBrushTool",			new IMAGE_BRUSH_SVG("Icons/HeightErosionBrush", Icon40x40, HeightSculptColor));
		StyleSet->Set("MeshTerrainModeManagerCommands.BeginSlopeErodeBrushTool.Small",		new IMAGE_BRUSH_SVG("Icons/HeightErosionBrush", Icon20x20, HeightSculptColor));

		// Pressure Sensitivity toggle button
		StyleSet->Set("BrushIcons.PressureSensitivity", new IMAGE_BRUSH_SVG("Icons/PressureSensitivity", Icon20x20));
		StyleSet->Set("BrushIcons.PressureSensitivity.Small", new IMAGE_BRUSH_SVG("Icons/PressureSensitivity", Icon16x16));

		// icon for world/adaptive toggle for BrushSize
		StyleSet->Set("QuickSettings.RelativeCoordinateSystem_World", new IMAGE_BRUSH_SVG("Icons/Globe_20", Icon20x20));

		//
		// icons and style for the mesh selection toolbar
		//
		StyleSet->Set("SelectionToolBarIcons.LockedTarget", new IMAGE_BRUSH_SVG("Icons/lock-red", Icon16x16));
		StyleSet->Set("SelectionToolBarIcons.UnlockedTarget", new IMAGE_BRUSH_SVG("Icons/lock-unlocked-green", Icon16x16));

		FToolBarStyle SelectionToolbarStyle = FAppStyle::Get().GetWidgetStyle<FToolBarStyle>("EditorViewportToolBar");
		StyleSet->Set("SelectionToolBar", SelectionToolbarStyle);

		// override red-button style
		const FButtonStyle SelectionToolbarRedButton = FButtonStyle(SelectionToolbarStyle.ButtonStyle)
			.SetNormal(FSlateRoundedBoxBrush(FStyleColors::AccentRed, 12.f, FLinearColor(0, 0, 0, .8f), 1.0))
			.SetPressed(FSlateRoundedBoxBrush(FStyleColors::AccentRed, 12.f, FLinearColor(0, 0, 0, .8f), 1.0))
			.SetHovered(FSlateRoundedBoxBrush(FStyleColors::AccentRed, 12.f, FLinearColor(0, 0, 0, .8f), 1.0))
			.SetDisabled(FSlateRoundedBoxBrush(FStyleColors::SelectInactive, 12.f, FLinearColor(0, 0, 0, .8f), 1.0))
			.SetNormalForeground(FSlateColor::UseForeground())
			.SetPressedForeground(FSlateColor::UseForeground())
			.SetHoveredForeground(FSlateColor::UseForeground());
		SelectionToolbarStyle.SetButtonStyle(SelectionToolbarRedButton);
		StyleSet->Set("SelectionToolBar.RedButton", SelectionToolbarStyle);

		// override green-button style
		const FButtonStyle SelectionToolbarGreenButton = FButtonStyle(SelectionToolbarStyle.ButtonStyle)
			.SetNormal(FSlateRoundedBoxBrush(FStyleColors::AccentGreen, 12.f, FLinearColor(0, 0, 0, .8f), 1.0))
			.SetPressed(FSlateRoundedBoxBrush(FStyleColors::AccentGreen, 12.f, FLinearColor(0, 0, 0, .8f), 1.0))
			.SetHovered(FSlateRoundedBoxBrush(FStyleColors::AccentGreen, 12.f, FLinearColor(0, 0, 0, .8f), 1.0))
			.SetDisabled(FSlateRoundedBoxBrush(FStyleColors::SelectInactive, 12.f, FLinearColor(0, 0, 0, .8f), 1.0))
			.SetNormalForeground(FSlateColor::UseForeground())
			.SetPressedForeground(FSlateColor::UseForeground())
			.SetHoveredForeground(FSlateColor::UseForeground());
		SelectionToolbarStyle.SetButtonStyle(SelectionToolbarGreenButton);
		StyleSet->Set("SelectionToolBar.GreenButton", SelectionToolbarStyle);

		//
		// Submode toolbar styling
		//
		static const FSlateColor SubmodeBackgroundColor = FSlateColor(EStyleColor::Background).GetSpecifiedColor() * FLinearColor(1.f, 1.f, 1.f, 0.25f);
		static const FSlateColor SubmodeBorderColor = FSlateColor(EStyleColor::Background);
		static const FSlateRoundedBoxBrush SubmodeBackground(SubmodeBackgroundColor, 8.f, SubmodeBorderColor, 0.f);
		
		FToolBarStyle SubmodePaletteStyle = FAppStyle::Get().GetWidgetStyle<FToolBarStyle>("PaletteToolBar");
		SubmodePaletteStyle.SetBackground(SubmodeBackground);
		SubmodePaletteStyle.SetShowLabels(true);
		SubmodePaletteStyle.SetBackgroundPadding( FMargin( 4 ) );
		//SubmodePaletteStyle.SetIconSize(FVector2D(60.0f, 60.0f));
		SubmodePaletteStyle.SetIconPadding( FMargin(0, 4, 0, 4 ) );
		SubmodePaletteStyle.SetIconPaddingWithVisibleLabel( FMargin(4) );
		SubmodePaletteStyle.SetLabelPadding( FMargin(0, 0, 0, 0) );
		FTextBlockStyle LabelTextStyle = FTextBlockStyle(NormalText)
										 .SetOverflowPolicy(ETextOverflowPolicy::Ellipsis)
										 .SetFont(DEFAULT_FONT("roboto", FCoreStyle::SmallTextSize));
		SubmodePaletteStyle.SetLabelStyle( LabelTextStyle );
		SubmodePaletteStyle.SetButtonPadding( FMargin( 0, 0, 0, 0 ) );
		StyleSet->Set("SubmodeToolBar.SubmodePalette", SubmodePaletteStyle);

		FToolBarStyle SubmodeToolPaletteStyle = FAppStyle::Get().GetWidgetStyle<FToolBarStyle>("PaletteToolBar");
		SubmodeToolPaletteStyle.SetBackground(SubmodeBackground);
		SubmodeToolPaletteStyle.SetShowLabels(false);
		SubmodeToolPaletteStyle.SetBackgroundPadding(FMargin(0, 0));
		SubmodeToolPaletteStyle.SetIconSize(FVector2D(20.0f, 20.0f));
		SubmodeToolPaletteStyle.SetIconPadding(FMargin(0, 0, 0, 0 ));
		SubmodeToolPaletteStyle.SetIconPaddingWithVisibleLabel( FMargin(0, 0, 0, 0 ) );
		SubmodeToolPaletteStyle.SetLabelPadding(FMargin(0, 0));
		//SubmodeToolPaletteStyle.SetCheckBoxPadding( FMargin(0, 0) );
		//SubmodeToolPaletteStyle.SetBlockPadding( FMargin(0, 0) );
		//SubmodeToolPaletteStyle.SetIndentedBlockPadding( FMargin(0, 0) );
		//SubmodePaletteStyle.SetButtonPadding( FMargin( 0, 0, 0, 0 ) );
		StyleSet->Set("SubmodeToolBar.ToolPalette", SubmodeToolPaletteStyle);
		StyleSet->Set("SubmodeToolBar.MenuIndicator", new FSlateVectorImageBrush(StyleSet->RootToCoreContentDir(TEXT("Starship/Common/caret-down.svg")), Icon8x8));

		/** Toggle Button on a dark (EStyleColor::Panel) background. */
		FCheckBoxStyle DarkBackgroundToggleButton = FAppStyle::Get().GetWidgetStyle<FCheckBoxStyle>("ToggleButtonCheckbox");
		DarkBackgroundToggleButton.SetCheckedPressedImage(FSlateRoundedBoxBrush(FStyleColors::Primary, 4.0f));
		DarkBackgroundToggleButton.SetUncheckedPressedImage(FSlateRoundedBoxBrush(FStyleColors::Header, 4.0f));
		StyleSet->Set("ToolPaletteToggleButton", DarkBackgroundToggleButton);

		/** Toggle button on a light (EStyleColor::Dropdown) background. */
		FCheckBoxStyle LightBackgroundToggleButton = FAppStyle::Get().GetWidgetStyle<FCheckBoxStyle>("ToggleButtonCheckbox");
		LightBackgroundToggleButton.SetUncheckedHoveredImage(FSlateRoundedBoxBrush(FStyleColors::Hover, 4.0f));
		LightBackgroundToggleButton.SetUncheckedPressedImage(FSlateRoundedBoxBrush(FStyleColors::DropdownOutline, 4.0f));
		LightBackgroundToggleButton.SetHoveredForegroundColor(FStyleColors::ForegroundHover);
		LightBackgroundToggleButton.SetPressedForegroundColor(FStyleColors::ForegroundHover);
		LightBackgroundToggleButton.SetCheckedPressedImage(FSlateRoundedBoxBrush(FStyleColors::Primary, 4.0f));
		StyleSet->Set("SubmodePaletteToggleButton", LightBackgroundToggleButton);

		/** Toggle button for DetailsView tabs */
		StyleSet->Set("DetailsViewToggleButton", LightBackgroundToggleButton);

		/** Large toggle button for QuickSettings bar */
		FCheckBoxStyle QuickSettingsLargeButton = FAppStyle::Get().GetWidgetStyle<FCheckBoxStyle>("ToggleButtonCheckbox");
		QuickSettingsLargeButton.SetUncheckedImage(FSlateRoundedBoxBrush(FStyleColors::Dropdown, 4.0f));
		QuickSettingsLargeButton.SetUncheckedHoveredImage(FSlateRoundedBoxBrush(FStyleColors::Hover, 4.0f));
		QuickSettingsLargeButton.SetUncheckedPressedImage(FSlateRoundedBoxBrush(FStyleColors::Dropdown, 4.0f));
		StyleSet->Set("QuickSettingsLargeButton", QuickSettingsLargeButton);

		/** Quick Settings Toggle Button (Icon only, will tint icon blue when active) */
		FCheckBoxStyle IconHighlightOnlyToggleButton = FAppStyle::Get().GetWidgetStyle<FCheckBoxStyle>("ToggleButtonCheckbox");
		IconHighlightOnlyToggleButton.SetCheckedImage(FSlateNoResource());
		IconHighlightOnlyToggleButton.SetCheckedHoveredImage(FSlateRoundedBoxBrush(FStyleColors::Dropdown, 4.0f));
		IconHighlightOnlyToggleButton.SetCheckedPressedImage(FSlateRoundedBoxBrush(FStyleColors::Header, 4.0f));
		IconHighlightOnlyToggleButton.SetUncheckedPressedImage(FSlateRoundedBoxBrush(FStyleColors::Header, 4.0f));
		IconHighlightOnlyToggleButton.SetForegroundColor(FStyleColors::Foreground);
		IconHighlightOnlyToggleButton.SetHoveredForegroundColor(FStyleColors::ForegroundHover);
		IconHighlightOnlyToggleButton.SetPressedForegroundColor(FStyleColors::Foreground);
		IconHighlightOnlyToggleButton.SetCheckedForegroundColor(FStyleColors::AccentBlue);
		IconHighlightOnlyToggleButton.SetCheckedHoveredForegroundColor(FStyleColors::AccentBlue);
		IconHighlightOnlyToggleButton.SetCheckedPressedForegroundColor(FStyleColors::AccentBlue);
		StyleSet->Set("QuickSettingsToggleButton", IconHighlightOnlyToggleButton);

		/** Combo Button with smaller R/L margins. */
		FComboButtonStyle SmallMarginComboButton = FComboButtonStyle(FAppStyle::Get().GetWidgetStyle<FComboButtonStyle>("ComboButton"));
		SmallMarginComboButton.ButtonStyle.NormalPadding.Left = 4.f;
		SmallMarginComboButton.ButtonStyle.NormalPadding.Right = 0.f;
		SmallMarginComboButton.ButtonStyle.PressedPadding.Left = 4.f;
		SmallMarginComboButton.ButtonStyle.PressedPadding.Right = 0.f;
		StyleSet->Set("QuickProperties.SmallMarginComboButton", SmallMarginComboButton);

		/* Round Button for QuickProperties icon-only buttons. */
		FCheckBoxStyle QuickPropertiesRoundButton = FCheckBoxStyle(FAppStyle::Get().GetWidgetStyle<FCheckBoxStyle>("DetailsView.SectionButton"))
		.SetUncheckedImage(FSlateRoundedBoxBrush(FStyleColors::Header, 11.0f, FStyleColors::Input, 1.0f))
		.SetUncheckedHoveredImage(FSlateRoundedBoxBrush(FStyleColors::Hover, 11.0f, FStyleColors::Input, 1.0f))
		.SetUncheckedPressedImage(FSlateRoundedBoxBrush(FStyleColors::Hover, 11.0f, FStyleColors::Input, 1.0f))
		.SetCheckedImage(FSlateRoundedBoxBrush(FStyleColors::Primary, 11.0f, FStyleColors::Input, 1.0f))
		.SetCheckedHoveredImage(FSlateRoundedBoxBrush(FStyleColors::PrimaryHover, 11.0f, FStyleColors::Input, 1.0f))
		.SetCheckedPressedImage(FSlateRoundedBoxBrush(FStyleColors::PrimaryHover, 11.0f, FStyleColors::Input, 1.0f))
		.SetPadding(FMargin(16, 4));
		StyleSet->Set("QuickProperties.RoundButton", QuickPropertiesRoundButton);

		/* Tab/Section buttons for the details view widget */
		FCheckBoxStyle DetailsViewTabButtons = FAppStyle::Get().GetWidgetStyle<FCheckBoxStyle>("ToggleButtonCheckbox");
		DetailsViewTabButtons.SetCheckedPressedImage(FSlateRoundedBoxBrush(FStyleColors::Primary, 4.0f));
		DetailsViewTabButtons.SetPadding(FMargin(4.0f,2.0f));
		StyleSet->Set("DetailsViewTabButton", DetailsViewTabButtons);
		
		//
		// Icons for brush falloffs in sculpt/etc tools
		//

		StyleSet->Set("BrushFalloffIcons.Smooth", new IMAGE_BRUSH_SVG("Icons/BrushIcons/Falloff_Smooth", Icon120));
		StyleSet->Set("BrushFalloffIcons.Linear", new IMAGE_BRUSH_SVG("Icons/BrushIcons/Falloff_Linear", Icon120));
		StyleSet->Set("BrushFalloffIcons.Inverse", new IMAGE_BRUSH_SVG("Icons/BrushIcons/Falloff_Inverse", Icon120));
		StyleSet->Set("BrushFalloffIcons.Round", new IMAGE_BRUSH_SVG("Icons/BrushIcons/Falloff_Round", Icon120));
		StyleSet->Set("BrushFalloffIcons.BoxSmooth", new IMAGE_BRUSH_SVG("Icons/BrushIcons/Falloff_BoxSmooth", Icon120));
		StyleSet->Set("BrushFalloffIcons.BoxLinear", new IMAGE_BRUSH_SVG("Icons/BrushIcons/Falloff_BoxLinear", Icon120));
		StyleSet->Set("BrushFalloffIcons.BoxInverse", new IMAGE_BRUSH_SVG("Icons/BrushIcons/Falloff_BoxInverse", Icon120));
		StyleSet->Set("BrushFalloffIcons.BoxRound", new IMAGE_BRUSH_SVG("Icons/BrushIcons/Falloff_BoxRound", Icon120));

		//
		// Icons for brush falloffs in the Quick Settings widget
		//
		StyleSet->Set("FalloffQuickSettings.Smooth", new IMAGE_BRUSH_SVG("Icons/BrushIcons/QS_Falloff_Smooth_16", Icon16x16));
		StyleSet->Set("FalloffQuickSettings.Linear", new IMAGE_BRUSH_SVG("Icons/BrushIcons/QS_Falloff_Linear_16", Icon16x16));
		StyleSet->Set("FalloffQuickSettings.Inverse", new IMAGE_BRUSH_SVG("Icons/BrushIcons/QS_Falloff_Inverse_16", Icon16x16));
		StyleSet->Set("FalloffQuickSettings.Round", new IMAGE_BRUSH_SVG("Icons/BrushIcons/QS_Falloff_Round_16", Icon16x16));
		StyleSet->Set("FalloffQuickSettings.BoxSmooth", new IMAGE_BRUSH_SVG("Icons/BrushIcons/QS_Falloff_BoxSmooth_16", Icon16x16));
		StyleSet->Set("FalloffQuickSettings.BoxLinear", new IMAGE_BRUSH_SVG("Icons/BrushIcons/QS_Falloff_BoxLinear_16", Icon16x16));
		StyleSet->Set("FalloffQuickSettings.BoxInverse", new IMAGE_BRUSH_SVG("Icons/BrushIcons/QS_Falloff_BoxInverse_16", Icon16x16));
		StyleSet->Set("FalloffQuickSettings.BoxRound", new IMAGE_BRUSH_SVG("Icons/BrushIcons/QS_Falloff_BoxRound_16", Icon16x16));


		//
		// Icons for brushes in sculpt/etc tools
		//

		StyleSet->Set("BrushTypeIcons.Smooth", new IMAGE_BRUSH_SVG("Icons/BrushIcons/Brush_Smooth", Icon120));
		StyleSet->Set("BrushTypeIcons.SmoothFill", new IMAGE_BRUSH_SVG("Icons/BrushIcons/Brush_SmoothFill", Icon120));
		StyleSet->Set("BrushTypeIcons.Move", new IMAGE_BRUSH_SVG("Icons/BrushIcons/Brush_Move", Icon120));
		StyleSet->Set("BrushTypeIcons.SculptN", new IMAGE_BRUSH_SVG("Icons/BrushIcons/Brush_SculptN", Icon120));
		StyleSet->Set("BrushTypeIcons.SculptV", new IMAGE_BRUSH_SVG("Icons/BrushIcons/Brush_SculptV", Icon120));
		StyleSet->Set("BrushTypeIcons.SculptMx", new IMAGE_BRUSH_SVG("Icons/BrushIcons/Brush_SculptMx", Icon120));
		StyleSet->Set("BrushTypeIcons.Inflate", new IMAGE_BRUSH_SVG("Icons/BrushIcons/Brush_Inflate", Icon120));
		StyleSet->Set("BrushTypeIcons.Pinch", new IMAGE_BRUSH_SVG("Icons/BrushIcons/Brush_Pinch", Icon120));
		StyleSet->Set("BrushTypeIcons.Flatten", new IMAGE_BRUSH_SVG("Icons/BrushIcons/Brush_Flatten", Icon120));
		StyleSet->Set("BrushTypeIcons.PlaneN", new IMAGE_BRUSH_SVG("Icons/BrushIcons/Brush_PlaneN", Icon120));
		StyleSet->Set("BrushTypeIcons.PlaneV", new IMAGE_BRUSH_SVG("Icons/BrushIcons/Brush_PlaneV", Icon120));
		StyleSet->Set("BrushTypeIcons.PlaneW", new IMAGE_BRUSH_SVG("Icons/BrushIcons/Brush_PlaneW", Icon120));
		StyleSet->Set("BrushTypeIcons.Scale", new IMAGE_BRUSH_SVG("Icons/BrushIcons/Brush_Scale", Icon120));
		StyleSet->Set("BrushTypeIcons.Grab", new IMAGE_BRUSH_SVG("Icons/BrushIcons/Brush_Grab", Icon120));
		StyleSet->Set("BrushTypeIcons.GrabSharp", new IMAGE_BRUSH_SVG("Icons/BrushIcons/Brush_GrabSharp", Icon120));
		StyleSet->Set("BrushTypeIcons.Twist", new IMAGE_BRUSH_SVG("Icons/BrushIcons/Brush_Twist", Icon120));

		//
		// Icons for selection buttons in PolyEd and TriEd
		//

		StyleSet->Set("PolyEd.SelectCorners", new IMAGE_BRUSH_SVG("Icons/SelectionVertices", Icon20x20));
		StyleSet->Set("PolyEd.SelectEdges", new IMAGE_BRUSH_SVG("Icons/SelectionBorderEdges", Icon20x20));
		StyleSet->Set("PolyEd.SelectFaces", new IMAGE_BRUSH_SVG("Icons/SelectionTriangles3", Icon20x20));
		StyleSet->Set("PolyEd.SelectEdgeLoops", new IMAGE_BRUSH_SVG("Icons/ModelingEdgeLoopSelection", Icon20x20));
		StyleSet->Set("PolyEd.SelectEdgeRings", new IMAGE_BRUSH_SVG("Icons/ModelingEdgeRingSelection", Icon20x20));

		// Icons for PolyEd and TriEd activities
		StyleSet->Set("PolyEd.InsertGroupEdge", new IMAGE_PLUGIN_BRUSH("Icons/ModelingGroupEdgeInsert_x40", Icon20x20));
		StyleSet->Set("PolyEd.InsertEdgeLoop", new IMAGE_PLUGIN_BRUSH("Icons/ModelingEdgeLoopInsert_x40", Icon20x20));
		StyleSet->Set("PolyEd.Extrude", new IMAGE_BRUSH_SVG("Icons/Extrude", Icon20x20));
		StyleSet->Set("PolyEd.Offset", new IMAGE_BRUSH_SVG("Icons/Offset", Icon20x20));
		StyleSet->Set("PolyEd.PushPull", new IMAGE_BRUSH_SVG("Icons/PushPull", Icon20x20));
		StyleSet->Set("PolyEd.Inset", new IMAGE_BRUSH_SVG("Icons/Inset", Icon20x20));
		StyleSet->Set("PolyEd.Outset", new IMAGE_BRUSH_SVG("Icons/Outset", Icon20x20));
		StyleSet->Set("PolyEd.CutFaces", new IMAGE_BRUSH_SVG("Icons/CutFaces", Icon20x20));
		StyleSet->Set("PolyEd.ProjectUVs", new IMAGE_PLUGIN_BRUSH("Icons/UVProjection_40x", Icon20x20));
		StyleSet->Set("PolyEd.Bevel", new IMAGE_BRUSH_SVG("Icons/Bevel", Icon20x20));
		StyleSet->Set("PolyEd.ExtrudeEdge", new IMAGE_BRUSH_SVG("Icons/ExtrudeEdge", Icon20x20));
	}

	// Style for the toolbar in the PolyEd customization
	{
		// For the selection button toolbar, we want to use something similar to the toolbar we use in the viewport
		StyleSet->Set("PolyEd.SelectionToolbar", FAppStyle::Get().GetWidgetStyle<FToolBarStyle>("EditorViewportToolBar"));

		// However, increase the size of the buttons a bit
		FCheckBoxStyle ToggleButtonStart = FAppStyle::Get().GetWidgetStyle<FCheckBoxStyle>("EditorViewportToolBar.ToggleButton.Start");
		ToggleButtonStart.SetPadding(FMargin(9, 7, 6, 7));
		StyleSet->Set("PolyEd.SelectionToolbar.ToggleButton.Start", ToggleButtonStart);

		FCheckBoxStyle ToggleButtonMiddle = FAppStyle::Get().GetWidgetStyle<FCheckBoxStyle>("EditorViewportToolBar.ToggleButton.Middle");
		ToggleButtonMiddle.SetPadding(FMargin(9, 7, 6, 7));
		StyleSet->Set("PolyEd.SelectionToolbar.ToggleButton.Middle", ToggleButtonMiddle);

		FCheckBoxStyle ToggleButtonEnd = FAppStyle::Get().GetWidgetStyle<FCheckBoxStyle>("EditorViewportToolBar.ToggleButton.End");
		ToggleButtonEnd.SetPadding(FMargin(7, 7, 8, 7));
		StyleSet->Set("PolyEd.SelectionToolbar.ToggleButton.End", ToggleButtonEnd);
	}

	// Style to be applied to customizable section headers so that the color shows up properly
	{
		// look up default radii for palette toolbar expandable area headers
		FVector4 HeaderRadii(4, 4, 0, 0);
		const FSlateBrush* BaseBrush = FAppStyle::Get().GetBrush("PaletteToolbar.ExpandableAreaHeader");
		if (BaseBrush != nullptr)
		{
			HeaderRadii = BaseBrush->OutlineSettings.CornerRadii;
		}
		StyleSet->Set("MeshTerrainMode.WhiteExpandableAreaHeader", new FSlateRoundedBoxBrush(FSlateColor(FLinearColor::White), HeaderRadii));
	}

	// Similar to EditorViewport.OverlayBrush, but opaque with a gray color to be able to be placed on top of other overlays.
	StyleSet->Set("MeshTerrainMode.OpaqueOverlayBrush", 
		new FSlateRoundedBoxBrush(FStyleColors::Panel.GetSpecifiedColor(), 8.0, FStyleColors::Dropdown, 1.0));
	
	// Similar to EditorViewport.OverlayBrush, but no outline and reduced alpha
	FLinearColor ViewportOverlayColor = FStyleColors::Panel.GetSpecifiedColor();
	StyleSet->Set("MeshTerrainMode.OpaqueOverlayBrushNoOutline",
		new FSlateRoundedBoxBrush(ViewportOverlayColor, 8.0f, FStyleColors::Dropdown, 0.0));

	FLinearColor TransparentBackgroundColor = FStyleColors::Transparent.GetSpecifiedColor();
	TransparentBackgroundColor.A = 0.0f;
	StyleSet->Set("MeshTerrainMode.TransparentBackgroundBrush",
		new FSlateRoundedBoxBrush(TransparentBackgroundColor, 8.0f, FStyleColors::Transparent, 0.0));

	{
		const FLinearColor SubmodePaletteColor = FStyleColors::Dropdown.GetSpecifiedColor();
		StyleSet->Set("MeshTerrainMode.SubmodePaletteLighterBrush",
			 new FSlateRoundedBoxBrush(SubmodePaletteColor, 6.0f, FColor(85, 85, 85), 1.0));

		const FLinearColor SubmodeToolsPaletteColor = FStyleColors::Panel.GetSpecifiedColor();
		StyleSet->Set("MeshTerrainMode.SubmodePaletteDarkerBrush",
			new FSlateRoundedBoxBrush(SubmodeToolsPaletteColor, 6.0, FColor(85, 85, 85), 1.0));
	}

	FSlateFontInfo ToolPanelFont = FAppStyle::Get().GetFontStyle("NormalFont");
	ToolPanelFont.Size = 8.0f;
	StyleSet->Set("ToolPanel.Font", ToolPanelFont);

	FSlateBrush* ToolPanelSeparatorBrush = new FSlateBrush();
	ToolPanelSeparatorBrush->DrawAs = ESlateBrushDrawType::Box;
	ToolPanelSeparatorBrush->TintColor = FStyleColors::Hover;
	StyleSet->Set("ToolPanel.SeparatorBrush", ToolPanelSeparatorBrush);

	// TODO : all temp
	FLinearColor SectionedDetailsViewHeaderColor = FStyleColors::Dropdown.GetSpecifiedColor();
	StyleSet->Set("MeshTerrainMode.SectionedDetailsViewHeaderBrush",
		 new FSlateRoundedBoxBrush(SectionedDetailsViewHeaderColor, 8.0f, FLinearColor(FColor(85, 85, 85)), 1.0));

	FToolBarStyle EditorViewportToolBar = FAppStyle::Get().GetWidgetStyle<FToolBarStyle>("EditorViewportToolBar");
	const FButtonStyle ToolShutdownButtons = FButtonStyle(EditorViewportToolBar.ButtonStyle)
		.SetNormal(FSlateRoundedBoxBrush(FStyleColors::Dropdown, 4.f, FStyleColors::Panel, 1.0))
		.SetPressed(FSlateRoundedBoxBrush(FStyleColors::Recessed, 4.f, FStyleColors::Panel, 1.0))
		.SetHovered(FSlateRoundedBoxBrush(FStyleColors::Hover, 4.f, FStyleColors::Panel, 1.0))
		.SetDisabled(FSlateRoundedBoxBrush(FStyleColors::Dropdown, 4.f, FStyleColors::Header, 1.0))
		.SetNormalForeground(FStyleColors::Foreground)
		.SetHoveredForeground(FStyleColors::ForegroundHover)
		.SetPressedForeground(FStyleColors::ForegroundHover)
		.SetDisabledForeground(FStyleColors::White);
	StyleSet->Set("MeshTerrainMode.DetailsViewToolShutdown", ToolShutdownButtons);

	const FButtonStyle CloseDetailsViewButton = FButtonStyle(EditorViewportToolBar.ButtonStyle)
		.SetNormal(FSlateRoundedBoxBrush(FStyleColors::Dropdown, 4.f, FStyleColors::Dropdown, 0.0))
		.SetPressed(FSlateRoundedBoxBrush(FStyleColors::Recessed, 4.f, FStyleColors::Recessed, 0.0))
		.SetHovered(FSlateRoundedBoxBrush(FStyleColors::Hover, 4.f, FStyleColors::Hover, 0.0))
		.SetDisabled(FSlateRoundedBoxBrush(FStyleColors::Dropdown, 4.f, FStyleColors::Dropdown, 0.0))
		.SetNormalForeground(FStyleColors::Foreground)
		.SetHoveredForeground(FStyleColors::ForegroundHover)
		.SetPressedForeground(FStyleColors::ForegroundHover)
		.SetDisabledForeground(FStyleColors::White);
	StyleSet->Set("MeshTerrainMode.DetailsViewClose", CloseDetailsViewButton);

	FSlateStyleRegistry::RegisterSlateStyle(*StyleSet.Get());
};

END_SLATE_FUNCTION_BUILD_OPTIMIZATION

#undef IMAGE_PLUGIN_BRUSH
#undef IMAGE_BRUSH
#undef BOX_BRUSH
#undef DEFAULT_FONT

void FMeshTerrainModeStyle::Shutdown()
{
	if (StyleSet.IsValid())
	{
		FSlateStyleRegistry::UnRegisterSlateStyle(*StyleSet.Get());
		ensure(StyleSet.IsUnique());
		StyleSet.Reset();
	}
}
