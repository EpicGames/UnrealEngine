// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Templates/SharedPointer.h"

struct FSlateBrush;

class FMeshTerrainModeStyle
{
public:
	static void Initialize();

	static void Shutdown();

	static TSharedPtr< class ISlateStyle > Get();

	static FName GetStyleSetName();

	// use to access icons defined by the style set by name, eg GetBrush("BrushFalloffIcons.Smooth")
	static const FSlateBrush* GetBrush(FName PropertyName, const ANSICHAR* Specifier = NULL);

private:
	static FString InContent(const FString& RelativePath, const ANSICHAR* Extension);

private:
	static TSharedPtr< class FSlateStyleSet > StyleSet;
};
