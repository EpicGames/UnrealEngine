// Copyright Epic Games, Inc. All Rights Reserved.

#include "MeshTerrainModeStylusInputHandler.h"

#if ENABLE_STYLUS_SUPPORT

#include "StylusInputTabletContext.h"
#include "Framework/Application/SlateApplication.h"

using namespace UE::MeshTerrain;
using namespace UE::StylusInput;

FStylusInputHandler::FStylusInputHandler()
{	
}

FStylusInputHandler::~FStylusInputHandler()
{
	for (TPair<TSharedPtr<SWindow>, IStylusInputInstance*> Pair : StylusInputInstances)
	{
		IStylusInputInstance* InputInstance = Pair.Value;
		InputInstance->RemoveEventHandler(this);
		ReleaseInstance(InputInstance);
	}
}

bool FStylusInputHandler::RegisterWindow(const TSharedRef<SWidget>& Widget)
{
	TSharedPtr<SWindow> Window = FSlateApplication::Get().FindWidgetWindow(Widget);
	if (!Window)
	{
		return false;
	}

	if (StylusInputInstances.Contains(Window))
	{
		return false;
	}

	IStylusInputInstance* InputInstance = CreateInstance(*Window);
	if (!InputInstance)
	{
		return false;
	}
	
	InputInstance->AddEventHandler(this, EEventHandlerThread::OnGameThread);
	StylusInputInstances.Add(Window, InputInstance);
	return true;
}

FString FStylusInputHandler::GetName()
{
	return "MeshTerrainModeStylusInputHandler";
}

void FStylusInputHandler::OnPacket(const FStylusInputPacket& Packet, IStylusInputInstance* Instance)
{
	if (Packet.Type != EPacketType::Invalid &&
		Packet.Type != EPacketType::AboveDigitizer)
	{
		ProcessPacket(Packet, Instance);
	}
	else if(Packet.Type == EPacketType::AboveDigitizer)
	{
		// Force instantiate the tablet context on hover for device supporting pressure detection.
		GetTabletContext(Instance, Packet.TabletContextID);
	}
}

void FStylusInputHandler::ProcessPacket(const FStylusInputPacket& Packet, IStylusInputInstance* Instance)
{
	const IStylusInputTabletContext* TabletContext = GetTabletContext(Instance, Packet.TabletContextID);
	const ETabletSupportedProperties SupportedProperties = TabletContext->GetSupportedProperties();
	const bool bSupportsNormalPressure = (SupportedProperties & ETabletSupportedProperties::NormalPressure) != ETabletSupportedProperties::None;
	ActivePressure = bSupportsNormalPressure ? Packet.NormalPressure : 1.0f;
}

const IStylusInputTabletContext* FStylusInputHandler::GetTabletContext(IStylusInputInstance* Instance, uint32 TabletContextID)
{
	if (!Instance)
	{
		return nullptr;
	}

	const TSharedPtr<IStylusInputTabletContext>* TabletContext = TabletContexts.Find(TabletContextID);
	if (!TabletContext)
	{
		if (const TSharedPtr<IStylusInputTabletContext>& NewTabletContext = Instance->GetTabletContext(TabletContextID))
		{
			// We currently assume that TabletContextIDs are unique across all instances.
			TabletContext = &TabletContexts.Emplace(TabletContextID, NewTabletContext);
		}
	}

	return TabletContext ? TabletContext->Get() : nullptr;
}

bool FStylusInputHandler::AnyContextSupportsPressure() const
{
	for (const TPair<uint32, TSharedPtr<IStylusInputTabletContext>>& Entry : TabletContexts)
	{
		if (!Entry.Value)
		{
			continue;
		}
		const ETabletSupportedProperties SupportedProperties = Entry.Value->GetSupportedProperties();
		if ((SupportedProperties & ETabletSupportedProperties::NormalPressure) != ETabletSupportedProperties::None)
		{
			return true;
		}
	}
	return false;
}

#endif // ENABLE_STYLUS_SUPPORT

