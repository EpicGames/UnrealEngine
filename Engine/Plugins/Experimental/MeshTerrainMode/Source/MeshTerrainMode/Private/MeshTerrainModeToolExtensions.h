// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Features/IModularFeature.h"
#include "Internationalization/Text.h"
#include "InteractiveToolManager.h"
#include "ModelingModeToolExtensions.h"
#include "Submodes/Submode.h"

class UInteractiveToolsContext;
class FUICommandInfo;
class UInteractiveToolBuilder;
namespace UE
{
	class IInteractiveToolCommandsInterface;
}

struct FExtensionSubmodeAddon
{
	FExtensionSubmodeAddon(FName Name, const UE::MeshTerrain::FSubmodeToolPalette& Palette)
	: SubmodeName(Name)
	, ToolPalette(Palette)	
	{}

	FExtensionSubmodeAddon(const FExtensionSubmodeAddon& Other)
	: SubmodeName(Other.SubmodeName)
	, ToolPalette(Other.ToolPalette)
	{}
	
	FName SubmodeName;
	UE::MeshTerrain::FSubmodeToolPalette ToolPalette;
};

/**
 * IModelingProtoModeToolExtension implementations return the list of Tools & Submodes they provide
 * via instances of FExtensionSubmodeDescription.
 */
struct FExtensionSubmodeDescription
{
	/** Submode that is added to the toolset. */
	TFunction<TSharedPtr<UE::MeshTerrain::FSubmode>()> MakeNewSubmode;
};

struct FExtensionCustomToolDescription : public FExtensionToolDescription
{
	/**
	 * Specify this function to override the default ExecuteAction behavior of executing the
	 * associated action.
	 * @return True, if the ExecuteAction was handled by this override, false otherwise.
	 */
	TFunction<bool(UInteractiveToolManager*, EToolSide)> ExecuteAction;

	/**
	 * Specify this function to override the default CanExecuteAction handled by the
	 * owning Mode.
	 * @return True if the action can be executed.
	 */
	TFunction<bool(UInteractiveToolManager*, EToolSide)> CanExecuteAction;

	/**
	 * Specify this function to override the default IsActionChecked behavior of comparing
	 * the ActiveToolName against the ToolIdentifier.
	 * @return True if the action checkbox should be checked.
	 */
	TFunction<bool(UInteractiveToolManager*, EToolSide)> IsActionChecked;
};

/**
 * IModelingProtoModeToolExtension uses the IModularFeature API to allow a Plugin to provide
 * a set of InteractiveTool's to be exposed in Modeling Proto Mode.
 * 
 */
class IMeshTerrainModeToolExtension : public IModelingModeToolExtension
{
public:
	virtual ~IMeshTerrainModeToolExtension() {}

	/**
	 * Query the Extension for a list of tools to add on to existing Submodes. 
	 */
	virtual void GetExtensionSubmodeAddons(TArray<FExtensionSubmodeAddon>& AddonsOut) {}

	/**
	 * Query the Extension for a list of Submodes to expose in Modeling Mode.
	 */
	virtual void GetExtensionSubmodes(TArray<FExtensionSubmodeDescription>& SubmodesOut) {}

	/**
	 * Custom tool registration with optional overrides for ExecuteAction, CanExecuteAction and IsActionChecked. 
	 */
	virtual void GetExtensionCustomTools(const FExtensionToolQueryInfo& QueryInfo, TArray<FExtensionCustomToolDescription>& OutTools) {}
	
	static FName GetModularFeatureName()
	{
		static FName FeatureName = FName(TEXT("MeshTerrainModeToolExtension"));
		return FeatureName;
	}
};
