// Copyright Epic Games, Inc. All Rights Reserved.

#include "MeshTerrainModeToolkit.h"
#include "Brushes/SlateRoundedBoxBrush.h"
#include "EditorModeManager.h"
#include "Tools/EdModeInteractiveToolsContext.h"
#include "MeshTerrainModeManagerActions.h"
#include "InteractiveToolManager.h"
#include "MeshTerrainModeSettings.h"
#include "MeshTerrainModeStyle.h"
#include "MeshTerrainMode.h"
#include "Engine/World.h"

#include "Modules/ModuleManager.h"
#include "IDetailsView.h"
#include "Toolkits/AssetEditorModeUILayer.h"
#include "PropertySets/CreateMeshObjectTypeProperties.h"

#include "STransformGizmoNumericalUIOverlay.h"
#include "Tools/UEdMode.h"
#include "Widgets/Docking/SDockTab.h"
#include "Widgets/Layout/SExpandableArea.h"
#include "Overlay/SDraggableBoxOverlay.h"
#include "Framework/MultiBox/SToolBarButtonBlock.h"

// for Tool Extensions
#include "Features/IModularFeatures.h"
#include "MeshTerrainModeToolExtensions.h"

// for quick settings
#include "ToolTargets/StaticMeshComponentToolTarget.h"
#include "SPrimaryButton.h"
#include "MeshTerrainModeEditablePaletteConfig.h"
#include "ISinglePropertyView.h"

// for showing toast notifications
#include "Widgets/Notifications/SNotificationList.h"
#include "Framework/Notifications/NotificationManager.h"

// for presets
#include "ToolPresetAsset.h"

#include "LevelEditorViewport.h"
#include "Widgets/Input/SEditableTextBox.h"
#include "ToolMenus.h"
#include "ToolkitBuilder.h"
#include "Dialogs/Dialogs.h"
#include "Widgets/Input/SComboButton.h"
#include "IToolPresetEditorModule.h"
#include "ToolPresetSettings.h"
#include "AssetRegistry/AssetRegistryModule.h"
#include "ModelingWidgets/SToolInputAssetComboPanel.h"
#include "Fonts/SlateFontInfo.h"
#include "ToolPresetAssetSubsystem.h"
#include "Selection/GeometrySelectionManager.h"
#include "Widgets/Input/SMultiLineEditableTextBox.h"
#include "IAssetViewport.h"
#include "Widgets/MeshTerrainModeUtil.h"
#include "Widgets/SSectionedDetailsViewWidget.h"
#include "PropertyHandle.h"

#if ENABLE_STYLUS_SUPPORT
#include "MeshTerrainModeStylusInputHandler.h"
#endif

#define LOCTEXT_NAMESPACE "FMeshTerrainModeToolkit"

namespace UE::MeshTerrain
{

static TAutoConsoleVariable<int32> CVarEnableToolPresets(
	TEXT("MeshTerrainMode.EnablePresets"),
	1,
	TEXT("Enable tool preset features and UX"));

static TAutoConsoleVariable<int32> CVarToolPaletteMode(
	TEXT("MeshTerrainMode.ToolPaletteMode"),
	1,
	TEXT("0 = Toolkit Builder, 1 = Viewport Overlay"));

static TAutoConsoleVariable<int32> CVarDetailsViewStyle(
	TEXT("MeshTerrainMode.DetailsViewStyle"),
	1,
	TEXT("0 = Details View Tabs, 1 = Full Details View"));

static TAutoConsoleVariable<int32> CVarEnableExperimentalTools(
	TEXT("MeshTerrainMode.EnableExperimentalTools"),
	0,
	TEXT("Enable experimental tools in Mesh Terrain Mode."));

namespace FMeshTerrainModeToolkitLocals
{
	typedef TFunction<void(UInteractiveToolsPresetCollectionAsset& Preset, UInteractiveTool& Tool)> PresetAndToolFunc;
	typedef TFunction<void(UInteractiveToolsPresetCollectionAsset& Preset)> PresetOnlyFunc;

	void ExecuteWithPreset(const FSoftObjectPath& PresetPath, PresetOnlyFunc Function)
	{
		UInteractiveToolsPresetCollectionAsset* Preset = nullptr;

		UToolPresetAssetSubsystem* PresetAssetSubsystem = GEditor->GetEditorSubsystem<UToolPresetAssetSubsystem>();

		if (PresetPath.IsNull() && ensure(PresetAssetSubsystem))
		{
			Preset = PresetAssetSubsystem->GetDefaultCollection();
		}
		if (PresetPath.IsAsset())
		{
			Preset = Cast<UInteractiveToolsPresetCollectionAsset>(PresetPath.TryLoad());
		}
		if (!Preset)
		{
			return;
		}
		Function(*Preset);


	}

	void ExecuteWithPresetAndTool(UEdMode& EdMode, EToolSide ToolSide, const FSoftObjectPath& PresetPath, PresetAndToolFunc Function)
	{
		UInteractiveToolsPresetCollectionAsset* Preset = nullptr;		
		UInteractiveTool* Tool = EdMode.GetToolManager()->GetActiveTool(EToolSide::Left);

		UToolPresetAssetSubsystem* PresetAssetSubsystem = GEditor->GetEditorSubsystem<UToolPresetAssetSubsystem>();

		if (PresetPath.IsNull() && ensure(PresetAssetSubsystem))
		{
			Preset = PresetAssetSubsystem->GetDefaultCollection();
		}
		if (PresetPath.IsAsset())
		{
			Preset = Cast<UInteractiveToolsPresetCollectionAsset>(PresetPath.TryLoad());
		}
		if (!Preset || !Tool)
		{
			return;
		}
		Function(*Preset, *Tool);
	}

	void ExecuteWithPresetAndTool(UEdMode& EdMode, EToolSide ToolSide, UInteractiveToolsPresetCollectionAsset& Preset, PresetAndToolFunc Function)
	{
		UInteractiveTool* Tool = EdMode.GetToolManager()->GetActiveTool(EToolSide::Left);

		if (!Tool)
		{
			return;
		}
		Function(Preset, *Tool);
	}
}

class FRecentPresetCollectionProvider : public SToolInputAssetComboPanel::IRecentAssetsProvider
{
	public:
		//~ SToolInputAssetComboPanel::IRecentAssetsProvider interface		
		virtual TArray<FAssetData> GetRecentAssetsList() override { return RecentPresetCollectionList; }		
		virtual void NotifyNewAsset(const FAssetData& NewAsset) {
			RecentPresetCollectionList.AddUnique(NewAsset);
		};

	protected:
		TArray<FAssetData> RecentPresetCollectionList;
};

FMeshTerrainModeToolkit::FMeshTerrainModeToolkit()
{
	UMeshTerrainModeEditableToolPaletteConfig::Initialize();
	UMeshTerrainModeEditableToolPaletteConfig::Get()->LoadEditorConfig();

	UToolPresetUserSettings::Initialize();
	UToolPresetUserSettings::Get()->LoadEditorConfig();

	RecentPresetCollectionProvider = MakeShared< FRecentPresetCollectionProvider>();
	CurrentPreset = MakeShared<FAssetData>();

	QuickEditCustomizations = MakeShared<UE::MeshTerrain::FModelingQuickPropertyCustomizations>(UE::MeshTerrain::FModelingQuickPropertyCustomizations());

#if ENABLE_STYLUS_SUPPORT
	// Do not initialize StylusInput for headless editor instances.
	if (!GUsingNullRHI)
	{
		StylusInputHandler = MakeUnique<UE::MeshTerrain::FStylusInputHandler>();
	}
#endif
}

FMeshTerrainModeToolkit::~FMeshTerrainModeToolkit()
{
	if (IsHosted())
	{
		if (GizmoNumericalUIOverlayWidget.IsValid())
		{
			GetToolkitHost()->RemoveViewportOverlayWidget(GizmoNumericalUIOverlayWidget.ToSharedRef());
			GizmoNumericalUIOverlayWidget.Reset();
		}

		if (SubmodeToolPanelOverlay.IsValid())
		{
			GetToolkitHost()->RemoveViewportOverlayWidget(SubmodeToolPanelOverlay.ToSharedRef());
			SubmodeToolPanelOverlay.Reset();
		}

		if (SubmodePaletteOverlay.IsValid())
		{
			GetToolkitHost()->RemoveViewportOverlayWidget(SubmodePaletteOverlay.ToSharedRef());
			SubmodePaletteOverlay.Reset();
		}
		
		if (DetailsOverlayWidget.IsValid())
		{
			GetToolkitHost()->RemoveViewportOverlayWidget(DetailsOverlayWidget.ToSharedRef());
			DetailsOverlayWidget.Reset();
		}
		if (QuickSettingsOverlayWidget.IsValid())
		{
			GetToolkitHost()->RemoveViewportOverlayWidget(QuickSettingsOverlayWidget.ToSharedRef());
			QuickSettingsOverlayWidget.Reset();
		}
	}
	GetScriptableEditorMode()->GetInteractiveToolsContext(EToolsContextScope::EdMode)->OnToolNotificationMessage.RemoveAll(this);
	GetScriptableEditorMode()->GetInteractiveToolsContext(EToolsContextScope::EdMode)->OnToolWarningMessage.RemoveAll(this);

	RecentPresetCollectionProvider = nullptr;

	DeactivateSubmode();
}


void FMeshTerrainModeToolkit::CustomizeModeDetailsViewArgs(FDetailsViewArgs& ArgsInOut)
{
	//ArgsInOut.ColumnWidth = 0.3f;
}


TSharedPtr<SWidget> FMeshTerrainModeToolkit::GetInlineContent() const
{
	return SNew(SVerticalBox)
		+ SVerticalBox::Slot()
		.FillHeight(1.0f)
		.VAlign(VAlign_Fill)
		[
			ToolkitWidget.ToSharedRef()
		];
}

void FMeshTerrainModeToolkit::RegisterPalettes()
{
	const FMeshTerrainModeManagerCommands& Commands = FMeshTerrainModeManagerCommands::Get();
	const TSharedPtr<FUICommandList> CommandList = GetToolkitCommands();

	UEdMode* ScriptableMode = GetScriptableEditorMode().Get();
	UMeshTerrainModeCustomizationSettings* UISettings = GetMutableDefault<UMeshTerrainModeCustomizationSettings>();
	UMeshTerrainModeSettings* ModelingModeSettings = GetMutableDefault<UMeshTerrainModeSettings>();
	UISettings->OnSettingChanged().AddSP(SharedThis(this), &FMeshTerrainModeToolkit::UpdateCategoryButtonLabelVisibility);
	
	ToolkitSections = MakeShared<FToolkitSections>();
	FToolkitBuilderArgs ToolkitBuilderArgs(ScriptableMode->GetModeInfo().ToolbarCustomizationName);
	ToolkitBuilderArgs.ToolkitCommandList = GetToolkitCommands();
	ToolkitBuilderArgs.ToolkitSections = ToolkitSections;
	ToolkitBuilderArgs.SelectedCategoryTitleVisibility = EVisibility::Collapsed;
	// This lets us re-show the buttons if the user clicks a category with a tool still active.
	ToolkitBuilderArgs.CategoryReclickBehavior = FToolkitBuilder::ECategoryReclickBehavior::TreatAsChanged;
	ToolkitBuilder = MakeShared<FToolkitBuilder>(ToolkitBuilderArgs);
	ToolkitBuilder->SetCategoryButtonLabelVisibility(UISettings->bShowCategoryButtonLabels);

	// We need to specify the modeling mode specific config instance because this cannot exist at the base toolkit level due to dependency issues currently
	FGetEditableToolPaletteConfigManager GetConfigManager = FGetEditableToolPaletteConfigManager::CreateStatic(&UMeshTerrainModeEditableToolPaletteConfig::GetAsConfigManager);

	if (CVarToolPaletteMode.GetValueOnGameThread() == 0)
	{
		for (const TPair<FName, TSharedPtr<FSubmode>>& Submode : Submodes)
		{
			// Empty PaletteItems since they will be driven external to the ToolkitBuilder system
			TArray<TSharedPtr<FUICommandInfo>> PaletteItems;
			TSharedPtr<FToolPalette> Palette = MakeShared<FToolPalette>(Submode.Value->GetEnterSubmodeAction(), PaletteItems);
			ToolkitBuilder->AddPalette( Palette );
		}

		ToolkitBuilder->SetActivePaletteCommandsVisibility(EVisibility::Collapsed);
		if (TSharedPtr<UE::MeshTerrain::FSubmode> DefaultSubmode = GetDefaultSubmode())
		{
			ToolkitBuilder->SetActivePaletteOnLoad(DefaultSubmode->GetEnterSubmodeAction().Get());
		}
	
		// if selected palette changes, make sure we are showing the palette command buttons, which may be hidden by active Tool
		ActivePaletteChangedHandle = ToolkitBuilder->OnActivePaletteChanged.AddLambda([this]()
		{
			const FName ActivePalette = ToolkitBuilder->GetActivePaletteName();
			if (TSharedPtr<FSubmode> Submode = Submodes.FindRef(ActivePalette))
			{
				ActivateSubmode(Submode);
			}
		});
	}
}


void FMeshTerrainModeToolkit::Init(const TSharedPtr<IToolkitHost>& InitToolkitHost, TWeakObjectPtr<UEdMode> InOwningMode)
{
	bUsesToolkitBuilder = true;
	
	// Have to create the ToolkitWidget here because FModeToolkit::Init() is going to ask for it and add
	// it to the Mode panel, and not ask again afterwards. However we have to call Init() to get the 
	// ModeDetailsView created, that we need to add to the ToolkitWidget. So, we will create the Widget
	// here but only add the rows to it after we call Init()

	const TSharedPtr<SVerticalBox> ToolkitWidgetVBox = SNew(SVerticalBox);
	
	if ( !bUsesToolkitBuilder )
	{
		SAssignNew(ToolkitWidget, SBorder)
			.HAlign(HAlign_Fill)
			.Padding(4)
			[
				ToolkitWidgetVBox->AsShared()
			];
	}

	FModeToolkit::Init(InitToolkitHost, InOwningMode);

	// set up a details view that only displays 'active' properties
	if ( CVarDetailsViewStyle.GetValueOnGameThread() == 0 )
	{
		FDetailsViewArgs SectionedDetailsViewArgs;
		SectionedDetailsViewArgs.bAllowSearch = false;
		SectionedDetailsViewArgs.NameAreaSettings = FDetailsViewArgs::HideNameArea;
		SectionedDetailsViewArgs.bHideSelectionTip = true;
		SectionedDetailsViewArgs.DefaultsOnlyVisibility = EEditDefaultsOnlyNodeVisibility::Automatic;
		SectionedDetailsViewArgs.bShowOptions = false;
		SectionedDetailsViewArgs.bAllowMultipleTopLevelObjects = true;
		SectionedDetailsViewArgs.ShouldForceHideProperty.BindLambda([this](const TSharedRef<IPropertyHandle>& PropertyNode)
			{
				if (!SectionedDetailsViewWidget.IsValid())
				{
					return true;
				}

				const TSharedPtr<TSet<FProperty*>> ActiveProperties = SectionedDetailsViewWidget->GetActiveProperties();

				if (!ActiveProperties)
				{
					return true;
				}

				if (ActiveProperties->Contains(PropertyNode->GetProperty()))
				{
					return false;
				}
				
				TSharedPtr<IPropertyHandle> Parent = PropertyNode->GetParentHandle();

				// check all parents of Property
				while (Parent && Parent->GetProperty())
				{
					if (ActiveProperties->Contains(Parent->GetProperty()))
					{
						return false;
					}
					Parent = Parent->GetParentHandle();
				}
				return true;
			});

		FPropertyEditorModule& PropertyEditor = FModuleManager::Get().LoadModuleChecked<FPropertyEditorModule>("PropertyEditor");
		CustomizeModeDetailsViewArgs(SectionedDetailsViewArgs);
		SectionedDetailsView = PropertyEditor.CreateDetailView(SectionedDetailsViewArgs);
	}

	GetToolkitHost()->OnActiveViewportChanged().AddSP(this, &FMeshTerrainModeToolkit::OnActiveViewportChanged);

	InitializeSubmodes();

	ModeWarningArea = SNew(STextBlock)
		.AutoWrapText(true)
		.Font(FCoreStyle::GetDefaultFontStyle("Bold", 9))
		.ColorAndOpacity(FSlateColor(FLinearColor(0.9f, 0.15f, 0.15f)));
	ModeWarningArea->SetText(FText::GetEmpty());
	ModeWarningArea->SetVisibility(EVisibility::Collapsed);

	ModeHeaderArea = SNew(STextBlock)
		.AutoWrapText(true)
		.Font(FCoreStyle::GetDefaultFontStyle("Bold", 12));
	ModeHeaderArea->SetText(LOCTEXT("SelectToolLabel", "Select a Tool from the Toolbar"));
	ModeHeaderArea->SetJustification(ETextJustify::Center);


	ToolWarningArea = SNew(STextBlock)
		.AutoWrapText(true)
		.Font(FCoreStyle::GetDefaultFontStyle("Bold", 9))
		.ColorAndOpacity(FSlateColor(FLinearColor(0.9f, 0.15f, 0.15f)));
	ToolWarningArea->SetText(FText::GetEmpty());

	ToolPresetArea = MakePresetPanel();

	// add the various sections to the mode toolbox
	ToolkitWidgetVBox->AddSlot().AutoHeight().HAlign(HAlign_Fill).Padding(5)
		[
			ToolPresetArea->AsShared()
		];
	ToolkitWidgetVBox->AddSlot().AutoHeight().HAlign(HAlign_Fill).Padding(5)
		[
			ModeWarningArea->AsShared()
		];

	if (bUsesToolkitBuilder)
	{
		RegisterPalettes();
	}
	else
	{
		ToolkitWidgetVBox->AddSlot().AutoHeight().HAlign(HAlign_Fill).Padding(5)
			[
				ModeHeaderArea->AsShared()
			];
		ToolkitWidgetVBox->AddSlot().AutoHeight().HAlign(HAlign_Fill).Padding(5)
		[
			ToolWarningArea->AsShared()
		];	
		ToolkitWidgetVBox->AddSlot().HAlign(HAlign_Fill).FillHeight(1.f)
		[
			ModeDetailsView->AsShared()
		];
	}
		

	ClearNotification();
	ClearWarning();

	if (HasToolkitBuilder())
	{
		ToolkitSections->ModeWarningArea = ModeWarningArea;
		ToolkitSections->ToolPresetArea = ToolPresetArea;
		ToolkitSections->DetailsView = ModeDetailsView;
		ToolkitSections->ToolWarningArea = ToolWarningArea;

		SAssignNew(ToolkitWidget, SBorder)
		.HAlign(HAlign_Fill)
		.Padding(0)
		.BorderImage(FAppStyle::GetBrush("ToolPanel.GroupBorder"))
		[
			ToolkitBuilder->GenerateWidget()->AsShared()
		];
	}

	ActiveToolName = FText::GetEmpty();
	ActiveToolMessage = FText::GetEmpty();

	GetScriptableEditorMode()->GetInteractiveToolsContext(EToolsContextScope::EdMode)->OnToolNotificationMessage.AddSP(this, &FMeshTerrainModeToolkit::PostNotification);
	GetScriptableEditorMode()->GetInteractiveToolsContext(EToolsContextScope::EdMode)->OnToolWarningMessage.AddSP(this, &FMeshTerrainModeToolkit::PostWarning);

	MakeToolShutdownOverlayWidget();

	// Note that the numerical UI widget should be created before making the selection palette so that
	// it can be bound to the buttons there.
	MakeGizmoNumericalUIOverlayWidget();

	if (CVarToolPaletteMode.GetValueOnGameThread() == 0)
	{
		MakeSubmodeToolPanelOverlayWidget();
		GetToolkitHost()->AddViewportOverlayWidget(SubmodeToolPanelOverlay.ToSharedRef());
	}
	else
	{
		MakeSubmodePaletteOverlayWidget();
		GetToolkitHost()->AddViewportOverlayWidget(SubmodePaletteOverlay.ToSharedRef());
	}

	MakeDetailsOverlayWidget();

	MakeQuickSettingsOverlayWidget();

	if (SectionedDetailsView.IsValid())
	{
		SectionedDetailsViewWidget->OnSectionedDetailsPanelRequestRebuild.BindLambda([this]()
		{
			SectionedDetailsView->RequestForceRefresh();
		});
	}

	QuickSettingsWidget->OnPropertiesButtonPressed.BindLambda([this](const bool bDisplayDetailsView)
	{
		if (DetailsOverlayWidget.IsValid())
		{
			if (bDisplayDetailsView)
			{
				if (SectionedDetailsView.IsValid())
				{
					RebuildSectionedDetailsOverlayWidget();
				}
				else
				{
					GetToolkitHost()->AddViewportOverlayWidget(DetailsOverlayWidget.ToSharedRef());
				}
			}
			else
			{
				GetToolkitHost()->RemoveViewportOverlayWidget(DetailsOverlayWidget.ToSharedRef());
			}
		}
	});

	CurrentPresetPath = FSoftObjectPath(); // Default to the default collection by leaving this null.

	UMeshTerrainModeCustomizationSettings* ModelingCustomizationSettings = GetMutableDefault<UMeshTerrainModeCustomizationSettings>();
	ModelingCustomizationSettings->OnSettingChanged().AddSP(SharedThis(this),  &FMeshTerrainModeToolkit::UpdateSelectionColors);
	ModelingCustomizationSettings->OnSettingChanged().AddSP(SharedThis(this),  &FMeshTerrainModeToolkit::UpdateOverlayWidgets);

	// enable/disable dynamic mesh actors
	UCreateMeshObjectTypeProperties::bEnableDynamicMeshActorSupport = true;

#if ENABLE_STYLUS_SUPPORT
	if (StylusInputHandler && InitToolkitHost)
	{
		StylusInputHandler->RegisterWindow(InitToolkitHost->GetParentWidget());
	}
#endif
}

void FMeshTerrainModeToolkit::RequestModeUITabs()
{
	if (CVarToolPaletteMode.GetValueOnGameThread() == 0)
	{
		FModeToolkit::RequestModeUITabs();
	}
}

void FMeshTerrainModeToolkit::MakeToolShutdownOverlayWidget()
{
	const FSlateBrush* OverlayBrush = FAppStyle::Get().GetBrush("EditorViewport.OverlayBrush");
	// If there is another mode, it might also have an overlay, and we would like ours to be opaque in that case
	// to draw on top cleanly (e.g., level instance editing mode has an overlay in the same place. Note that level
	// instance mode currently marks itself as not visible despite the overlay, so we shouldn't use IsOnlyVisibleActiveMode)
	if (!GetEditorModeManager().IsOnlyActiveMode(UMeshTerrainMode::EM_MeshTerrainEditorModeId))
	{
		OverlayBrush = FMeshTerrainModeStyle::Get()->GetBrush("MeshTerrainMode.OpaqueOverlayBrush");
	}

	// Helpers to determine button/label visibility based on overrides
	auto GetSubActionIcon = [this]() -> const FSlateBrush*
	{
		if (AcceptCancelButtonParams.IsSet())
		{
			if (AcceptCancelButtonParams->IconName.IsSet())
			{
				return FMeshTerrainModeStyle::Get()->GetOptionalBrush(AcceptCancelButtonParams->IconName.GetValue(), nullptr, nullptr);
			}
		}
		else if (CompleteButtonParams.IsSet() && CompleteButtonParams->IconName.IsSet())
		{
			return FMeshTerrainModeStyle::Get()->GetOptionalBrush(CompleteButtonParams->IconName.GetValue(), nullptr, nullptr);
		}
		return nullptr;
	};
	auto GetSubActionIconVisibility = [this]()
	{
		if (AcceptCancelButtonParams.IsSet())
		{
			if (AcceptCancelButtonParams->IconName.IsSet() 
				&& FMeshTerrainModeStyle::Get()->GetOptionalBrush(AcceptCancelButtonParams->IconName.GetValue(), nullptr, nullptr))
			{
				return EVisibility::Visible;
			}
		}
		else if (CompleteButtonParams.IsSet() && CompleteButtonParams->IconName.IsSet()
			&& FMeshTerrainModeStyle::Get()->GetOptionalBrush(CompleteButtonParams->IconName.GetValue(), nullptr, nullptr))
		{
			return EVisibility::Visible;
		}
		return EVisibility::Collapsed;
	};
	auto GetSubActionLabel = [this]()
	{
		return AcceptCancelButtonParams.IsSet() ? AcceptCancelButtonParams->Label
			: CompleteButtonParams.IsSet() ? CompleteButtonParams->Label
			: FText::GetEmpty();
	};
	auto GetSubActionLabelVisibility = [this]()
	{
		return (AcceptCancelButtonParams.IsSet() || CompleteButtonParams.IsSet()) ?
			EVisibility::Visible : EVisibility::Collapsed;
	};
	auto GetAcceptButtonText = [this]()
	{
		return AcceptCancelButtonParams.IsSet() && AcceptCancelButtonParams->OverrideAcceptButtonText.IsSet() ?
			AcceptCancelButtonParams->OverrideAcceptButtonText.GetValue() : LOCTEXT("OverlayAccept", "Accept");
	};
	auto GetAcceptButtonTooltip = [this]()
	{
		return AcceptCancelButtonParams.IsSet() && AcceptCancelButtonParams->OverrideAcceptButtonTooltip.IsSet() ?
			AcceptCancelButtonParams->OverrideAcceptButtonTooltip.GetValue()
			: LOCTEXT("OverlayAcceptTooltip", "Accept/Commit the results of the active Tool [Enter]");
	};
	auto GetAcceptButtonEnabled = [this]()
	{
		return AcceptCancelButtonParams.IsSet() ? AcceptCancelButtonParams->CanAccept()
			: GetScriptableEditorMode()->GetInteractiveToolsContext(EToolsContextScope::EdMode)->CanAcceptActiveTool();
	};
	auto GetAcceptCancelButtonVisibility = [this]()
	{
		if (AcceptCancelButtonParams.IsSet()
			|| (!CompleteButtonParams.IsSet()
				&& GetScriptableEditorMode()->GetInteractiveToolsContext(EToolsContextScope::EdMode)->ActiveToolHasAccept()))
		{
			return EVisibility::Visible;
		}
		return EVisibility::Collapsed;
	};
	auto GetCancelButtonText = [this]()
	{
		return AcceptCancelButtonParams.IsSet() && AcceptCancelButtonParams->OverrideCancelButtonText.IsSet() ?
			AcceptCancelButtonParams->OverrideCancelButtonText.GetValue() : LOCTEXT("OverlayCancel", "Cancel");
	};
	auto GetCancelButtonTooltip = [this]()
	{
		if (AcceptCancelButtonParams.IsSet() && AcceptCancelButtonParams->OverrideCancelButtonTooltip.IsSet())
		{
			return AcceptCancelButtonParams->OverrideCancelButtonTooltip.GetValue();
		}
		return GetDefault<UMeshTerrainModeCustomizationSettings>()->bEscapeAcceptsToolResult
			? LOCTEXT("OverlayCancelTooltip_NoEsc", "Cancel the active Tool")
			: LOCTEXT("OverlayCancelTooltip", "Cancel the active Tool [Esc]");
	};
	auto GetCancelButtonEnabled = [this]() 
	{
		return AcceptCancelButtonParams.IsSet()
			|| GetScriptableEditorMode()->GetInteractiveToolsContext(EToolsContextScope::EdMode)->CanCancelActiveTool();
	};
	auto GetCompleteButtonText = [this]()
	{
		return CompleteButtonParams.IsSet() && CompleteButtonParams->OverrideCompleteButtonText.IsSet() ?
			CompleteButtonParams->OverrideCompleteButtonText.GetValue() : LOCTEXT("OverlayComplete", "Complete");
	};
	auto GetCompleteButtonTooltip = [this]()
	{
		return CompleteButtonParams.IsSet() && CompleteButtonParams->OverrideCompleteButtonTooltip.IsSet() ?
			CompleteButtonParams->OverrideCompleteButtonTooltip.GetValue()
			: LOCTEXT("OverlayCompleteTooltip", "Exit the active Tool [Enter]");
	};
	auto GetCompleteButtonEnabled = [this]()
	{
		return CompleteButtonParams.IsSet()
			|| GetScriptableEditorMode()->GetInteractiveToolsContext(EToolsContextScope::EdMode)->CanCompleteActiveTool();
	};
	auto GetCompleteButtonVisibility = [this]()
	{
		if (CompleteButtonParams.IsSet()
			|| (!AcceptCancelButtonParams.IsSet()
				&& GetScriptableEditorMode()->GetInteractiveToolsContext(EToolsContextScope::EdMode)->CanCompleteActiveTool()))
		{
			return EVisibility::Visible;
		}
		return EVisibility::Collapsed;
	};

	SAssignNew(ToolShutdownViewportOverlayWidget, SHorizontalBox)

	+SHorizontalBox::Slot()
	.HAlign(HAlign_Center)
	.VAlign(VAlign_Bottom)
	.Padding(FMargin(0.0f, 0.0f, 0.f, 15.f))
	[
		SNew(SBorder)
		.BorderImage(OverlayBrush)
		.Padding(8.f)
		[
			SNew(SHorizontalBox)

			// Tool icon and name
			+SHorizontalBox::Slot()
			.AutoWidth()
			.VAlign(VAlign_Center)
			.Padding(FMargin(0.f, 0.f, 8.f, 0.f))
			[
				SNew(SImage)
				.Image_Lambda([this] () { return ActiveToolIcon; })
			]
			+SHorizontalBox::Slot()
			.AutoWidth()
			.VAlign(VAlign_Center)
			.Padding(FMargin(0.f, 0.f, 8.f, 0.f))
			[
				SNew(STextBlock)
				.Text(this, &FMeshTerrainModeToolkit::GetActiveToolDisplayName)
			]

			// Optional: "-> [icon] SubtoolAction"
			// arrow
			+SHorizontalBox::Slot()
			.AutoWidth()
			.VAlign(EVerticalAlignment::VAlign_Center)
			.Padding(FMargin(0., 0.f, 8.f, 0.f))
			[
				SNew(SImage)
				.Image(FMeshTerrainModeStyle::Get()->GetBrush("MeshTerrainMode.SubToolArrow"))
				.ColorAndOpacity(FSlateColor::UseForeground())
				.Visibility_Lambda(GetSubActionLabelVisibility)
			]
			// subaction icon
			+SHorizontalBox::Slot()
			.AutoWidth()
			.VAlign(VAlign_Center)
			.Padding(FMargin(0.f, 0.f, 8.f, 0.f))
			[
				SNew(SImage)
				.Image_Lambda(GetSubActionIcon)
				.Visibility_Lambda(GetSubActionIconVisibility)
			]
			// subaction label
			+SHorizontalBox::Slot()
			.AutoWidth()
			.VAlign(VAlign_Center)
			.Padding(FMargin(0.f, 0.f, 8.f, 0.f))
			[
				SNew(STextBlock)
				.Text_Lambda(GetSubActionLabel)
				.Visibility_Lambda(GetSubActionLabelVisibility)
			]

			// Buttons:
			// Accept
			+SHorizontalBox::Slot()
			.AutoWidth()
			.VAlign(VAlign_Center)
			.Padding(FMargin(0.0, 0.f, 2.f, 0.f))
			[
				SNew(SPrimaryButton)
				.Text_Lambda(GetAcceptButtonText)
				.ToolTipText_Lambda(GetAcceptButtonTooltip)
				.OnClicked_Raw(this, &FMeshTerrainModeToolkit::HandleAcceptCancelClick, true)
				.IsEnabled_Lambda(GetAcceptButtonEnabled)
				.Visibility_Lambda(GetAcceptCancelButtonVisibility)
			]
			// Cancel
			+SHorizontalBox::Slot()
			.AutoWidth()
			.VAlign(VAlign_Center)
			.Padding(FMargin(2.0, 0.f, 0.f, 0.f))
			[
				SNew(SButton)
				.Text_Lambda(GetCancelButtonText)
				.ToolTipText_Lambda(GetCancelButtonTooltip)
				.HAlign(HAlign_Center)
				.OnClicked_Raw(this, &FMeshTerrainModeToolkit::HandleAcceptCancelClick, false)
				.IsEnabled_Lambda(GetCancelButtonEnabled)
				.Visibility_Lambda(GetAcceptCancelButtonVisibility)
			]
			// Complete
			+SHorizontalBox::Slot()
			.AutoWidth()
			.VAlign(VAlign_Center)
			.Padding(FMargin(2.0, 0.f, 0.f, 0.f))
			[
				SNew(SPrimaryButton)
				.Text_Lambda(GetCompleteButtonText)
				.ToolTipText_Lambda(GetCompleteButtonTooltip)
				.OnClicked_Raw(this, &FMeshTerrainModeToolkit::HandleCompleteClick)
				.IsEnabled_Lambda(GetCompleteButtonEnabled)
				.Visibility_Lambda(GetCompleteButtonVisibility)
			]
		]	
	];

}


void FMeshTerrainModeToolkit::MakeGizmoNumericalUIOverlayWidget()
{
	GizmoNumericalUIOverlayWidget = SNew(STransformGizmoNumericalUIOverlay)
		.DefaultLeftPadding(15)
		// Position above the little axis visualization
		.DefaultVerticalPadding(75)
		.bPositionRelativeToBottom(true);
}


TSharedRef<SWidget> FMeshTerrainModeToolkit::MakePresetComboWidget(TSharedPtr<FString> InItem)
{
	return
		SNew(STextBlock)
		.Text(FText::FromString(*InItem));
}

TSharedPtr<SWidget> FMeshTerrainModeToolkit::MakePresetPanel()
{
	const TSharedPtr<SVerticalBox> Content = SNew(SVerticalBox);
	UMeshTerrainModeSettings* Settings = GetMutableDefault<UMeshTerrainModeSettings>();

	bool bEnableToolPresets = (CVarEnableToolPresets.GetValueOnGameThread() > 0);	
	if (!bEnableToolPresets || Settings->InRestrictiveMode())
	{
		return SNew(SVerticalBox);
	}

	const TSharedPtr<SHorizontalBox> NewContent = SNew(SHorizontalBox);
	
	auto IsToolActive = [this]() {
		if (this->OwningEditorMode.IsValid())
		{
			return this->OwningEditorMode->GetToolManager()->GetActiveTool(EToolSide::Left) != nullptr ? EVisibility::Visible : EVisibility::Collapsed;
		}
		return EVisibility::Collapsed;
	};

	NewContent->AddSlot().HAlign(HAlign_Right)
	[
		SNew(SComboButton)
		.ComboButtonStyle(&FAppStyle::Get().GetWidgetStyle<FComboButtonStyle>("SimpleComboButton"))
		.OnGetMenuContent(this, &FMeshTerrainModeToolkit::GetPresetCreateButtonContent)
		.HasDownArrow(true)
		.Visibility_Lambda(IsToolActive)
		.ButtonContent()
		[
			SNew(SHorizontalBox)
			+ SHorizontalBox::Slot()
			.AutoWidth()
			.VAlign(VAlign_Center)
			.Padding(4.0, 0.0f)
			[
				SNew(STextBlock)
				.Text(LOCTEXT("ModelingPresetPanelHeader", "Presets"))
				.Justification(ETextJustify::Center)
				.Font(FAppStyle::Get().GetFontStyle("EditorModesPanel.CategoryFontStyle"))
			]

		]
	];

	TSharedPtr<SHorizontalBox> AssetConfigPanel = SNew(SHorizontalBox)
		+ SHorizontalBox::Slot()
		[
			NewContent->AsShared()
		];

	return AssetConfigPanel;
}

bool FMeshTerrainModeToolkit::IsPresetEnabled() const
{
	return CurrentPresetPath.IsAsset();
}


TSharedRef<SWidget> FMeshTerrainModeToolkit::GetPresetCreateButtonContent()
{
	RebuildPresetListForTool(false);

		auto OpenNewPresetDialog = [this]()
	{
		NewPresetLabel.Empty();
		NewPresetTooltip.Empty();

		// Set the result if they just click Ok
		SGenericDialogWidget::FArguments FolderDialogArguments;
		FolderDialogArguments.OnOkPressed_Lambda([this]()
			{
				CreateNewPresetInCollection(NewPresetLabel,
					CurrentPresetPath,
					NewPresetTooltip,
					NewPresetIcon);
			});

		// Present the Dialog
		SGenericDialogWidget::OpenDialog(LOCTEXT("ToolPresets_CreatePreset", "Create new preset from active tool's settings"),
			SNew(SVerticalBox)
			+ SVerticalBox::Slot()
			.AutoHeight()
			.Padding(5.0f)
			[
				SNew(SHorizontalBox)
				+ SHorizontalBox::Slot()
				[
					SNew(STextBlock)
					.Text(LOCTEXT("ToolPresets_CreatePresetLabel", "Label"))
				]
				+ SHorizontalBox::Slot()
				.MaxWidth(300)
				[
					SNew(SEditableTextBox)
					// Cap the number of characters sent out of the text box, so we don't overflow menus and tooltips
					.OnTextCommitted_Lambda([this](const FText& NewLabel, const ETextCommit::Type&) { NewPresetLabel = NewLabel.ToString().Left(255); })
					.OverflowPolicy(ETextOverflowPolicy::Ellipsis)
					.ToolTipText(LOCTEXT("ToolPresets_CreatePresetLabel_Tooltip", "A short, descriptive identifier for the new preset."))
				]
			]
		+ SVerticalBox::Slot()
			.AutoHeight()
			.Padding(5.0f)
			[
				SNew(SHorizontalBox)
				+ SHorizontalBox::Slot()
				[
					SNew(STextBlock)
					.Text(LOCTEXT("ToolPresets_CreatePresetTooltip", "Tooltip"))
				]
				+ SHorizontalBox::Slot()
				.MaxWidth(300)
				[
					SNew(SBox)
					.MinDesiredHeight(44.f)
					.MaxDesiredHeight(44.0f)
					[
						SNew(SMultiLineEditableTextBox)
						// Cap the number of characters sent out of the text box, so we don't overflow menus and tooltips
						.OnTextCommitted_Lambda([this](const FText& NewToolTip, const ETextCommit::Type&) { NewPresetTooltip = NewToolTip.ToString().Left(2048); })
						.OverflowPolicy(ETextOverflowPolicy::Ellipsis)
						.AllowMultiLine(false)
						.AutoWrapText(true)
						.WrappingPolicy(ETextWrappingPolicy::DefaultWrapping)
						.OverflowPolicy(ETextOverflowPolicy::Ellipsis)
						.ToolTipText(LOCTEXT("ToolPresets_CreatePresetTooltip_Tooltip", "A descriptive tooltip for the new preset."))
					]
				]
			]
		+ SVerticalBox::Slot()
			.AutoHeight()
			.Padding(5.0f)
			[
				SNew(SHorizontalBox)
				+ SHorizontalBox::Slot()
				.AutoWidth()
				[
					SNew(SToolInputAssetComboPanel)
					.AssetClassType(UInteractiveToolsPresetCollectionAsset::StaticClass())
					.OnSelectionChanged(this, &FMeshTerrainModeToolkit::HandlePresetAssetChanged)
					.ToolTipText(LOCTEXT("ToolPresets_CreatePresetCollection_Tooltip", "The asset in which to store this new preset."))
					//.RecentAssetsProvider(RecentPresetCollectionProvider) // TODO: Improve this widget before enabling this feature
					.InitiallySelectedAsset(*CurrentPreset)
					.FlyoutTileSize(FVector2D(80, 80))
					.ComboButtonTileSize(FVector2D(80, 80))
					.AssetThumbnailLabel(EThumbnailLabel::AssetName)
					.bForceShowPluginContent(true)
					.bForceShowEngineContent(true)
					.AssetViewType(EAssetViewType::List)
				]
				+ SHorizontalBox::Slot()
				[
					SNew(SVerticalBox)
					+ SVerticalBox::Slot()
					.AutoHeight()
					.Padding(10, 5)
					[
						SNew(STextBlock)
						.Text(LOCTEXT("ToolPresets_CreatePresetCollection", "Collection"))
						.Font(FSlateFontInfo(FCoreStyle::GetDefaultFont(),12, "Bold"))
					]
					+ SVerticalBox::Slot()
					.AutoHeight()
					.Padding(10, 5)
					[
						SNew(STextBlock)
						.Text_Lambda([this](){
						if (CurrentPresetLabel.IsEmpty())
						{
							return LOCTEXT("NewPresetNoCollectionSpecifiedMessage", "None - Preset will be added to the default Personal Presets Collection.");
						}
						else {
							return CurrentPresetLabel;
						}})
					]

				]					
			],
			FolderDialogArguments, true);
	};

	constexpr bool bShouldCloseMenuAfterSelection = true;
	FMenuBuilder MenuBuilder(bShouldCloseMenuAfterSelection, nullptr);

	constexpr bool bNoIndent = false;
	constexpr bool bSearchable = false;
	static const FName NoExtensionHook = NAME_None;


	{
		FMenuEntryParams MenuEntryParams;

		typedef TMap<FString, TArray<TSharedPtr<FToolPresetOption>>> FPresetsByNameMap;
		FPresetsByNameMap PresetsByCollectionName;
		for (TSharedPtr<FToolPresetOption> ToolPresetOption : AvailablePresetsForTool)
		{
			FMeshTerrainModeToolkitLocals::ExecuteWithPreset(ToolPresetOption->PresetCollection,
				[this, &PresetsByCollectionName, &ToolPresetOption](UInteractiveToolsPresetCollectionAsset& Preset) {
					PresetsByCollectionName.FindOrAdd(Preset.CollectionLabel.ToString()).Add(ToolPresetOption);
				});
		}

		for (FPresetsByNameMap::TConstIterator Iterator = PresetsByCollectionName.CreateConstIterator(); Iterator; ++Iterator)
		{
			MenuBuilder.BeginSection(NoExtensionHook, FText::FromString(Iterator.Key()));
			for (const TSharedPtr<FToolPresetOption>& ToolPresetOption : Iterator.Value())
			{
				FUIAction ApplyPresetAction;
				ApplyPresetAction.ExecuteAction = FExecuteAction::CreateLambda([this, ToolPresetOption]()
					{
						LoadPresetFromCollection(ToolPresetOption->PresetIndex, ToolPresetOption->PresetCollection);
					});

				ApplyPresetAction.CanExecuteAction = FCanExecuteAction::CreateLambda([this, ToolPresetOption]()
					{
						return this->OwningEditorMode->GetToolManager()->GetActiveTool(EToolSide::Left) != nullptr;
					});

				MenuBuilder.AddMenuEntry(
					FText::FromString(ToolPresetOption->PresetLabel),
					FText::FromString(ToolPresetOption->PresetTooltip),
					ToolPresetOption->PresetIcon,
					ApplyPresetAction);
			}
			MenuBuilder.EndSection();
		}

		MenuBuilder.BeginSection(NoExtensionHook, LOCTEXT("ModelingPresetPanelHeaderManagePresets", "Manage Presets"));


		FUIAction CreateNewPresetAction;
		CreateNewPresetAction.ExecuteAction = FExecuteAction::CreateLambda(OpenNewPresetDialog);
		MenuBuilder.AddMenuEntry(
			FText(LOCTEXT("ModelingPresetPanelCreateNewPreset", "Create New Preset")),
			FText(LOCTEXT("ModelingPresetPanelCreateNewPresetTooltip", "Create New Preset in specified Collection")),
			FSlateIcon( FAppStyle::Get().GetStyleSetName(), "Icons.Plus"),
			CreateNewPresetAction);


		FUIAction OpenPresetManangerAction;
		OpenPresetManangerAction.ExecuteAction = FExecuteAction::CreateLambda([this]() {
			IToolPresetEditorModule::Get().ExecuteOpenPresetEditor();
		});
		MenuBuilder.AddMenuEntry(
			FText(LOCTEXT("ModelingPresetPanelOpenPresetMananger", "Manage Presets...")),
			FText(LOCTEXT("ModelingPresetPanelOpenPresetManagerTooltip", "Open Preset Manager to manage presets and their collections")),
			FSlateIcon(FAppStyle::Get().GetStyleSetName(), "Icons.Settings"),
			OpenPresetManangerAction);

		MenuBuilder.EndSection();

	}

	return MenuBuilder.MakeWidget();
}

void FMeshTerrainModeToolkit::ClearPresetComboList()
{
	AvailablePresetsForTool.Empty();
}

void FMeshTerrainModeToolkit::RebuildPresetListForTool(bool bSettingsOpened)
{	
	TObjectPtr<UToolPresetUserSettings> UserSettings = UToolPresetUserSettings::Get();
	UserSettings->LoadEditorConfig();

	// We need to generate a combined list of Project Loaded and User available presets to intersect the enabled set against...
	const UToolPresetProjectSettings* ProjectSettings = GetDefault<UToolPresetProjectSettings>();
	TSet<FSoftObjectPath> AllUserPresets;
	
	FAssetRegistryModule& AssetRegistryModule = FModuleManager::LoadModuleChecked<FAssetRegistryModule>("AssetRegistry");
	
	TArray<FAssetData> AssetData;
	FARFilter Filter;
	Filter.ClassPaths.Add(UInteractiveToolsPresetCollectionAsset::StaticClass()->GetClassPathName());
	Filter.PackagePaths.Add(FName("/ToolPresets"));
	Filter.bRecursiveClasses = false;
	Filter.bRecursivePaths = true;
	Filter.bIncludeOnlyOnDiskAssets = false;
	
	AssetRegistryModule.Get().GetAssets(Filter, AssetData);
	for (int i = 0; i < AssetData.Num(); i++) {
		UInteractiveToolsPresetCollectionAsset* Object = Cast<UInteractiveToolsPresetCollectionAsset>(AssetData[i].GetAsset());
		if (Object)
		{
			AllUserPresets.Add(Object->GetPathName());
		}
	}

	TSet<FSoftObjectPath> AvailablePresetCollections = UserSettings->EnabledPresetCollections.Intersect( ProjectSettings->LoadedPresetCollections.Union(AllUserPresets));
	if (UserSettings->bDefaultCollectionEnabled)
	{
		AvailablePresetCollections.Add(FSoftObjectPath());
	}


	AvailablePresetsForTool.Empty();
	for (const FSoftObjectPath& PresetCollection : AvailablePresetCollections)
	{
		FMeshTerrainModeToolkitLocals::ExecuteWithPresetAndTool(*OwningEditorMode, EToolSide::Left, PresetCollection,
			[this, PresetCollection](UInteractiveToolsPresetCollectionAsset& Preset, UInteractiveTool& Tool) {
		
				if (!Preset.PerToolPresets.Contains(Tool.GetClass()->GetName()))
				{
					return;
				}
				AvailablePresetsForTool.Reserve(Preset.PerToolPresets[Tool.GetClass()->GetName()].NamedPresets.Num());
				for (int32 PresetIndex = 0; PresetIndex < Preset.PerToolPresets[Tool.GetClass()->GetName()].NamedPresets.Num(); ++PresetIndex)
				{
					const FInteractiveToolPresetDefinition& PresetDef = Preset.PerToolPresets[Tool.GetClass()->GetName()].NamedPresets[PresetIndex];
					if (!PresetDef.IsValid())
					{
						continue;
					}
					TSharedPtr<FToolPresetOption> NewOption = MakeShared<FToolPresetOption>();
					if (PresetDef.Label.Len() > 50)
					{
						NewOption->PresetLabel = PresetDef.Label.Left(50) + FString("...");
					}
					else
					{
						NewOption->PresetLabel = PresetDef.Label;
					}
					if (PresetDef.Tooltip.Len() > 2048)
					{
						NewOption->PresetTooltip = PresetDef.Tooltip.Left(2048) + FString("...");
					}
					else
					{
						NewOption->PresetTooltip = PresetDef.Tooltip;
					}
					NewOption->PresetIndex = PresetIndex;
					NewOption->PresetCollection = PresetCollection;

					AvailablePresetsForTool.Add(NewOption);
				}
			});
	}
}

void FMeshTerrainModeToolkit::HandlePresetAssetChanged(const FAssetData& InAssetData)
{
	CurrentPresetPath.SetPath(InAssetData.GetObjectPathString());
	CurrentPresetLabel = FText();
	*CurrentPreset = InAssetData;

	UInteractiveToolsPresetCollectionAsset* Preset = nullptr;
	if (CurrentPresetPath.IsAsset())
	{
		Preset = Cast<UInteractiveToolsPresetCollectionAsset>(CurrentPresetPath.TryLoad());
	}
	if (Preset)
	{
		CurrentPresetLabel = Preset->CollectionLabel;
	}
	
}

bool FMeshTerrainModeToolkit::HandleFilterPresetAsset(const FAssetData& InAssetData)
{
	return false;
}

void FMeshTerrainModeToolkit::LoadPresetFromCollection(const int32 PresetIndex, FSoftObjectPath CollectionPath)
{
	FMeshTerrainModeToolkitLocals::ExecuteWithPresetAndTool(*OwningEditorMode, EToolSide::Left, CollectionPath,
	[this, PresetIndex](UInteractiveToolsPresetCollectionAsset& Preset, UInteractiveTool& Tool) {
		TArray<UObject*> PropertySets = Tool.GetToolProperties();

		// We only want to load the properties that are actual property sets, since the tool might have added other types of objects we don't
		// want to deserialize.
		PropertySets.RemoveAll([this](UObject* Object) {
			return Cast<UInteractiveToolPropertySet>(Object) == nullptr;
		});

		if (!Preset.PerToolPresets.Contains(Tool.GetClass()->GetName()))			
		{
			return;
		}
		if(PresetIndex < 0 || PresetIndex >= Preset.PerToolPresets[Tool.GetClass()->GetName()].NamedPresets.Num())
		{
			return;
		}

		Preset.PerToolPresets[Tool.GetClass()->GetName()].NamedPresets[PresetIndex].LoadStoredPropertyData(PropertySets);

	});
}

void FMeshTerrainModeToolkit::CreateNewPresetInCollection(const FString& PresetLabel, FSoftObjectPath CollectionPath, const FString& ToolTip, FSlateIcon Icon)
{
	FMeshTerrainModeToolkitLocals::ExecuteWithPresetAndTool(*OwningEditorMode, EToolSide::Left, CurrentPresetPath,
	[this, PresetLabel, ToolTip, Icon, CollectionPath](UInteractiveToolsPresetCollectionAsset& Preset, UInteractiveTool& Tool) {

		TArray<UObject*> PropertySets = Tool.GetToolProperties();

		// We only want to add the properties that are actual property sets, since the tool might have added other types of objects we don't
		// want to serialize.
		PropertySets.RemoveAll([this](UObject* Object) {
			return Cast<UInteractiveToolPropertySet>(Object) == nullptr;
		});

		Preset.PerToolPresets.FindOrAdd(Tool.GetClass()->GetName()).ToolLabel = ActiveToolName;
		if (ensure(ActiveToolIcon))
		{
			Preset.PerToolPresets.FindOrAdd(Tool.GetClass()->GetName()).ToolIcon = *ActiveToolIcon;
		}
		FInteractiveToolPresetDefinition PresetValuesToCreate;
		if (PresetLabel.IsEmpty())
		{
			PresetValuesToCreate.Label = FString::Printf(TEXT("Unnamed_Preset-%d"), Preset.PerToolPresets.FindOrAdd(Tool.GetClass()->GetName()).NamedPresets.Num()+1);
		}
		else
		{
			PresetValuesToCreate.Label = PresetLabel;
		}
		PresetValuesToCreate.Tooltip = ToolTip;
		//PresetValuesToCreate.Icon = Icon;

		PresetValuesToCreate.SetStoredPropertyData(PropertySets);


		Preset.PerToolPresets[Tool.GetClass()->GetName()].NamedPresets.Add(PresetValuesToCreate);
		Preset.MarkPackageDirty();

		// Finally add this to the current tool's preset list, since we know it should be there.
		TSharedPtr<FToolPresetOption> NewOption = MakeShared<FToolPresetOption>();
		NewOption->PresetLabel = PresetLabel;
		NewOption->PresetTooltip = ToolTip;
		NewOption->PresetIndex = Preset.PerToolPresets[Tool.GetClass()->GetName()].NamedPresets.Num() - 1;
		NewOption->PresetCollection = CollectionPath;

		AvailablePresetsForTool.Add(NewOption);
		
	});

	UToolPresetAssetSubsystem* PresetAssetSubsystem = GEditor->GetEditorSubsystem<UToolPresetAssetSubsystem>();
	if (CollectionPath.IsNull() && ensure(PresetAssetSubsystem))
	{
		ensure(PresetAssetSubsystem->SaveDefaultCollection());
	}
}

void FMeshTerrainModeToolkit::UpdatePresetInCollection(const FToolPresetOption& PresetToEditIn, bool bUpdateStoredPresetValues)
{
	FMeshTerrainModeToolkitLocals::ExecuteWithPresetAndTool(*OwningEditorMode, EToolSide::Left, PresetToEditIn.PresetCollection,
		[this, PresetToEditIn, bUpdateStoredPresetValues](UInteractiveToolsPresetCollectionAsset& Preset, UInteractiveTool& Tool) {
			TArray<UObject*> PropertySets = Tool.GetToolProperties();

			// We only want to add the properties that are actual property sets, since the tool might have added other types of objects we don't
			// want to serialize.
			PropertySets.RemoveAll([this](UObject* Object) {
				return Cast<UInteractiveToolPropertySet>(Object) == nullptr;
			});

			if (!Preset.PerToolPresets.Contains(Tool.GetClass()->GetName()))
			{
				return;
			}
			if (PresetToEditIn.PresetIndex < 0 || PresetToEditIn.PresetIndex >= Preset.PerToolPresets[Tool.GetClass()->GetName()].NamedPresets.Num() )
			{
				return;
			}

			if (bUpdateStoredPresetValues)
			{
				Preset.PerToolPresets.FindOrAdd(Tool.GetClass()->GetName()).NamedPresets[PresetToEditIn.PresetIndex].SetStoredPropertyData(PropertySets);
				Preset.MarkPackageDirty();
			}

			Preset.PerToolPresets.FindOrAdd(Tool.GetClass()->GetName()).NamedPresets[PresetToEditIn.PresetIndex].Label = PresetToEditIn.PresetLabel;
			Preset.PerToolPresets.FindOrAdd(Tool.GetClass()->GetName()).NamedPresets[PresetToEditIn.PresetIndex].Tooltip = PresetToEditIn.PresetTooltip;

		});

	RebuildPresetListForTool(false);

	UToolPresetAssetSubsystem* PresetAssetSubsystem = GEditor->GetEditorSubsystem<UToolPresetAssetSubsystem>();
	if (PresetToEditIn.PresetCollection.IsNull() && ensure(PresetAssetSubsystem))
	{
		ensure(PresetAssetSubsystem->SaveDefaultCollection());
	}
}




void FMeshTerrainModeToolkit::InitializeAfterModeSetup()
{
	if (bFirstInitializeAfterModeSetup)
	{
		bFirstInitializeAfterModeSetup = false;
	}
}

void FMeshTerrainModeToolkit::UninitializeOnModeDeactivate()
{
	// Restore LevelViewportSplitter layout config
	if (TSharedPtr<SSplitter> Splitter = LevelViewportSplitter.Pin())
	{
		if (Splitter->IsValidSlotIndex(LevelViewportSplitterSlot))
		{
			Splitter->SlotAt(LevelViewportSplitterSlot).SetSizingRule(SSplitter::FractionOfParent);
			Splitter->SlotAt(LevelViewportSplitterSlot).SetResizable(true);
		}
	}
	LevelViewportSplitterSlot = INDEX_NONE;
	LevelViewportSplitter.Reset();
}


void FMeshTerrainModeToolkit::UpdateActiveToolProperties()
{
	UInteractiveTool* CurTool = GetScriptableEditorMode()->GetToolManager(EToolsContextScope::EdMode)->GetActiveTool(EToolSide::Left);
	if (CurTool == nullptr)
	{
		return;
	}

	// Before actually changing the detail panel, we need to see where the current keyboard focus is, because
	// if it's inside the detail panel, we'll need to reset it to the detail panel as a whole, else we might
	// lose it entirely when that detail panel element gets destroyed (which would make us unable to receive any
	// hotkey presses until the user clicks somewhere).
	TSharedPtr<SWidget> FocusedWidget = FSlateApplication::Get().GetKeyboardFocusedWidget();
	if (FocusedWidget != ModeDetailsView) 
	{
		// Search upward from the currently focused widget
		TSharedPtr<SWidget> CurrentWidget = FocusedWidget;
		while (CurrentWidget.IsValid())
		{
			if (CurrentWidget == ModeDetailsView)
			{
				// Reset focus to the detail panel as a whole to avoid losing it when the inner elements change.
				FSlateApplication::Get().SetKeyboardFocus(ModeDetailsView);
				break;
			}

			CurrentWidget = CurrentWidget->GetParentWidget();
		}
	}
		
	ModeDetailsView->SetObjects(CurTool->GetToolProperties(true));
	if (SectionedDetailsView.IsValid())
	{
		SectionedDetailsView->SetObjects(CurTool->GetToolProperties(true));
	}
}

void FMeshTerrainModeToolkit::InvalidateCachedDetailPanelState(UObject* ChangedObject)
{
	ModeDetailsView->InvalidateCachedState();
}


void FMeshTerrainModeToolkit::PostNotification(const FText& Message)
{
	ClearNotification();

	ActiveToolMessage = Message;

	if (ModeUILayer.IsValid())
	{
		TSharedPtr<FAssetEditorModeUILayer> ModeUILayerPtr = ModeUILayer.Pin();
		ActiveToolMessageHandle = GEditor->GetEditorSubsystem<UStatusBarSubsystem>()->PushStatusBarMessage(ModeUILayerPtr->GetStatusBarName(), ActiveToolMessage);
	}
}

void FMeshTerrainModeToolkit::ClearNotification()
{
	ActiveToolMessage = FText::GetEmpty();

	if (ModeUILayer.IsValid())
	{
		TSharedPtr<FAssetEditorModeUILayer> ModeUILayerPtr = ModeUILayer.Pin();
		GEditor->GetEditorSubsystem<UStatusBarSubsystem>()->PopStatusBarMessage(ModeUILayerPtr->GetStatusBarName(), ActiveToolMessageHandle);
	}
	ActiveToolMessageHandle.Reset();
}


void FMeshTerrainModeToolkit::PostWarning(const FText& Message)
{
	ToolWarningArea->SetText(Message);
	ToolWarningArea->SetVisibility(EVisibility::Visible);
}

void FMeshTerrainModeToolkit::ClearWarning()
{
	ToolWarningArea->SetText(FText());
	ToolWarningArea->SetVisibility(EVisibility::Collapsed);
}

FName FMeshTerrainModeToolkit::GetToolkitFName() const
{
	return FName("ModelingToolsEditorMode");
}

FText FMeshTerrainModeToolkit::GetBaseToolkitName() const
{
	return NSLOCTEXT("ModelingToolsEditorModeToolkit", "DisplayName", "ModelingToolsEditorMode Tool");
}

static const FName PrimitiveTabName(TEXT("Shapes"));
static const FName CreateTabName(TEXT("Create"));
static const FName AttributesTabName(TEXT("Attributes"));
static const FName TriModelingTabName(TEXT("TriModel"));
static const FName PolyModelingTabName(TEXT("PolyModel"));
static const FName MeshProcessingTabName(TEXT("MeshOps"));
static const FName UVTabName(TEXT("UVs"));
static const FName TransformTabName(TEXT("Transform"));
static const FName DeformTabName(TEXT("Deform"));
static const FName VolumesTabName(TEXT("Volumes"));
static const FName PrototypesTabName(TEXT("Prototypes"));
static const FName PolyEditTabName(TEXT("PolyEdit"));
static const FName VoxToolsTabName(TEXT("VoxOps"));
static const FName LODToolsTabName(TEXT("LODs"));
static const FName BakingToolsTabName(TEXT("Baking"));
static const FName ModelingFavoritesTabName(TEXT("Favorites"));

static const FName SelectionActionsTabName(TEXT("Selection"));


const TArray<FName> FMeshTerrainModeToolkit::PaletteNames_Standard = { PrimitiveTabName, CreateTabName, PolyModelingTabName, TriModelingTabName, DeformTabName, TransformTabName, MeshProcessingTabName, VoxToolsTabName, AttributesTabName, UVTabName, BakingToolsTabName, VolumesTabName, LODToolsTabName };


void FMeshTerrainModeToolkit::GetToolPaletteNames(TArray<FName>& PaletteNames) const
{
	PaletteNames = PaletteNames_Standard;

	TArray<FName> ExistingNames;
	for ( FName Name : PaletteNames )
	{
		ExistingNames.Add(Name);
	}

	const UMeshTerrainModeSettings* ModelingModeSettings = GetDefault<UMeshTerrainModeSettings>();
	bool bEnableSelectionUI = ModelingModeSettings && ModelingModeSettings->GetMeshSelectionsEnabled();
	if (bEnableSelectionUI)
	{
		if (bShowActiveSelectionActions)
		{
			PaletteNames.Insert(SelectionActionsTabName, 0);
			ExistingNames.Add(SelectionActionsTabName);
		}
	}

	if (IModularFeatures::Get().IsModularFeatureAvailable(IMeshTerrainModeToolExtension::GetModularFeatureName()))
	{
		TArray<IMeshTerrainModeToolExtension*> Extensions = IModularFeatures::Get().GetModularFeatureImplementations<IMeshTerrainModeToolExtension>(
			IMeshTerrainModeToolExtension::GetModularFeatureName());
		for (int32 k = 0; k < Extensions.Num(); ++k)
		{
			FText ExtensionName = Extensions[k]->GetExtensionName();
			FText SectionName = Extensions[k]->GetToolSectionName();
			FName SectionIndex(SectionName.ToString());
			if (ExistingNames.Contains(SectionIndex))
			{
				UE_LOGF(LogTemp, Warning, "Modeling Mode Extension [%ls] uses existing Section Name [%ls] - buttons may not be visible", *ExtensionName.ToString(), *SectionName.ToString());
			}
			else
			{
				PaletteNames.Add(SectionIndex);
				ExistingNames.Add(SectionIndex);
			}
		}
	}
	
	// if user has provided a list of favorite tools, add that palette to the list
	if (FavoritesPalette && FavoritesPalette->GetPaletteCommandNames().Num() > 0)
	{
		PaletteNames.Insert(ModelingFavoritesTabName, 0);
	}

}


FText FMeshTerrainModeToolkit::GetToolPaletteDisplayName(FName Palette) const
{
	return FText::FromName(Palette);
}


void FMeshTerrainModeToolkit::InvokeUI()
{
	FModeToolkit::InvokeUI();

	// FModeToolkit::UpdatePrimaryModePanel() wrapped our GetInlineContent() output in a SScrollBar widget,
	// however this doesn't make sense as we want to dock panels to the "top" and "bottom" of our mode panel area,
	// and the details panel in the middle has it's own scrollbar already. The SScrollBar is hardcoded as the content
	// of FModeToolkit::InlineContentHolder so we can just replace it here
	if (InlineContentHolder.IsValid())
	{
		InlineContentHolder->SetContent(GetInlineContent().ToSharedRef());
	}

	if (CVarToolPaletteMode.GetValueOnGameThread() == 0)
	{
		// Hide the main category content vertical box.
		if (TSharedPtr<SWidget> MainContentWidget = ToolkitSections->ModeWarningArea->GetParentWidget())
		{
			MainContentWidget->SetVisibility(EVisibility::Collapsed);
		}

		// Display only the category toolbar and disable resizable. This must be restored on mode exit.
		if (TSharedPtr<SDockTab> PrimaryTabPtr = PrimaryTab.Pin())
		{
			const TSharedPtr<SDockingTabStack> PrimaryTabStack = PrimaryTabPtr->GetParentDockTabStack();
			if (PrimaryTabStack)
			{
				auto FindParentWidgetByPredicate = [](const TSharedPtr<SWidget>& Widget, TFunctionRef<bool(const TSharedPtr<SWidget>&)> Predicate) -> TSharedPtr<SWidget>
				{
					auto FindParentWidgetByPredicateImpl = [](const auto& Self, const TSharedPtr<SWidget>& Widget, TFunctionRef<bool(const TSharedPtr<SWidget>&)> Predicate) -> TSharedPtr<SWidget>
					{
						if (!Widget)
						{
							return nullptr;
						}

						TSharedPtr<SWidget> ParentWidget = Widget->GetParentWidget();
						return Predicate(ParentWidget) ? ParentWidget : Self(Self, ParentWidget, Predicate);
					};
					return FindParentWidgetByPredicateImpl(FindParentWidgetByPredicateImpl, Widget, Predicate);
				};

				auto DockingTabStackPredicate = [](const TSharedPtr<SWidget>& Widget) -> bool
				{
					const FName DockingTabStackType("SDockingTabStack");
					const FName WidgetType = Widget ? Widget->GetType() : NAME_None;
					return WidgetType == DockingTabStackType;
				};

				if (TSharedPtr<SWidget> ParentTabStack = FindParentWidgetByPredicate(PrimaryTabPtr, DockingTabStackPredicate))
				{
					TSharedPtr<SWidget> LastSplitterChild = ParentTabStack;
					auto HorizontalSplitterTabStackPredicate = [&LastSplitterChild](const TSharedPtr<SWidget>& Widget) -> bool
					{
						const FName SplitterType("SSplitter");
						const FName WidgetType = Widget ? Widget->GetType() : NAME_None;
						if (WidgetType == SplitterType)
						{
							TSharedPtr<SSplitter> Splitter = StaticCastSharedPtr<SSplitter>(Widget);
							return Splitter->GetOrientation() == Orient_Horizontal;
						}
						LastSplitterChild = Widget;
						return false;
					};
					
					if (TSharedPtr<SWidget> LevelViewportSplitterWidget = FindParentWidgetByPredicate(ParentTabStack, HorizontalSplitterTabStackPredicate))
					{
						TSharedPtr<SSplitter> Splitter = StaticCastSharedPtr<SSplitter>(LevelViewportSplitterWidget);
						// Identify the splitter slot that houses the docking tab stack for this tab.
						FChildren* SplitterChildren = Splitter->GetChildren();
						for (int Id = 0; Id < SplitterChildren->Num(); ++Id)
						{
							if (LastSplitterChild == SplitterChildren->GetChildAt(Id))
							{
								if (Splitter->IsValidSlotIndex(Id))
								{
									Splitter->SlotAt(Id).SetSizingRule(SSplitter::SizeToContent);
									Splitter->SlotAt(Id).SetResizable(false);
									LevelViewportSplitterSlot = Id;
								}
							}
						}
						LevelViewportSplitter = Splitter;
					}
				}
			}
		}
	}
}

void FMeshTerrainModeToolkit::ShutdownUI()
{
	// On switching editor modes, OnBroadcastEditorModeIDChanged is processed by UModeManagerInteractive
	// In order to auto accept tool results on mode exit, we must handle it here. Processing in UEdMode::Exit
	// is too late because it is the UModeManagerInteractiveToolsContext::OnChildEdModeDeactivated that force
	// deactivates all tools w/ Cancel when a new Mode is activated.
	// 
	// ShutdownUI is invoked during OnToolkitHostShutdown that occurs when a Toolkit is "closed" (but not destroyed)
	// which occurs before OnChildEdModeDeactivated.
	UMeshTerrainModeCustomizationSettings* UISettings = GetMutableDefault<UMeshTerrainModeCustomizationSettings>();
	if (UISettings->bExitModeAcceptsToolResult)
	{
		if (UInteractiveToolManager* ToolManager = GetScriptableEditorMode()->GetToolManager(EToolsContextScope::EdMode);
			ToolManager && ToolManager->HasActiveTool(EToolSide::Left))
		{
			const EToolShutdownType ShutdownType = UE::GetIsEditorLoadingPackage() ? EToolShutdownType::Cancel : EToolShutdownType::Accept;
			ToolManager->DeactivateTool(EToolSide::Left, ShutdownType);
		}
	}
}

void FMeshTerrainModeToolkit::BindGizmoNumericalUI()
{
	if (ensure(GizmoNumericalUIOverlayWidget.IsValid()))
	{
		ensure(GizmoNumericalUIOverlayWidget->BindToGizmoContextObject(GetScriptableEditorMode()->GetInteractiveToolsContext(EToolsContextScope::EdMode)));
	}
}


UGeometrySelectionManager* FMeshTerrainModeToolkit::GetMeshSelectionManager()
{
	if (CachedSelectionManager == nullptr)
	{
		CachedSelectionManager = Cast<UMeshTerrainMode>(GetScriptableEditorMode())->GetSelectionManager();
	}
	return CachedSelectionManager;
}


void FMeshTerrainModeToolkit::OnToolPaletteChanged(FName PaletteName) 
{
}


bool FMeshTerrainModeToolkit::ExperimentalToolsEnabled() const
{
	return CVarEnableExperimentalTools.GetValueOnGameThread() == 1;
}


void FMeshTerrainModeToolkit::ShowRealtimeAndModeWarnings(bool bShowRealtimeWarning)
{
	FText WarningText{};
	if (GEditor->bIsSimulatingInEditor)
	{
		WarningText = LOCTEXT("ModelingModeToolkitSimulatingWarning", "Cannot use Modeling Tools while simulating.");
	}
	else if (GEditor->PlayWorld != NULL)
	{
		WarningText = LOCTEXT("ModelingModeToolkitPIEWarning", "Cannot use Modeling Tools in PIE.");
	}
	else if (bShowRealtimeWarning)
	{
		WarningText = LOCTEXT("ModelingModeToolkitRealtimeWarning", "Realtime Mode is required for Modeling Tools to work correctly. Please enable Realtime Mode in the Viewport Options or with the Ctrl+r hotkey.");
	}
	else if (UEdMode* EdMode = GetScriptableEditorMode().Get())
	{
		if (UWorld* World = EdMode->GetWorld())
		{
			if (!World->GetWorldPartition())
			{
				WarningText = LOCTEXT("MeshTerrainModeWorldPartitionWarning", "Mesh Terrain Mode tools require World Partition to be enabled.");
			}
		}
	}
	if (!WarningText.IdenticalTo(ActiveWarning))
	{
		ActiveWarning = WarningText;
		ModeWarningArea->SetVisibility(ActiveWarning.IsEmpty() ? EVisibility::Collapsed : EVisibility::Visible);
		ModeWarningArea->SetText(ActiveWarning);
	}
}

void FMeshTerrainModeToolkit::OnToolStarted(UInteractiveToolManager* Manager, UInteractiveTool* Tool)
{
	// We are only interested in left side tool starts against the EdMode tool manager.
	if (UInteractiveToolManager* EdModeToolManager = GetScriptableEditorMode()->GetToolManager(EToolsContextScope::EdMode);
		!EdModeToolManager || EdModeToolManager != Manager || EdModeToolManager->GetActiveTool(EToolSide::Left) != Tool)
	{
		return;
	}
	
	bInActiveTool = true;

	UpdateActiveToolProperties();

	Tool->OnPropertySetsModified.AddSP(this, &FMeshTerrainModeToolkit::UpdateActiveToolProperties);
	Tool->OnPropertyModifiedDirectlyByTool.AddSP(this, &FMeshTerrainModeToolkit::InvalidateCachedDetailPanelState);

	ModeHeaderArea->SetVisibility(EVisibility::Collapsed);
	ActiveToolName = Tool->GetToolInfo().ToolDisplayName;

	if (HasToolkitBuilder())
	{
		ToolkitBuilder->SetActiveToolDisplayName(ActiveToolName);
		if (const UMeshTerrainModeCustomizationSettings* Settings = GetDefault<UMeshTerrainModeCustomizationSettings>())
		{
			if (Settings->bAlwaysShowToolButtons == false)
			{
				ToolkitBuilder->SetActivePaletteCommandsVisibility(EVisibility::Collapsed);
			}
		}
	}

	// try to update icon
	FString ActiveToolIdentifier = GetScriptableEditorMode()->GetToolManager(EToolsContextScope::EdMode)->GetActiveToolName(EToolSide::Left);
	ActiveToolIdentifier.InsertAt(0, ".");
	FName ActiveToolIconName = ISlateStyle::Join(FMeshTerrainModeManagerCommands::Get().GetContextName(), TCHAR_TO_ANSI(*ActiveToolIdentifier));
	ActiveToolIcon = FMeshTerrainModeStyle::Get()->GetOptionalBrush(ActiveToolIconName);
	
	if (ActiveSubmode)
	{
		ActiveSubmode->OnToolStarted(Tool);
	}
	
	// hide the existing accept/cancel overlay when using the SectionedDetailsView, which has its own accept/cancel buttons
	if (!SectionedDetailsView.IsValid() && !bShowQuickSettingsOverlayWidget)
	{
		GetToolkitHost()->AddViewportOverlayWidget(ToolShutdownViewportOverlayWidget.ToSharedRef());
	}
	
	RebuildSectionedDetailsOverlayWidget();
	RebuildQuickSettingsWidget(Tool);

	if (GizmoNumericalUIOverlayWidget.IsValid() && bShowNumericalUIOverlayWidget)
	{
		GetToolkitHost()->AddViewportOverlayWidget(GizmoNumericalUIOverlayWidget.ToSharedRef());
	}

	if (DetailsOverlayWidget.IsValid() && !SectionedDetailsView.IsValid())
	{
		if (bShowDetailsOverlayWidget || QuickSettingsWidget->ShowingDetailsView())
		{
			UpdateDetailsOverlayWidget();
			GetToolkitHost()->AddViewportOverlayWidget(DetailsOverlayWidget.ToSharedRef());
		}
	}

	// Invalidate all the level viewports so that e.g. hitproxy buffers are cleared
	// (fixes the editor gizmo still being clickable despite not being visible)
	if (GIsEditor)
	{
		for (FLevelEditorViewportClient* Viewport : GEditor->GetLevelViewportClients())
		{
			Viewport->Invalidate();
		}
	}
	RebuildPresetListForTool(false);

	GetActiveToolPropertiesImpl = [this, Tool]()
	{
		if (bInActiveTool && Tool)
		{
			return Tool->GetToolProperties(true);
		}

		return TArray<UObject*>();
	};
}

void FMeshTerrainModeToolkit::OnToolEnded(UInteractiveToolManager* Manager, UInteractiveTool* Tool)
{
	// We are only interested in left side tool shutdowns against the EdMode tool manager.
	if (UInteractiveToolManager* EdModeToolManager = GetScriptableEditorMode()->GetToolManager(EToolsContextScope::EdMode);
		!EdModeToolManager || EdModeToolManager != Manager || EdModeToolManager->GetActiveTool(EToolSide::Left) != nullptr)
	{
		return;
	}
	
	bInActiveTool = false;

	if (IsHosted())
	{
		GetToolkitHost()->RemoveViewportOverlayWidget(ToolShutdownViewportOverlayWidget.ToSharedRef());

		if (GizmoNumericalUIOverlayWidget.IsValid())
		{
			GetToolkitHost()->RemoveViewportOverlayWidget(GizmoNumericalUIOverlayWidget.ToSharedRef());
		}

		if (DetailsOverlayWidget.IsValid())
		{
			GetToolkitHost()->RemoveViewportOverlayWidget(DetailsOverlayWidget.ToSharedRef());
		}

		if (QuickSettingsOverlayWidget.IsValid())
		{
			GetToolkitHost()->RemoveViewportOverlayWidget(QuickSettingsOverlayWidget.ToSharedRef());
		}
	}

	ModeDetailsView->SetObject(nullptr);
	if (SectionedDetailsView)
	{
		SectionedDetailsView->SetObject(nullptr);
	}
	ActiveToolName = FText::GetEmpty();
	if (HasToolkitBuilder())
	{
		ToolkitBuilder->SetActiveToolDisplayName(FText::GetEmpty());		
		ToolkitBuilder->SetActivePaletteCommandsVisibility(EVisibility::Visible);
	}

	ModeHeaderArea->SetVisibility(EVisibility::Visible);
	ModeHeaderArea->SetText(LOCTEXT("SelectToolLabel", "Select a Tool from the Toolbar"));
	ClearNotification();
	ClearWarning();
	UInteractiveTool* CurTool = GetScriptableEditorMode()->GetToolManager(EToolsContextScope::EdMode)->GetActiveTool(EToolSide::Left);
	if ( CurTool )
	{
		CurTool->OnPropertySetsModified.RemoveAll(this);
		CurTool->OnPropertyModifiedDirectlyByTool.RemoveAll(this);
	}

	ClearPresetComboList();
	
	if (ActiveSubmode)
	{
		ActiveSubmode->OnToolEnded(Tool);
	}
	
	GetActiveToolPropertiesImpl.Reset();
}

void FMeshTerrainModeToolkit::OnActiveViewportChanged(TSharedPtr<IAssetViewport> OldViewport, TSharedPtr<IAssetViewport> NewViewport)
{
	if (bInActiveTool)
	{
		// Check first to see if this changed because the old viewport was deleted and if not, remove our hud
		if (OldViewport)	
		{
			GetToolkitHost()->RemoveViewportOverlayWidget(ToolShutdownViewportOverlayWidget.ToSharedRef(), OldViewport);

			if (GizmoNumericalUIOverlayWidget.IsValid())
			{
				GetToolkitHost()->RemoveViewportOverlayWidget(GizmoNumericalUIOverlayWidget.ToSharedRef(), OldViewport);
			}

			if (DetailsOverlayWidget.IsValid())
			{
				GetToolkitHost()->RemoveViewportOverlayWidget(DetailsOverlayWidget.ToSharedRef(), OldViewport);
			}

			if (QuickSettingsOverlayWidget.IsValid())
			{
				GetToolkitHost()->RemoveViewportOverlayWidget(QuickSettingsOverlayWidget.ToSharedRef(), OldViewport);
				if (QuickSettingsWidget->ShowingDetailsView())
				{
					GetToolkitHost()->RemoveViewportOverlayWidget(DetailsOverlayWidget.ToSharedRef(), OldViewport);
				}
			}
		}

		// Add the hud to the new viewport, unless we're using the SectionedDetailsView widget, which has its own accept/cancel buttons
		if (!SectionedDetailsView.IsValid() && !bShowQuickSettingsOverlayWidget)
		{
			GetToolkitHost()->AddViewportOverlayWidget(ToolShutdownViewportOverlayWidget.ToSharedRef(), NewViewport);
		}

		if (GizmoNumericalUIOverlayWidget.IsValid() && bShowNumericalUIOverlayWidget)
		{
			GetToolkitHost()->AddViewportOverlayWidget(GizmoNumericalUIOverlayWidget.ToSharedRef(), NewViewport);
		}
		
		if (DetailsOverlayWidget.IsValid() && bShowDetailsOverlayWidget)
		{
			UpdateDetailsOverlayWidget();
			GetToolkitHost()->AddViewportOverlayWidget(DetailsOverlayWidget.ToSharedRef(), NewViewport);
		}

		if (QuickSettingsOverlayWidget.IsValid() && bShowQuickSettingsOverlayWidget)
		{
			GetToolkitHost()->AddViewportOverlayWidget(QuickSettingsOverlayWidget.ToSharedRef(), NewViewport);
			if (QuickSettingsWidget->ShowingDetailsView())
			{
				UpdateDetailsOverlayWidget();
				GetToolkitHost()->AddViewportOverlayWidget(DetailsOverlayWidget.ToSharedRef(), NewViewport);
			}
		}
	}

	if (OldViewport)
	{
		if (SubmodeToolPanelOverlay.IsValid())
		{
			GetToolkitHost()->RemoveViewportOverlayWidget(SubmodeToolPanelOverlay.ToSharedRef(), OldViewport);
		}

		if (SubmodePaletteOverlay.IsValid())
		{
			GetToolkitHost()->RemoveViewportOverlayWidget(SubmodePaletteOverlay.ToSharedRef(), OldViewport);
		}
	}

	if (SubmodeToolPanelOverlay.IsValid())
	{
		GetToolkitHost()->AddViewportOverlayWidget(SubmodeToolPanelOverlay.ToSharedRef(), NewViewport);
	}
	
	if (SubmodePaletteOverlay.IsValid())
	{
		GetToolkitHost()->AddViewportOverlayWidget(SubmodePaletteOverlay.ToSharedRef(), NewViewport);
	}

#if ENABLE_STYLUS_SUPPORT
	if (StylusInputHandler)
	{
		StylusInputHandler->RegisterWindow(NewViewport->AsWidget());
	}
#endif
}

void FMeshTerrainModeToolkit::UpdateCategoryButtonLabelVisibility(UObject* Obj, FPropertyChangedEvent& ChangeEvent)
{
	UMeshTerrainModeCustomizationSettings* UISettings = GetMutableDefault<UMeshTerrainModeCustomizationSettings>();
	ToolkitBuilder->SetCategoryButtonLabelVisibility(UISettings != nullptr  ? UISettings->bShowCategoryButtonLabels : true);
	ToolkitBuilder->RefreshCategoryToolbarWidget();
}

void FMeshTerrainModeToolkit::UpdateSelectionColors(UObject* Obj, FPropertyChangedEvent& ChangeEvent) const
{
	if (CachedSelectionManager)
	{
		const UMeshTerrainModeCustomizationSettings* UISettings = GetMutableDefault<UMeshTerrainModeCustomizationSettings>();
		CachedSelectionManager->SetSelectionColors(UISettings->UnselectedColor, UISettings->HoverOverSelectedColor, UISettings->HoverOverUnselectedColor, UISettings->GeometrySelectedColor,UISettings->GeometrySoftSelectedColor);
	}
}

void FMeshTerrainModeToolkit::UpdateOverlayWidgets(UObject* Obj, FPropertyChangedEvent& ChangeEvent) const
{
	auto UpdateSubmodePanels = [this]()
		{
			UMeshTerrainModeCustomizationSettings* UISettings = GetMutableDefault<UMeshTerrainModeCustomizationSettings>();
			FVector2f SavedWidth = UISettings->SubmodeToolPanelSize;
			if (SubmodePaletteOverlay.IsValid())
			{
				SubmodePaletteOverlay->SetWidthOverride(SavedWidth.X > 0.f ? SavedWidth.X : TOptional<float>());
			}
			if (SubmodeToolPanelOverlay.IsValid())
			{
				SubmodeToolPanelOverlay->SetWidthOverride(SavedWidth.X > 0.f ? SavedWidth.X : TOptional<float>());
			}
		};
	
	UpdateSubmodePanels();
	UpdateDetailsOverlayWidget();
	UpdateQuickSettingsOverlayWidget();
}

FText FMeshTerrainModeToolkit::GetRestrictiveModeAutoGeneratedAssetPathText() const
{
	const UMeshTerrainModeSettings* Settings = GetDefault<UMeshTerrainModeSettings>();
	return FText::FromString(Settings->GetRestrictiveModeAutoGeneratedAssetPath());
}

void FMeshTerrainModeToolkit::OnRestrictiveModeAutoGeneratedAssetPathTextCommitted(const FText& InNewText, ETextCommit::Type InTextCommit) const
{
	UMeshTerrainModeSettings* Settings = GetMutableDefault<UMeshTerrainModeSettings>();
	Settings->SetRestrictiveModeAutoGeneratedAssetPath(InNewText.ToString());
}

void FMeshTerrainModeToolkit::NotifySelectionSystemEnabledStateModified()
{
	if (UMeshTerrainMode* ModelingMode = Cast<UMeshTerrainMode>(GetScriptableEditorMode()))
	{
		ModelingMode->NotifySelectionSystemEnabledStateModified();
	}
	
	//if (bUsesToolkitBuilder && ToolkitBuilder.IsValid())
	//{
	//	ToolkitBuilder->InitCategoryToolbarContainerWidget();
	//}
	//else
	//{
		// required due to above
		FNotificationInfo Info(LOCTEXT("ChangeSelectionEnabledToast",
			"You should exit and re-enter Modeling Mode after toggling Mesh Element Selection Support to fully update the UI"));
		Info.ExpireDuration = 5.0f;
		FSlateNotificationManager::Get().AddNotification(Info);
	//}
}

bool FMeshTerrainModeToolkit::RequestAcceptCancelButtonOverride(IToolHostCustomizationAPI::FAcceptCancelButtonOverrideParams& Params)
{
	if (!Params.OnAcceptCancelTriggered || !Params.CanAccept || Params.Label.IsEmpty())
	{
		UE_LOGF(LogGeometry, Warning, "FMeshTerrainModeToolkit::RequestAcceptCancelButtonOverride received request with insufficient parameters.");
		return false;
	}

	AcceptCancelButtonParams = Params;
	CompleteButtonParams.Reset();
	bCurrentOverrideButtonsWereClicked = false;

	if (ToolShutdownViewportOverlayWidget)
	{
		ToolShutdownViewportOverlayWidget->Invalidate(EInvalidateWidgetReason::Layout);
	}
	return true;
}
bool FMeshTerrainModeToolkit::RequestCompleteButtonOverride(IToolHostCustomizationAPI::FCompleteButtonOverrideParams& Params)
{
	if (!Params.OnCompleteTriggered || Params.Label.IsEmpty())
	{
		UE_LOGF(LogGeometry, Warning, "FMeshTerrainModeToolkit::RequestCompleteButtonOverride received request with insufficient parameters.");
		return false;
	}

	CompleteButtonParams = Params;
	AcceptCancelButtonParams.Reset();
	bCurrentOverrideButtonsWereClicked = false;

	if (ToolShutdownViewportOverlayWidget)
	{
		ToolShutdownViewportOverlayWidget->Invalidate(EInvalidateWidgetReason::Layout);
	}
	return true;
}
void FMeshTerrainModeToolkit::ClearButtonOverrides()
{
	AcceptCancelButtonParams.Reset();
	CompleteButtonParams.Reset();
	if (ToolShutdownViewportOverlayWidget)
	{
		ToolShutdownViewportOverlayWidget->Invalidate(EInvalidateWidgetReason::Layout);
	}
}


FReply FMeshTerrainModeToolkit::HandleAcceptCancelClick(bool bAccept)
{
	if (AcceptCancelButtonParams.IsSet())
	{
		bCurrentOverrideButtonsWereClicked = true;
		if (ensure(AcceptCancelButtonParams->OnAcceptCancelTriggered))
		{
			AcceptCancelButtonParams->OnAcceptCancelTriggered(bAccept);
		}
		
		// This will be reset back to false if the callback above triggers another override request.
		if (bCurrentOverrideButtonsWereClicked)
		{
			ClearButtonOverrides();
		}
	}
	else
	{
		GetScriptableEditorMode()->GetInteractiveToolsContext(EToolsContextScope::EdMode)->EndTool(
			bAccept ? EToolShutdownType::Accept : EToolShutdownType::Cancel);
	}
	return FReply::Handled();
}
FReply FMeshTerrainModeToolkit::HandleCompleteClick()
{
	if (CompleteButtonParams.IsSet())
	{
		bCurrentOverrideButtonsWereClicked = true;
		if (ensure(CompleteButtonParams->OnCompleteTriggered))
		{
			CompleteButtonParams->OnCompleteTriggered();
		}

		// This will be reset back to false if the callback above triggers another override request.
		if (bCurrentOverrideButtonsWereClicked)
		{
			ClearButtonOverrides();
		}
	}
	else
	{
		GetScriptableEditorMode()->GetInteractiveToolsContext(EToolsContextScope::EdMode)->EndTool(EToolShutdownType::Completed);
	}

	return FReply::Handled();
}

void FMeshTerrainModeToolkit::DisconnectStylusStateProviderAPI()
{
#if ENABLE_STYLUS_SUPPORT
	if (StylusInputHandler)
	{
		// replace the input handler with a new, empty version -- disconnecting old windows/contexts
		StylusInputHandler = MakeUnique<UE::MeshTerrain::FStylusInputHandler>();
	}
#endif
}

IToolStylusStateProviderAPI* FMeshTerrainModeToolkit::GetStylusStateProviderAPI() const
{
	return StylusInputHandler.Get();
}

bool FMeshTerrainModeToolkit::HandleClick(FClickContext& InOutContext)
{
	using namespace UE::MeshTerrain;
	
	if (ActiveSubmode)
	{
		// give submode a chance to do what it wants
		if (ActiveSubmode->HandleClick(InOutContext))
		{
			return true;
		}
		
		// if there is an active tool, look through its properties and for potential customizations to be added to context menu
		if (UInteractiveTool* CurTool = GetScriptableEditorMode()->GetToolManager(EToolsContextScope::EdMode)->GetActiveTool(EToolSide::Left))
		{
			// there may be multiple Quick Edit properties (and therefore widgets) assigned to each priority num.
			//	we will collect these widgets in an array and map this array to the priority num
			TMap<int, TArray<FPropertyWidget>> PriorityToWidgets;

			// default handling of properties tagged with ModelingQuickEdit (no customizations found)
			auto DefaultHandleQuickEditProps = [this](FProperty* Property, UObject* PropertySet, TArray<FPropertyWidget>& Widgets)
			{
				// collect widgets for float props (represented by slider/SSpinBox)
				if (FFloatProperty* FloatProperty = CastField<FFloatProperty>(Property))
				{
					AddSliderWidget(FloatProperty, PropertySet, Widgets, EQuickPropertyDisplay::WidgetOnly);
				}
				// collect widgets for bool props (represented by checkbox)
				else if (FBoolProperty* BoolProperty = CastField<FBoolProperty>(Property))
				{
					AddBoolWidget(BoolProperty, PropertySet, Widgets, QuickEditCustomizations->GetBoolCustomizations(),
						EQuickPropertyDisplay::WidgetOnly, EBoolPropertyDisplay::CheckBox);
				}
				// collect widgets for enum props (represented by drop down menu)
				else if (FEnumProperty* EnumProperty = CastField<FEnumProperty>(Property))
				{
					AddEnumWidget(EnumProperty, PropertySet, Widgets, EQuickPropertyDisplay::WidgetOnly);
				}
				// collect widgets for all other prop types using their ISinglePropertyView representations
				else
				{
					FPropertyEditorModule& PropertyEditor = FModuleManager::Get().LoadModuleChecked<FPropertyEditorModule>(TEXT("PropertyEditor"));
					FSinglePropertyParams SinglePropertyParams;
					SinglePropertyParams.bHideResetToDefault = true;
					SinglePropertyParams.NamePlacement = EPropertyNamePlacement::Hidden;

					const TSharedRef<ISinglePropertyView> SinglePropertyView =
						PropertyEditor.CreateSingleProperty(PropertySet, Property->GetFName(), SinglePropertyParams).ToSharedRef();

					Widgets.Add(FPropertyWidget(Property->GetDisplayNameText(),
						SNew(SHorizontalBox)
						+ SHorizontalBox::Slot()
						.AutoWidth()
						.HAlign(HAlign_Center)
						[
							SinglePropertyView
						]));
				}
			};
			
			const TArray<UObject*> PropertySets = GetActiveToolProperties();

			// iterate through all the tool's property sets
			for (UObject* PropertySet : PropertySets)
			{
				if (!PropertySet) { continue; }
				// iterate through all properties within a set
				for (TFieldIterator<FProperty> PropertyIterator(PropertySet->GetClass()); PropertyIterator; ++PropertyIterator)
				{
					FProperty* Property = *PropertyIterator;

					CollectTaggedWidgetsRecursive(Property, PropertySet, TEXT("ModelingQuickEdit"), CurTool, PriorityToWidgets,
						DefaultHandleQuickEditProps, QuickEditCustomizations->GetCustomizations(), EQuickPropertyDisplay::WidgetOnly,
						QuickEditCustomizations->GetStructTypeCustomizations());
				}
			}

			// if widgets were found (found tagged properties or customizations)
			if (!PriorityToWidgets.IsEmpty())
			{
				// sort widget arrays by priority (low->high)
				PriorityToWidgets.KeySort(TLess<int>());

				TArray<TArray<FPropertyWidget>> SortedWidgetArrays;
				PriorityToWidgets.GenerateValueArray(SortedWidgetArrays);
				
				// add each widget to the menu builder
				for (TArray<FPropertyWidget> WidgetArray : SortedWidgetArrays)
				{
					for (const FPropertyWidget &Widget : WidgetArray)
					{
						InOutContext.Builder->AddWidget(CreateQuickEditPropertyRow(Widget),
							FText(), false, true, Widget.PropertyName);
					}
				}
				return true; // complete context menu
			}
		}
	}
	// if neither tool nor submode build a context menu, the click is not handled here
	return false;
}

TArray<UObject*> FMeshTerrainModeToolkit::GetActiveToolProperties() const
{
	return GetActiveToolPropertiesImpl ? GetActiveToolPropertiesImpl() : TArray<UObject*>();
}

TSharedPtr<UE::MeshTerrain::FModelingQuickPropertyCustomizations> FMeshTerrainModeToolkit::GetQuickSettingsCustomizations() const
{
	if (QuickSettingsWidget.IsValid())
	{
		return QuickSettingsWidget->GetCustomizations();
	}
	return nullptr;
}

TSharedPtr<UE::MeshTerrain::FModelingQuickPropertyCustomizations> FMeshTerrainModeToolkit::GetQuickEditCustomizations() const
{
	return QuickEditCustomizations;
}

TSharedPtr<SWidget> FMeshTerrainModeToolkit::MakeToolShutdownButtons()
{
	// accept button param funcs
	TFunction<FText()> GetAcceptButtonTooltip = [this]()
		{
			return AcceptCancelButtonParams.IsSet() && AcceptCancelButtonParams->OverrideAcceptButtonTooltip.IsSet() ?
				AcceptCancelButtonParams->OverrideAcceptButtonTooltip.GetValue()
				: LOCTEXT("OverlayAcceptTooltip", "Accept/Commit the results of the active Tool [Enter]");
		};
	TFunction<bool()> GetAcceptButtonEnabled = [this]()
		{
			return AcceptCancelButtonParams.IsSet() ? AcceptCancelButtonParams->CanAccept()
				: GetScriptableEditorMode()->GetInteractiveToolsContext(EToolsContextScope::EdMode)->CanAcceptActiveTool();
		};
	// cancel button param funcs
	TFunction<FText()> GetCancelButtonTooltip = [this]()
		{
			if (AcceptCancelButtonParams.IsSet() && AcceptCancelButtonParams->OverrideCancelButtonTooltip.IsSet())
			{
				return AcceptCancelButtonParams->OverrideCancelButtonTooltip.GetValue();
			}
			return GetDefault<UMeshTerrainModeCustomizationSettings>()->bEscapeAcceptsToolResult
				? LOCTEXT("OverlayCancelTooltip_NoEsc", "Cancel the active Tool")
				: LOCTEXT("OverlayCancelTooltip", "Cancel the active Tool [Esc]");
		};
	TFunction<bool()> GetCancelButtonEnabled = [this]() 
		{
			return AcceptCancelButtonParams.IsSet()
				|| GetScriptableEditorMode()->GetInteractiveToolsContext(EToolsContextScope::EdMode)->CanCancelActiveTool();
		};
	// accept/cancel visibility func
	TFunction<EVisibility()> GetAcceptCancelButtonVisibility = [this]()
		{
			if (AcceptCancelButtonParams.IsSet()
				|| (!CompleteButtonParams.IsSet()
					&& GetScriptableEditorMode()->GetInteractiveToolsContext(EToolsContextScope::EdMode)->ActiveToolHasAccept()))
			{
				return EVisibility::Visible;
			}
			return EVisibility::Collapsed;
		};

	// complete button param funcs
	TFunction<FText()> GetCompleteButtonTooltip = [this]()
		{
			return CompleteButtonParams.IsSet() && CompleteButtonParams->OverrideCompleteButtonTooltip.IsSet() ?
				CompleteButtonParams->OverrideCompleteButtonTooltip.GetValue()
				: LOCTEXT("OverlayCompleteTooltip", "Exit the active Tool [Enter]");
		};
	TFunction<bool()> GetCompleteButtonEnabled = [this]()
		{
			return CompleteButtonParams.IsSet()
				|| GetScriptableEditorMode()->GetInteractiveToolsContext(EToolsContextScope::EdMode)->CanCompleteActiveTool();
		};
	TFunction<EVisibility()> GetCompleteButtonVisibility = [this]()
		{
			if (CompleteButtonParams.IsSet()
				|| (!AcceptCancelButtonParams.IsSet()
					&& GetScriptableEditorMode()->GetInteractiveToolsContext(EToolsContextScope::EdMode)->CanCompleteActiveTool()))
			{
				return EVisibility::Visible;
			}
			return EVisibility::Collapsed;
		};

	return SNew(SHorizontalBox)
	+ SHorizontalBox::Slot()
	[
		SNew(SHorizontalBox)
		+ SHorizontalBox::Slot()
		.HAlign(HAlign_Left)
		[
			// accept button
			SNew(SButton)
			.ToolTipText_Lambda([GetAcceptButtonTooltip](){ return GetAcceptButtonTooltip(); })
			.OnClicked_Raw(this, &FMeshTerrainModeToolkit::HandleAcceptCancelClick, true)
			.IsEnabled_Lambda([GetAcceptButtonEnabled](){ return GetAcceptButtonEnabled(); })
			.Visibility_Lambda([GetAcceptCancelButtonVisibility](){return GetAcceptCancelButtonVisibility(); })
			.ButtonStyle(&FMeshTerrainModeStyle::Get()->GetWidgetStyle<FButtonStyle>("MeshTerrainMode.DetailsViewToolShutdown"))
			[
				SNew(SImage)
				.Image_Lambda([GetAcceptButtonEnabled]()
				{
					return GetAcceptButtonEnabled() ? FMeshTerrainModeStyle::Get()->GetBrush("ToolShutdown.Accept") :
						FMeshTerrainModeStyle::Get()->GetBrush("ToolShutdown.AcceptDisabled");
				})
				.DesiredSizeOverride(FVector2D(20, 20))
			]
		]
		+ SHorizontalBox::Slot()
		.HAlign(HAlign_Right)
		.Padding(8.f, 0.f, 4.f, 0.f)
		[
			// cancel button
			SNew(SButton)
			.ToolTipText_Lambda([GetCancelButtonTooltip](){return GetCancelButtonTooltip(); })
			.HAlign(HAlign_Center)
			.OnClicked_Raw(this, &FMeshTerrainModeToolkit::HandleAcceptCancelClick, false)
			.IsEnabled_Lambda([GetCancelButtonEnabled]() { return GetCancelButtonEnabled(); })
			.Visibility_Lambda([GetAcceptCancelButtonVisibility](){ return GetAcceptCancelButtonVisibility(); })
			.ContentPadding(FMargin(2.f))
			.ButtonStyle(&FMeshTerrainModeStyle::Get()->GetWidgetStyle<FButtonStyle>("MeshTerrainMode.DetailsViewToolShutdown"))
			[
				SNew(SImage)
				.Image(FMeshTerrainModeStyle::Get()->GetBrush("ToolShutdown.Cancel"))
				.DesiredSizeOverride(FVector2D(20, 20))
			]
		]
		// Complete
		+SHorizontalBox::Slot()
		.HAlign(HAlign_Right)
		.Padding(8.f, 0.f, 4.f, 0.f)
		[
			SNew(SButton)
			.ToolTipText_Lambda([GetCompleteButtonTooltip](){ return GetCompleteButtonTooltip(); })
			.OnClicked_Raw(this, &FMeshTerrainModeToolkit::HandleCompleteClick)
			.IsEnabled_Lambda([GetCompleteButtonEnabled]() { return GetCompleteButtonEnabled(); })
			.Visibility_Lambda([GetCompleteButtonVisibility](){ return GetCompleteButtonVisibility(); })
			.ButtonStyle(&FMeshTerrainModeStyle::Get()->GetWidgetStyle<FButtonStyle>("MeshTerrainMode.DetailsViewToolShutdown"))
			[
				SNew(SImage)
				.Image(FMeshTerrainModeStyle::Get()->GetBrush("MeshTerrainModeManagerCommands.AcceptActiveTool"))
				.DesiredSizeOverride(FVector2D(20, 20))
			]
		]
	];
}
	
}

#undef LOCTEXT_NAMESPACE
