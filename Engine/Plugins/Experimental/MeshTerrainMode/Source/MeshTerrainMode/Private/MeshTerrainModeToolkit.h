// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Toolkits/BaseToolkit.h"
#include "StatusBarSubsystem.h"
#include "ToolHostCustomizationAPI.h" // FAcceptCancelButtonOverrideParams
#include "Widgets/ModelingQuickSettingsWidget.h"
#include "Overlay/DragBoxPosition.h"

class IAssetViewport;
class SEditableTextBox;
class STextComboBox;
namespace UE::MeshTerrain { class SSectionedDetailsViewWidget; }
namespace ETextCommit { enum Type : int; }

#define LOCTEXT_NAMESPACE "MeshTerrainModeToolkit"

class STransformGizmoNumericalUIOverlay;
class IDetailsView;
class SButton;
class STextBlock;
class SSplitter;
class UGeometrySelectionManager;
class UInteractiveToolsPresetCollectionAsset;
struct FAssetData;
class IToolStylusStateProviderAPI;

namespace UE::ToolWidgets
{
	class SDraggableBoxOverlay;
}

namespace UE::MeshTerrain
{
struct FClickContext;
class SSubmodeToolPanel;
class SSubmodePalette;
class FStylusInputHandler;
class FSubmode;
class FRecentPresetCollectionProvider;

struct FToolPresetOption
{
	FString PresetLabel;
	FString PresetTooltip;
	FSlateIcon PresetIcon;
	int32 PresetIndex;
	FSoftObjectPath PresetCollection;
};

class FMeshTerrainModeToolkit : public FModeToolkit
{
public:

	FMeshTerrainModeToolkit();
	virtual ~FMeshTerrainModeToolkit() override;
	
	/** FModeToolkit interface */
	virtual void Init(const TSharedPtr<IToolkitHost>& InitToolkitHost, TWeakObjectPtr<UEdMode> InOwningMode) override;
	virtual void RequestModeUITabs() override;
	virtual void InvokeUI() override;
	virtual void ShutdownUI() override;

	/** IToolkit interface */
	virtual FName GetToolkitFName() const override;
	virtual FText GetBaseToolkitName() const override;
	virtual TSharedPtr<class SWidget> GetInlineContent() const override;

	// initialize toolkit widgets that need to wait until mode is initialized/entered
	virtual void InitializeAfterModeSetup();
	virtual void UninitializeOnModeDeactivate();

	// set/clear notification message area
	virtual void PostNotification(const FText& Message);
	virtual void ClearNotification();

	// set/clear warning message area
	virtual void PostWarning(const FText& Message);
	virtual void ClearWarning();

	/** Returns the Mode specific tabs in the mode toolbar **/ 
	virtual void GetToolPaletteNames(TArray<FName>& InPaletteName) const override;
	virtual FText GetToolPaletteDisplayName(FName PaletteName) const override; 
	virtual void OnToolPaletteChanged(FName PaletteName) override;
	virtual bool HasIntegratedToolPalettes() const override { return false; }
	virtual bool HasExclusiveToolPalettes() const override { return false; }

	virtual FText GetActiveToolDisplayName() const override { return ActiveToolName; }
	virtual FText GetActiveToolMessage() const override { return ActiveToolMessage; }

	virtual void ShowRealtimeAndModeWarnings(bool bShowRealtimeWarning);

	virtual void OnToolStarted(UInteractiveToolManager* Manager, UInteractiveTool* Tool) override;
	virtual void OnToolEnded(UInteractiveToolManager* Manager, UInteractiveTool* Tool) override;

	virtual void CustomizeModeDetailsViewArgs(FDetailsViewArgs& ArgsInOut) override;

	void OnActiveViewportChanged(TSharedPtr<IAssetViewport>, TSharedPtr<IAssetViewport> );

	virtual bool RequestAcceptCancelButtonOverride(IToolHostCustomizationAPI::FAcceptCancelButtonOverrideParams& Params);
	virtual bool RequestCompleteButtonOverride(IToolHostCustomizationAPI::FCompleteButtonOverrideParams& Params);
	virtual void ClearButtonOverrides();

	// Read and write Autogen Subfolder quick setting in restrictive mode.
	virtual FText GetRestrictiveModeAutoGeneratedAssetPathText() const;
	virtual void OnRestrictiveModeAutoGeneratedAssetPathTextCommitted(const FText& InNewText, ETextCommit::Type InTextCommit) const;

	// The mode is expected to call this after setting up the gizmo context object so that any subsequently
	// created gizmos are bound to the numerical UI.
	void BindGizmoNumericalUI();

	// get cached UGeometrySelectionManager pointer for the current Mode
	UGeometrySelectionManager* GetMeshSelectionManager();

	// @return true if we are currently in a tool, used to enable/disable UI things
	bool IsInActiveTool() const { return bInActiveTool; }

	// This is exposed only for the convenience of being able to create the numerical UI submenu
	// in a non-member function in ModelingModeToolkit_Toolbars.cpp
	TSharedPtr<STransformGizmoNumericalUIOverlay> GetGizmoNumericalUIOverlayWidget() { return GizmoNumericalUIOverlayWidget; }

	void NotifySelectionSystemEnabledStateModified();
	
	// Tells the stylus API to clean up any active tracked windows / contexts
    void DisconnectStylusStateProviderAPI();

	IToolStylusStateProviderAPI* GetStylusStateProviderAPI() const;

	bool HandleClick(FClickContext& InOutContext);

	// Gets the Modeling Quick Settings customizations
	TSharedPtr<UE::MeshTerrain::FModelingQuickPropertyCustomizations> GetQuickSettingsCustomizations() const;

	// Gets the Modeling Quick Edit customizations
	TSharedPtr<UE::MeshTerrain::FModelingQuickPropertyCustomizations> GetQuickEditCustomizations() const;

	// Rebuild the QuickSettings widget to reflect the provided tool. If no tool is provided, uses the currently active tool
	void RebuildQuickSettingsWidget(UInteractiveTool* Tool = nullptr) const;

	// Rebuild the sectioned DetailsOverlay widget to reflect the provided tool. If none is provided, will use the active tool
	void RebuildSectionedDetailsOverlayWidget();

	void ShowQuickSettingsOverlayWidget(bool bVisible) { bShowQuickSettingsOverlayWidget = bVisible; }
	void ShowDetailsOverlayWidget(bool bVisible) { bShowDetailsOverlayWidget = bVisible; }
	void ShowNumericalUIOverlayWidget(bool bVisible) { bShowNumericalUIOverlayWidget = bVisible; }

	bool ExperimentalToolsEnabled() const;
	
private:
	friend class UMeshTerrainMode;

	const static TArray<FName> PaletteNames_Standard;

	bool bInActiveTool = false;
	FText ActiveToolName;
	FText ActiveToolMessage;
	FStatusBarMessageHandle ActiveToolMessageHandle;
	const FSlateBrush* ActiveToolIcon = nullptr;

	TOptional<IToolHostCustomizationAPI::FAcceptCancelButtonOverrideParams> AcceptCancelButtonParams;
	TOptional<IToolHostCustomizationAPI::FCompleteButtonOverrideParams> CompleteButtonParams;
	bool bCurrentOverrideButtonsWereClicked = false;

	FReply HandleAcceptCancelClick(bool bAccept);
	FReply HandleCompleteClick();

	UGeometrySelectionManager* CachedSelectionManager = nullptr;

	TSharedPtr<SWidget> ToolkitWidget;
	void UpdateActiveToolProperties();
	void InvalidateCachedDetailPanelState(UObject* ChangedObject);

	/** A utility function to register the tool palettes with the ToolkitBuilder */
	void RegisterPalettes();
	FDelegateHandle ActivePaletteChangedHandle;

	TSharedPtr<SWidget> ToolShutdownViewportOverlayWidget;
	void MakeToolShutdownOverlayWidget();

	TSharedPtr<UE::ToolWidgets::SDraggableBoxOverlay> SubmodeToolPanelOverlay;
	TSharedPtr<UE::MeshTerrain::SSubmodeToolPanel> SubmodeToolPanel;
	void MakeSubmodeToolPanelOverlayWidget();
	TSharedPtr<UE::ToolWidgets::SDraggableBoxOverlay> SubmodePaletteOverlay;
	TSharedPtr<UE::MeshTerrain::SSubmodePalette> SubmodePalette;
	void MakeSubmodePaletteOverlayWidget();
	constexpr static float SubmodePalettePadding = 4.0f;
	
	void InitializeSubmodes();
	TSharedPtr<UE::MeshTerrain::FSubmode> GetDefaultSubmode() const;
	TMap<FName, TSharedPtr<UE::MeshTerrain::FSubmode>> Submodes;

	// activates and deactivates submodes
	void ActivateSubmode(TSharedPtr<UE::MeshTerrain::FSubmode> Submode);
	void DeactivateSubmode();

	// ModelingQuickSettings widget
	void MakeQuickSettingsOverlayWidget();
	TSharedPtr<SModelingQuickSettingsWidget> QuickSettingsWidget;
	TSharedPtr<UE::ToolWidgets::SDraggableBoxOverlay> QuickSettingsOverlayWidget;
	bool bShowQuickSettingsOverlayWidget = false;

	// represents the Quick Edit (RMB context menu) customizations
	TSharedPtr<UE::MeshTerrain::FModelingQuickPropertyCustomizations> QuickEditCustomizations;

	TSharedPtr<SWidget> MakeToolShutdownButtons();

	FToolWidget_DragBoxPosition GetDetailsOverlayPosition() const;
	TOptional<FVector2f> GetDetailsOverlaySize() const;
	FToolWidget_DragBoxPosition GetQuickSettingsOverlayPosition() const;

	void MakeDetailsOverlayWidget();

	// the two different types of details views that can be built
	void MakeFullDetailsOverlayWidget();
	void MakeSectionedDetailsOverlayWidget();
	
	void UpdateDetailsOverlayWidget() const;
	void UpdateQuickSettingsOverlayWidget() const;
	TSharedPtr<UE::ToolWidgets::SDraggableBoxOverlay> DetailsOverlayWidget;
	bool bShowDetailsOverlayWidget = true;

	void MakeGizmoNumericalUIOverlayWidget();
	TSharedPtr<STransformGizmoNumericalUIOverlay> GizmoNumericalUIOverlayWidget;
	bool bShowNumericalUIOverlayWidget = true;

	/* Sectioned Details Overlay Widget */
	TSharedPtr<UE::MeshTerrain::SSectionedDetailsViewWidget> SectionedDetailsViewWidget;
	TSharedPtr<IDetailsView> SectionedDetailsView;

	TSharedRef<SWidget> MakeMenu_ModelingModeConfigSettings();

	TSharedPtr<STextBlock> ModeWarningArea;
	TSharedPtr<STextBlock> ModeHeaderArea;
	TSharedPtr<STextBlock> ToolWarningArea;
	TSharedPtr<SButton> AcceptButton;
	TSharedPtr<SButton> CancelButton;
	TSharedPtr<SButton> CompletedButton;

	FText ActiveWarning{};

	// Presets
	TSharedPtr<SWidget> ToolPresetArea;
	TSharedPtr<SWidget> MakePresetPanel();
	TSharedPtr<FAssetData> CurrentPreset;
	FSoftObjectPath CurrentPresetPath;
	FText CurrentPresetLabel;
	FToolPresetOption SelectedPresetToEdit;
	FString NewPresetLabel;
	FString NewPresetTooltip;
	FSlateIcon NewPresetIcon;

	TArray<TSharedPtr<FToolPresetOption>> AvailablePresetsForTool;
	TSharedPtr<SEditableComboBox<TSharedPtr<FString>>> PresetComboBox;
	TSharedPtr<FRecentPresetCollectionProvider> RecentPresetCollectionProvider;

	TSharedRef<SWidget> GetPresetCreateButtonContent();

	void CreateNewPresetInCollection(const FString& PresetLabel, FSoftObjectPath CollectionPath, const FString& ToolTip, FSlateIcon Icon);
	void LoadPresetFromCollection(const int32 PresetIndex, FSoftObjectPath CollectionPath);
	void UpdatePresetInCollection(const FToolPresetOption& PresetToEditIn, bool bUpdateStoredPresetValues);

	FString GetCurrentPresetPath() const { return CurrentPresetPath.GetAssetPathString(); }
	void HandlePresetAssetChanged(const FAssetData& InAssetData);
	bool HandleFilterPresetAsset(const FAssetData& InAssetData);
	
	void RebuildPresetListForTool(bool bSettingsOpened);
	TSharedRef<SWidget> MakePresetComboWidget(TSharedPtr<FString> InItem);
	bool IsPresetEnabled() const;
	void ClearPresetComboList();

	/**
	 *  Updates the category button labels' visibility based off of the current editor preference for it
	 */
	void UpdateCategoryButtonLabelVisibility(UObject* Obj, FPropertyChangedEvent& ChangeEvent);

	/**
	 *  Updates the mesh element selection colors based off of the current editor preference for it
	 */
	void UpdateSelectionColors(UObject* Obj, FPropertyChangedEvent& ChangeEvent) const;
	
	void UpdateOverlayWidgets(UObject* Obj, FPropertyChangedEvent& ChangeEvent) const;

	TSharedPtr<FEditablePalette> FavoritesPalette;

	bool bFirstInitializeAfterModeSetup = true;
	bool bShowActiveSelectionActions = true;

	TSharedPtr<UE::MeshTerrain::FSubmode> ActiveSubmode;
	
	TUniquePtr<UE::MeshTerrain::FStylusInputHandler> StylusInputHandler;

	// This is sort of temporary, I imagine there is a better way.
	TArray<UObject*> GetActiveToolProperties() const;
	TFunction<TArray<UObject*>(void)> GetActiveToolPropertiesImpl;

	TWeakPtr<SSplitter> LevelViewportSplitter;
	int LevelViewportSplitterSlot = INDEX_NONE;
};
}

#undef LOCTEXT_NAMESPACE
