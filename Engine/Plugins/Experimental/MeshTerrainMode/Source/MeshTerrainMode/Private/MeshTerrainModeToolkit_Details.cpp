// Copyright Epic Games, Inc. All Rights Reserved.

#include "MeshTerrainModeToolkit.h"
#include "MeshTerrainModeStyle.h"
#include "MeshTerrainModeSettings.h"

#include "Overlay/SDraggableBoxOverlay.h"
#include "Widgets/Layout/SBackgroundBlur.h"
#include "Widgets/Layout/SBorder.h"
#include "Widgets/Layout/SScrollBox.h"
#include "Widgets/Images/SImage.h"
#include "Widgets/Input/SButton.h"
#include "Tools/UEdMode.h"
#include "Tools/EdModeInteractiveToolsContext.h"
#include "Widgets/SSubmodePalette.h"
#include "Widgets/SSectionedDetailsViewWidget.h"

#define LOCTEXT_NAMESPACE "FMeshTerrainModeToolkit_Details"

namespace UE::MeshTerrain
{

static TAutoConsoleVariable<int32> CVarDetailsViewConstructionType(
TEXT("MeshTerrainMode.DetailsViewConstructionType"),
0,
TEXT("0 - using existing details view ; 1 - building from scratch"));

FToolWidget_DragBoxPosition FMeshTerrainModeToolkit::GetDetailsOverlayPosition() const
{
	TSharedPtr<UE::ToolWidgets::SDraggableBoxOverlay> OverlayWidget;
	TSharedPtr<SWidget> PaletteWidget;
	if (SubmodeToolPanel.IsValid())
	{
		OverlayWidget = SubmodeToolPanelOverlay;
		PaletteWidget = SubmodeToolPanel;
	}
	else if (SubmodePalette.IsValid())
	{
		OverlayWidget = SubmodePaletteOverlay;
		PaletteWidget = SubmodePalette;
	}

	FVector2f AlignmentOffset(0.f, 0.f);
	if (OverlayWidget.IsValid())
	{
		AlignmentOffset = OverlayWidget->GetBoxAlignmentOffset();
		
		float SubmodePaletteWidth = 0.f;
		TOptional<float> OverlayWidthOverride = OverlayWidget->GetWidthOverride();
		TOptional<float> OverlayMaxWidth = OverlayWidget->GetMaximumBoxWidth();
		if (OverlayWidthOverride.IsSet())
		{
			SubmodePaletteWidth = OverlayWidthOverride.GetValue();
		}
		else if (OverlayMaxWidth.IsSet())
		{
			SubmodePaletteWidth = OverlayMaxWidth.GetValue();
		}
		else
		{
			// Fallback to reading PaintSpaceGeometry which is a tick behind.
			SubmodePaletteWidth = PaletteWidget->GetPaintSpaceGeometry().GetLocalSize().X;
		}
		SubmodePaletteWidth += (2 * SubmodePalettePadding);
		AlignmentOffset.X += SubmodePaletteWidth;
	}
	
	FToolWidget_DragBoxPosition OverlayPosition{ AlignmentOffset, HAlign_Left, VAlign_Top };
	UMeshTerrainModeCustomizationSettings* UISettings = GetMutableDefault<UMeshTerrainModeCustomizationSettings>();
	if (UISettings->DetailsOverlayPosition != FToolWidget_DragBoxPosition())
	{
		OverlayPosition = UISettings->DetailsOverlayPosition;
	}
	return OverlayPosition;
}
	
TOptional<FVector2f> FMeshTerrainModeToolkit::GetDetailsOverlaySize() const
{
	TOptional<FVector2f> OverlaySize;
	UMeshTerrainModeCustomizationSettings* UISettings = GetMutableDefault<UMeshTerrainModeCustomizationSettings>();
	if (UISettings->DetailsOverlaySize != FVector2f::ZeroVector)
	{
		OverlaySize = UISettings->DetailsOverlaySize;
	}
	return OverlaySize;
}

void FMeshTerrainModeToolkit::MakeDetailsOverlayWidget()
{
	if (SectionedDetailsView)
	{
		MakeSectionedDetailsOverlayWidget();
	}
	else
	{
		MakeFullDetailsOverlayWidget();
	}
}

void FMeshTerrainModeToolkit::MakeFullDetailsOverlayWidget()
{
	FToolWidget_DragBoxPosition OverlayPosition = GetDetailsOverlayPosition();
	const FSlateBrush* OverlayBrush = FMeshTerrainModeStyle::Get()->GetBrush("MeshTerrainMode.OpaqueOverlayBrushNoOutline");
	const FSlateBrush* OverlayBackgroundBrush = FMeshTerrainModeStyle::Get()->GetBrush("MeshTerrainMode.TransparentBackgroundBrush");
	static const FSlateFontInfo TitleFont = FCoreStyle::GetDefaultFontStyle("Bold", 12);

	SAssignNew(DetailsOverlayWidget, UE::ToolWidgets::SDraggableBoxOverlay)
		.HAlign(OverlayPosition.HAlign)
		.VAlign(OverlayPosition.VAlign)
		.InitialAlignmentOffset(OverlayPosition.RelativeOffset)
		.OnUserDraggedToNewPosition_Lambda([this]()
			{
				UMeshTerrainModeCustomizationSettings* UISettings = GetMutableDefault<UMeshTerrainModeCustomizationSettings>();
				UISettings->DetailsOverlayPosition = DetailsOverlayWidget->GetDragBoxPosition();
				UISettings->SaveConfig();
			})
		.Visibility(EVisibility::SelfHitTestInvisible)
		.Resizable(ToolWidgets::EResizeEdges::Right | ToolWidgets::EResizeEdges::Bottom)
		.ResizeHandleThickness(6.f)
		.MinimumBoxWidth(300.f)
		.MinimumBoxHeight(400.f)
		.OnUserResized_Lambda([this]()
			{
				UMeshTerrainModeCustomizationSettings* UISettings = GetMutableDefault<UMeshTerrainModeCustomizationSettings>();
				UISettings->DetailsOverlaySize.X = DetailsOverlayWidget->GetWidthOverride().Get(0.f);
				UISettings->DetailsOverlaySize.Y = DetailsOverlayWidget->GetHeightOverride().Get(0.f);
				UISettings->SaveConfig();
			})
		.Content()
		[
			SNew(SBorder)
			.BorderImage(OverlayBrush)
			.Padding(2.f, 4.f, 2.f, 2.f)
			.HAlign(HAlign_Fill)
			.VAlign(VAlign_Fill)
			[
				SNew(SBox)
				.WidthOverride(300.f)
				.MaxDesiredHeight(400.f)
				[
					SNew(SVerticalBox)
					+ SVerticalBox::Slot()
					.AutoHeight()
					.Padding(0.f, 0.f, 0.f, 2.f)
					[
						SNew(SHorizontalBox)
						+ SHorizontalBox::Slot()
						.AutoWidth()
						.Padding(8.f, 0.f, 0.f, 4.f)
						.VAlign(VAlign_Bottom)
						[
							SNew(SImage)
							.Image_Lambda([this]()
							{
								return ActiveToolIcon;
							})
							.DesiredSizeOverride(FVector2D(16.f))
							.ColorAndOpacity(FSlateColor::UseForeground())
						]
						+ SHorizontalBox::Slot()
						.FillWidth(1.0f)
						.Padding(8.f, 0.f, 0.f, 0.f)
						.VAlign(VAlign_Bottom)
						.HAlign(HAlign_Left)
						[
							SNew(STextBlock)
							.Font(TitleFont)
							.Text_Lambda([this]()
							{
								return ActiveToolName;
							})
						]
					]
					+ SVerticalBox::Slot()
					.FillHeight(1.0f)
					[
						SNew(SScrollBox)
						.Orientation(Orient_Vertical)
						.ConsumeMouseWheel(EConsumeMouseWheel::Always)
						+ SScrollBox::Slot()
						[
							SNew(SVerticalBox)
							+ SVerticalBox::Slot()
							.AutoHeight()
							.Padding(8.f, 0.f, 0.f, 0.f)
							[
								ToolWarningArea.ToSharedRef()
							]
							+ SVerticalBox::Slot()
							.AutoHeight()
							[
								ModeDetailsView.ToSharedRef()
							]
						]
					]
				]
			]
		];
}

void FMeshTerrainModeToolkit::UpdateDetailsOverlayWidget() const
{
	if (DetailsOverlayWidget.IsValid())
	{
		const FToolWidget_DragBoxPosition OverlayPosition = GetDetailsOverlayPosition();
		DetailsOverlayWidget->SetDragBoxPosition(OverlayPosition);
		
		const TOptional<FVector2f> OverlaySize = GetDetailsOverlaySize();
		if (OverlaySize.IsSet())
		{
			DetailsOverlayWidget->SetWidthOverride(OverlaySize->X);
			DetailsOverlayWidget->SetHeightOverride(OverlaySize->Y);
		}
		else
		{
			DetailsOverlayWidget->SetWidthOverride(TOptional<float>());
			DetailsOverlayWidget->SetHeightOverride(TOptional<float>());
		}
	}
}

void FMeshTerrainModeToolkit::MakeSectionedDetailsOverlayWidget()
{
	FToolWidget_DragBoxPosition OverlayPosition = GetDetailsOverlayPosition();
	
	const TSharedPtr<SWidget> CloseButton =
		SNew(SHorizontalBox)
		+SHorizontalBox::Slot()
		.AutoWidth()
		[
			SNew(SButton)
			.ContentPadding(2.f)
			.OnClicked_Lambda([this]()
			{
				GetToolkitHost()->RemoveViewportOverlayWidget(DetailsOverlayWidget.ToSharedRef());
					QuickSettingsWidget->SetShowingDetailsView(false);
				return FReply::Handled();
			})
			.ButtonStyle(&FMeshTerrainModeStyle::Get()->GetWidgetStyle<FButtonStyle>("MeshTerrainMode.DetailsViewClose"))
			[
				SNew(SImage)
				.Image(FMeshTerrainModeStyle::Get()->GetBrush("ToolShutdown.Close"))
				.DesiredSizeOverride(FVector2D(16, 16))
				.ColorAndOpacity(FSlateColor::UseForeground())
			]
		];
	
	SAssignNew(DetailsOverlayWidget, UE::ToolWidgets::SDraggableBoxOverlay)
		.Draggable(true)
		.Cursor(EMouseCursor::Default)
		.HAlign(OverlayPosition.HAlign)
		.VAlign(OverlayPosition.VAlign)
		.InitialAlignmentOffset(OverlayPosition.RelativeOffset)
		.OnUserDraggedToNewPosition_Lambda([this]()
			{
				UMeshTerrainModeCustomizationSettings* UISettings = GetMutableDefault<UMeshTerrainModeCustomizationSettings>();
				UISettings->DetailsOverlayPosition = DetailsOverlayWidget->GetDragBoxPosition();
				UISettings->SaveConfig();
			})
		.Visibility(EVisibility::SelfHitTestInvisible)
		.Resizable(ToolWidgets::EResizeEdges::Right | ToolWidgets::EResizeEdges::Bottom)
		.ResizeHandleThickness(6.f)
		.MinimumBoxWidth(300.f)
		.MinimumBoxHeight(400.f)
		.OnUserResized_Lambda([this]()
			{
				UMeshTerrainModeCustomizationSettings* UISettings = GetMutableDefault<UMeshTerrainModeCustomizationSettings>();
				UISettings->DetailsOverlaySize.X = DetailsOverlayWidget->GetWidthOverride().Get(0.f);
				UISettings->DetailsOverlaySize.Y = DetailsOverlayWidget->GetHeightOverride().Get(0.f);
				UISettings->SaveConfig();
			})
		.Content()
		[
			SNew(SBorder)
			.BorderImage(FMeshTerrainModeStyle::Get()->GetBrush("MeshTerrainMode.SectionedDetailsViewHeaderBrush"))
			[
				SNew(SBox)
				// TODO : get exact max dimensions
				.WidthOverride(250.f)
				.MaxDesiredHeight(300.f)
				[
					SNew(SScrollBox)
					.Orientation(Orient_Vertical)
					.ConsumeMouseWheel(EConsumeMouseWheel::Always)
					+ SScrollBox::Slot()
					[
						SNew(SVerticalBox)
						+ SVerticalBox::Slot()
						.VAlign(VAlign_Fill)
						.HAlign(HAlign_Fill)
						.AutoHeight()
						[
							SAssignNew(SectionedDetailsViewWidget, UE::MeshTerrain::SSectionedDetailsViewWidget)
							.ToolShutdownButtons(MakeToolShutdownButtons())
							.CloseDetailsViewButton(CloseButton)
						]
					]
				]
			]
		];
}

void FMeshTerrainModeToolkit::RebuildSectionedDetailsOverlayWidget()
{
	if (!bInActiveTool || !SectionedDetailsView.IsValid())
	{
		return;
	}

	// rebuild DetailsView if its valid and displayed normally OR if its valid and displayed by the Quick Settings widget
	if (SectionedDetailsViewWidget.IsValid() && (bShowDetailsOverlayWidget || QuickSettingsWidget->ShowingDetailsView()))
	{
		UInteractiveTool* Tool = GetScriptableEditorMode()->GetToolManager(EToolsContextScope::EdMode)->GetActiveTool(EToolSide::Left);
		ensure(Tool);

		(bShowQuickSettingsOverlayWidget && QuickSettingsWidget->ShowingDetailsView()) ?
			SectionedDetailsViewWidget->SetActiveTool(Tool, ActiveToolIcon, ActiveToolName, false) :
			SectionedDetailsViewWidget->SetActiveTool(Tool, ActiveToolIcon, ActiveToolName);

		if (CVarDetailsViewConstructionType.GetValueOnGameThread() == 0)
		{
			SectionedDetailsViewWidget->SetDetailsView(SectionedDetailsView);
		}
	}

	// add sectioned details overlay to viewport
	if (DetailsOverlayWidget.IsValid())
	{
		GetToolkitHost()->RemoveViewportOverlayWidget(DetailsOverlayWidget.ToSharedRef());
		if (bShowDetailsOverlayWidget || QuickSettingsWidget->ShowingDetailsView())
		{
			UpdateDetailsOverlayWidget();
			GetToolkitHost()->AddViewportOverlayWidget(DetailsOverlayWidget.ToSharedRef());
		}
	}
}
	
}

#undef LOCTEXT_NAMESPACE