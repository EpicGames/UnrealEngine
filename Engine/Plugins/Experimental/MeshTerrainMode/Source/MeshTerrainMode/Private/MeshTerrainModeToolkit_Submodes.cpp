// Copyright Epic Games, Inc. All Rights Reserved.

#include "InteractiveToolManager.h"
#include "InteractiveToolQueryInterfaces.h"
#include "MeshTerrainModeToolkit.h"
#include "MeshTerrainModeStyle.h"
#include "MeshTerrainModeSettings.h"

#include "Submodes/CreateSubmode.h"
#include "Submodes/EditSubmode.h"
#include "Submodes/ModifiersSubmode.h"
#include "Submodes/PaintSubmode.h"
#include "Submodes/SculptSubmode.h"
#include "Submodes/ShapesSubmode.h"

#include "Features/IModularFeatures.h"
#include "MeshTerrainModeToolExtensions.h"
#include "ToolMenus.h"

#include "Tools/UEdMode.h"
#include "Overlay/SDraggableBoxOverlay.h"
#include "Widgets/SSubmodePalette.h"
#include "Widgets/Layout/SBackgroundBlur.h"

namespace UE::MeshTerrain
{

void FMeshTerrainModeToolkit::MakeSubmodePaletteOverlayWidget()
{
	using namespace UE::MeshTerrain;
	
	UMeshTerrainModeCustomizationSettings* UISettings = GetMutableDefault<UMeshTerrainModeCustomizationSettings>();
	FVector2f SavedWidth = UISettings->SubmodeToolPanelSize;
	
	const FSlateBrush* OverlayBrush = FMeshTerrainModeStyle::Get()->GetBrush("MeshTerrainMode.TransparentBackgroundBrush");
	static const FSlateFontInfo TitleFont = FCoreStyle::GetDefaultFontStyle("Bold", 12);
	
	SAssignNew(SubmodePaletteOverlay, UE::ToolWidgets::SDraggableBoxOverlay)
		.HAlign(HAlign_Left)
		.VAlign(VAlign_Top)
		.InitialAlignmentOffset(FVector2f(16.0f, 16.0f))
		.Cursor(EMouseCursor::Default)
		.Draggable(false)
		.Resizable(ToolWidgets::EResizeEdges::Right)
		.WidthOverride(SavedWidth.X > 0.f ? SavedWidth.X : TOptional<float>())
		.MinimumBoxWidth(104.f)
		.MaximumBoxWidth(200.f)
		.OnUserResized_Lambda([this]()
			{
				// Clamp the width override to the maximum width of the content to keep
				// our overlay resize handles affixed to the bounds of the content.
				UMeshTerrainModeCustomizationSettings* UISettings = GetMutableDefault<UMeshTerrainModeCustomizationSettings>();
				UISettings->SubmodeToolPanelSize.X = SubmodePaletteOverlay->GetWidthOverride().Get(0.f);
				UISettings->SubmodeToolPanelSize.Y = 0.f;
				UISettings->SaveConfig();
			})
		.Visibility(EVisibility::SelfHitTestInvisible)
		.Content()
		[
			SNew(SBorder)
			.BorderImage(OverlayBrush)
			.Padding(0.f)
			.HAlign(HAlign_Left)
			.VAlign(VAlign_Top)
			[
				SAssignNew(SubmodePalette, SSubmodePalette)
				.Width(200.f)
				.ToolPanelFont(FMeshTerrainModeStyle::Get()->GetFontStyle("ToolPanel.Font"))
			]
		];

	// Initialize the widget
	SubmodePalette->SetToolCommandList(GetToolkitCommands());

	for (const TPair<FName, TSharedPtr<FSubmode>>& SubmodePair : Submodes)
	{
		const TSharedPtr<FSubmode>& SubmodePtr = SubmodePair.Value;
		if (!SubmodePtr)
		{
			continue;
		}
		
		FSubmode& Submode = *SubmodePtr;
		const TSharedPtr<FUICommandList> SubmodePaletteCmdList = SubmodePalette->GetSubmodePaletteCommandList();
		SubmodePaletteCmdList.Get()->MapAction(
			Submode.GetEnterSubmodeAction(),
			FExecuteAction::CreateSP(SharedThis(this), &FMeshTerrainModeToolkit::ActivateSubmode, SubmodePtr),
			FCanExecuteAction(),
			FIsActionChecked::CreateLambda([this, &SubmodePtr]() {return ActiveSubmode == SubmodePtr; }));

		// keep track of submodes we've added
		SubmodePalette->AddSubmode(SubmodePtr);
	}
	
	SubmodePalette->CreateSubmodesPalette();

	// Activate Create Submode by default
	ActivateSubmode(Submodes.FindRef(FCreateSubmode::GetStaticName()));
}

void FMeshTerrainModeToolkit::MakeSubmodeToolPanelOverlayWidget()
{
	using namespace UE::MeshTerrain;
	
	UMeshTerrainModeCustomizationSettings* UISettings = GetMutableDefault<UMeshTerrainModeCustomizationSettings>();
	FVector2f SavedWidth = UISettings->SubmodeToolPanelSize;
	
	static const FSlateFontInfo TitleFont = FCoreStyle::GetDefaultFontStyle("Bold", 12);
	
	SAssignNew(SubmodeToolPanelOverlay, UE::ToolWidgets::SDraggableBoxOverlay)
		.HAlign(HAlign_Left)
		.VAlign(VAlign_Top)
		.InitialAlignmentOffset(FVector2f(16.0f, 16.0f))
		.Cursor(EMouseCursor::Default)
		.Draggable(false)
		.Resizable(ToolWidgets::EResizeEdges::Right)
		.WidthOverride(SavedWidth.X > 0.f ? SavedWidth.X : TOptional<float>())
		.MinimumBoxWidth(48.f)
		.MaximumBoxWidth(140.f)
		.OnUserResized_Lambda([this]()
			{
				UMeshTerrainModeCustomizationSettings* UISettings = GetMutableDefault<UMeshTerrainModeCustomizationSettings>();
				UISettings->SubmodeToolPanelSize.X = SubmodeToolPanelOverlay->GetWidthOverride().Get(0.f);
				UISettings->SubmodeToolPanelSize.Y = 0.f;
				UISettings->SaveConfig();
			})
		.Visibility(EVisibility::SelfHitTestInvisible)
		.Content()
		[
			SNew(SBorder)
			.BorderImage(FAppStyle::Get().GetBrush("ProjectBrowser.ProjectTile.DropShadow"))
			.Padding(FMargin(0.0f, 0.0f, 0.0f, 4.0f))
			.HAlign(HAlign_Left)
			.VAlign(VAlign_Top)
			[
				SAssignNew(SubmodeToolPanel, SSubmodeToolPanel)
				.Font(FMeshTerrainModeStyle::Get()->GetFontStyle("ToolPanel.Font"))
			]
		];

	// Initialize the widget
	SubmodeToolPanel->SetToolCommandList(GetToolkitCommands());

	// Activate Create Submode by default
	ActivateSubmode(Submodes.FindRef(FCreateSubmode::GetStaticName()));
}

void FMeshTerrainModeToolkit::InitializeSubmodes()
{
	using namespace UE::MeshTerrain;
	
	const FName SubmodesToolbarName = "MeshTerrainModeSubmodes.SubmodesToolbar";
	if (!UToolMenus::Get()->IsMenuRegistered(SubmodesToolbarName))
	{
		UToolMenus::Get()->RegisterMenu(SubmodesToolbarName, NAME_None, EMultiBoxType::VerticalToolBar);
	}

	auto RegisterSubmode = [this](TSharedPtr<FSubmode> Submode)
	{
		Submodes.Add(Submode->GetName(), Submode);
	};

	// Register default submodes
	RegisterSubmode(MakeShared<FCreateSubmode>(SharedThis(this)));
	RegisterSubmode(MakeShared<FEditSubmode>(SharedThis(this)));
	RegisterSubmode(MakeShared<FModifiersSubmode>(SharedThis(this)));
	RegisterSubmode(MakeShared<FSculptSubmode>(SharedThis(this)));
	RegisterSubmode(MakeShared<FPaintSubmode>(SharedThis(this)));
	RegisterSubmode(MakeShared<FShapesSubmode>(SharedThis(this)));

	TArray<IMeshTerrainModeToolExtension*> Extensions = IModularFeatures::Get().GetModularFeatureImplementations<IMeshTerrainModeToolExtension>(
		IMeshTerrainModeToolExtension::GetModularFeatureName());
	if (Extensions.Num() > 0)
	{
		for (int32 k = 0; k < Extensions.Num(); ++k)
		{
			TArray<FExtensionSubmodeDescription> ExtSubmodes;
			Extensions[k]->GetExtensionSubmodes(ExtSubmodes);
			
			for (const FExtensionSubmodeDescription& SubmodeDesc : ExtSubmodes)
			{
				if (SubmodeDesc.MakeNewSubmode)
				{
					RegisterSubmode(SubmodeDesc.MakeNewSubmode());
				}
			}
		}

		// Post-pass process submode addons
		for (int32 k = 0; k < Extensions.Num(); ++k)
		{
			TArray<FExtensionSubmodeAddon> ExtSubmodeAddons;
			Extensions[k]->GetExtensionSubmodeAddons(ExtSubmodeAddons);
			
			for (const FExtensionSubmodeAddon& Addon : ExtSubmodeAddons)
			{
				if (TSharedPtr<FSubmode> Submode = Submodes.FindRef(Addon.SubmodeName))
				{
					Submode->AddToolPalette(Addon.ToolPalette);
				}
			}
		}
	}
}

TSharedPtr<UE::MeshTerrain::FSubmode> FMeshTerrainModeToolkit::GetDefaultSubmode() const
{
	return Submodes.FindRef(UE::MeshTerrain::FCreateSubmode::GetStaticName());
}

void FMeshTerrainModeToolkit::ActivateSubmode(TSharedPtr<UE::MeshTerrain::FSubmode> SubmodePtr)
{
	if (SubmodePtr)
	{
		// deactivate previous submode
		DeactivateSubmode();
		
		ActiveSubmode = SubmodePtr;
		if (SubmodeToolPanel)
		{
			SubmodeToolPanel->SetActiveSubmode(SubmodePtr);
		}
		if (SubmodePalette)
		{
			SubmodePalette->EnterSubmode(SubmodePtr);
		}
		ActiveSubmode->Activate();
	}
}

void FMeshTerrainModeToolkit::DeactivateSubmode()
{
	UEdMode* EdMode = OwningEditorMode.Get();
	if(IsInActiveTool() && EdMode)
	{
		EToolShutdownType ShutdownType = EToolShutdownType::Accept;
		if (UInteractiveToolManager* ToolManager = EdMode->GetToolManager())
		{
			UInteractiveTool* ActiveLeftTool = ToolManager->GetActiveTool(EToolSide::Left);
			if (IInteractiveToolShutdownQueryAPI* ShutdownQueryAPI = Cast<IInteractiveToolShutdownQueryAPI>(ActiveLeftTool))
			{
				ShutdownType = ShutdownQueryAPI->GetPreferredShutdownType(EToolShutdownReason::SwitchTool, ShutdownType);
			}
			ToolManager->DeactivateTool(EToolSide::Left, ShutdownType);
		}
	}

	if (ActiveSubmode)
	{
		ActiveSubmode->Deactivate();
		ActiveSubmode = nullptr;
	}
}

FToolWidget_DragBoxPosition FMeshTerrainModeToolkit::GetQuickSettingsOverlayPosition() const
{
	FToolWidget_DragBoxPosition OverlayPosition
	{
		FVector2f(0.0f, 16.0f),
		HAlign_Center,
		VAlign_Bottom,
	};
	UMeshTerrainModeCustomizationSettings* CustomizationSettings = GetMutableDefault<UMeshTerrainModeCustomizationSettings>();
	if (CustomizationSettings->QuickSettingsOverlayPosition != FToolWidget_DragBoxPosition())
	{
		OverlayPosition = CustomizationSettings->QuickSettingsOverlayPosition;
	}
	return OverlayPosition;
}

void FMeshTerrainModeToolkit::MakeQuickSettingsOverlayWidget()
{
	FToolWidget_DragBoxPosition OverlayPosition = GetQuickSettingsOverlayPosition();
	SAssignNew(QuickSettingsOverlayWidget, UE::ToolWidgets::SDraggableBoxOverlay)
		.HAlign(OverlayPosition.HAlign)
		.VAlign(OverlayPosition.VAlign)
		.Draggable(true)
		.DragOverridesCenterAlignment(true)
		.OnUserDraggedToNewPosition_Lambda([this]()
			{
				UMeshTerrainModeCustomizationSettings* CustomizationSettings = GetMutableDefault<UMeshTerrainModeCustomizationSettings>();
				CustomizationSettings->QuickSettingsOverlayPosition = QuickSettingsOverlayWidget->GetDragBoxPosition();
				CustomizationSettings->SaveConfig();
			})
		.InitialAlignmentOffset(OverlayPosition.RelativeOffset)
		.Cursor(EMouseCursor::Default)
		.Visibility(EVisibility::SelfHitTestInvisible)
		.Content()
		[
			SNew(SBorder)
			.BorderImage(FAppStyle::Get().GetBrush("ProjectBrowser.ProjectTile.DropShadow"))
			.Padding(FMargin(0.0f, 0.0f, 0.0f, 4.0f))
			.HAlign(HAlign_Center)
			.VAlign(VAlign_Top)
			[
				SNew(SBorder)
				.Padding(6.f)
				.BorderImage( FMeshTerrainModeStyle::Get()->GetBrush("MeshTerrainMode.SubmodePaletteDarkerBrush"))
				[
					SAssignNew(QuickSettingsWidget, SModelingQuickSettingsWidget)
					.ToolShutdownButtons(MakeToolShutdownButtons())
				]
			]
		];
}

void FMeshTerrainModeToolkit::RebuildQuickSettingsWidget(UInteractiveTool* Tool) const
{
	// set up the ModelingQuickSettings Widget
	if (QuickSettingsWidget.IsValid())
	{
		QuickSettingsWidget->SetDetailsView(ModeDetailsView);
		// when tool is null, we are not switching tools, but still want to rebuild the widget; use currently active tool instead
		if (!Tool)
		{
			Tool = GetScriptableEditorMode()->GetToolManager(EToolsContextScope::EdMode)->GetActiveTool(EToolSide::Left);
		}
		ensure(Tool);
		QuickSettingsWidget->SetActiveTool(Tool);
	}
	
	// add the QuickSettings widget to the viewport
	if (QuickSettingsOverlayWidget.IsValid())
	{
		UpdateQuickSettingsOverlayWidget();
		GetToolkitHost()->RemoveViewportOverlayWidget(QuickSettingsOverlayWidget.ToSharedRef());
		if (bShowQuickSettingsOverlayWidget)
		{
			GetToolkitHost()->AddViewportOverlayWidget(QuickSettingsOverlayWidget.ToSharedRef());
		}
	}
}
	
void FMeshTerrainModeToolkit::UpdateQuickSettingsOverlayWidget() const
{
	if (QuickSettingsOverlayWidget.IsValid())
	{
		const FToolWidget_DragBoxPosition OverlayPosition = GetQuickSettingsOverlayPosition();
		QuickSettingsOverlayWidget->SetDragBoxPosition(OverlayPosition);
	}
}

}