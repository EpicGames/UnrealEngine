// Copyright Epic Games, Inc. All Rights Reserved.

#include "CreateSubmode.h"
#include "MeshTerrainModeToolkit.h"

#include "MeshTerrainModeManagerActions.h"

#define LOCTEXT_NAMESPACE "FCreateSubmode"

using namespace UE::MeshTerrain;

FCreateSubmode::FCreateSubmode(const TSharedPtr<FModeToolkit>& InToolkit)
	: FSubmode(InToolkit)
{
	FMeshTerrainModeToolkit* ModeToolkit = static_cast<FMeshTerrainModeToolkit*>(Toolkit.Pin().Get());
	bool bShowExperimentalTools = ModeToolkit ? ModeToolkit->ExperimentalToolsEnabled() : false;
	
	const FMeshTerrainModeManagerCommands& Commands = FMeshTerrainModeManagerCommands::Get();
	TArray<TSharedPtr<FUICommandInfo>> CreateCommands =
	{
		Commands.BeginCreateMegaMeshRectangleTool,
		Commands.BeginHeightmapImport,
		Commands.BeginDrawSplineTool
	};
	if (bShowExperimentalTools)
	{
		CreateCommands.Add(Commands.BeginPatternTool);
	}
	ToolPalettes.Emplace(LOCTEXT("CreatePaletteLabel", "Create"), CreateCommands);
	EnterSubmodeAction = GetStaticEnterSubmodeAction();
}

FName FCreateSubmode::GetName() const
{
	return GetStaticName();
}

void FCreateSubmode::Activate()
{
	if (FMeshTerrainModeToolkit* ModeToolkit = static_cast<FMeshTerrainModeToolkit*>(Toolkit.Pin().Get()))
	{
		ModeToolkit->ShowDetailsOverlayWidget(true);
		ModeToolkit->ShowQuickSettingsOverlayWidget(false);
		ModeToolkit->ShowNumericalUIOverlayWidget(false);
	}
}

void FCreateSubmode::Deactivate()
{
}

FName FCreateSubmode::GetStaticName()
{
	return GetStaticEnterSubmodeAction()->GetCommandName();
}

TSharedPtr<FUICommandInfo> FCreateSubmode::GetStaticEnterSubmodeAction()
{
	const FMeshTerrainModeManagerCommands& Commands = FMeshTerrainModeManagerCommands::Get();
	return Commands.EnterCreateSubmode;
}


#undef LOCTEXT_NAMESPACE