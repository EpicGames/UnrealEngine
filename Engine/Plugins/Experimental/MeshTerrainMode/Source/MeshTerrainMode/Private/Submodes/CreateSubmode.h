// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Submode.h"

namespace UE::MeshTerrain
{

class FCreateSubmode : public FSubmode
{
public:
	FCreateSubmode(const TSharedPtr<FModeToolkit>& InToolkit);
	virtual ~FCreateSubmode() override = default;

	// FSubmode 
	virtual FName GetName() const override;
	virtual void Activate() override;
	virtual void Deactivate() override;

	static FName GetStaticName();
	static TSharedPtr<FUICommandInfo> GetStaticEnterSubmodeAction();
};

}
