// Copyright Epic Games, Inc. All Rights Reserved.

#include "EditSubmode.h"
#include "MeshTerrainModeToolkit.h"

#include "MeshTerrainModeManagerActions.h"

#define LOCTEXT_NAMESPACE "FEditSubmode"

using namespace UE::MeshTerrain;

FEditSubmode::FEditSubmode(const TSharedPtr<FModeToolkit>& InToolkit)
: FSubmode(InToolkit)
{
	const FMeshTerrainModeToolkit* const ModeToolkit = static_cast<FMeshTerrainModeToolkit*>(Toolkit.Pin().Get());
	const bool bShowExperimentalTools = ModeToolkit ? ModeToolkit->ExperimentalToolsEnabled() : false;

	const FMeshTerrainModeManagerCommands& Commands = FMeshTerrainModeManagerCommands::Get();
	const TArray<TSharedPtr<FUICommandInfo>> EditCommands =
	{
		Commands.BeginConvertMegaMeshTool,
		Commands.BeginExpandMegaMeshTool,
		Commands.BeginSplitMegaMeshTool,
		Commands.BeginStitchMegaMeshTool,
		Commands.BeginMergeMegaMeshTool,
		Commands.BeginResectionMeshTool
	};
	TArray<TSharedPtr<FUICommandInfo>> GeneralCommands =
	{
		Commands.BeginEditPivotTool,
		Commands.BeginBakeTransformTool,
		Commands.BeginAttributeEditorTool,
		Commands.BeginMeshInspectorTool
	};
	if (bShowExperimentalTools)
	{
		GeneralCommands.Insert(Commands.BeginDuplicateMeshesTool, 0);
	}
	ToolPalettes.Emplace(LOCTEXT("EditPaletteLabel", "Edit"), EditCommands);
	ToolPalettes.Emplace(LOCTEXT("GeneralPaletteLabel", "General"), GeneralCommands);
	EnterSubmodeAction = GetStaticEnterSubmodeAction();
}

FName FEditSubmode::GetName() const
{
	return GetStaticName();
}

void FEditSubmode::Activate()
{
	if (FMeshTerrainModeToolkit* ModeToolkit = static_cast<FMeshTerrainModeToolkit*>(Toolkit.Pin().Get()))
	{
		ModeToolkit->ShowDetailsOverlayWidget(true);
		ModeToolkit->ShowQuickSettingsOverlayWidget(false);
		ModeToolkit->ShowNumericalUIOverlayWidget(false);
	}
}

void FEditSubmode::Deactivate()
{
}

FName FEditSubmode::GetStaticName()
{
	return GetStaticEnterSubmodeAction()->GetCommandName();
}

TSharedPtr<FUICommandInfo> FEditSubmode::GetStaticEnterSubmodeAction()
{
	const FMeshTerrainModeManagerCommands& Commands = FMeshTerrainModeManagerCommands::Get();
	return Commands.EnterEditSubmode;
}


#undef LOCTEXT_NAMESPACE