// Copyright Epic Games, Inc. All Rights Reserved.

#include "ModifiersSubmode.h"
#include "MeshPartitionModifierActor.h"
#include "MeshTerrainModeToolkit.h"
#include "MeshTerrainModeStyle.h"
#include "Modifiers/MeshPartitionBooleanModifier.h"
#include "Modifiers/MeshPartitionLatticeModifier.h"
#include "Modifiers/MeshPartitionInstancedPatchModifier.h"
#include "Modifiers/MeshPartitionMeshProjectModifier.h"
#include "Modifiers/MeshPartitionNoiseModifier.h"
#include "Modifiers/MeshPartitionPatchModifier.h"
#include "Modifiers/MeshPartitionProjectSculptLayersModifier.h"
#include "Modifiers/MeshPartitionRemeshModifier.h"
#include "Modifiers/MeshPartitionSplineModifier.h"
#include "Modifiers/MeshPartitionSplineRemeshModifier.h"
#include "Modifiers/MeshPartitionTexturePatchModifier.h"
#include "Widgets/SWidget.h"
#include "Widgets/SPlaceableItemEntry.h"
#include "Widgets/Layout/SSeparator.h"

#include "MeshTerrainModeManagerActions.h"

#define LOCTEXT_NAMESPACE "FModifiersSubmode"

using namespace UE::MeshTerrain;

FModifiersSubmode::FModifiersSubmode(const TSharedPtr<FModeToolkit>& InToolkit)
	: FSubmode(InToolkit)
{
	const FMeshTerrainModeToolkit* const ModeToolkit = static_cast<FMeshTerrainModeToolkit*>(Toolkit.Pin().Get());
	const bool bShowExperimentalTools = ModeToolkit ? ModeToolkit->ExperimentalToolsEnabled() : false;

	EnterSubmodeAction = GetStaticEnterSubmodeAction();

	auto CreatePlaceableItem = [](UClass* ModifierType, const FText& DisplayName)
	{
		TSharedPtr<FPlaceableItem> Item = MakeShared<FPlaceableItem>(MeshPartition::UModifierActorFactory::StaticClass(), FAssetData(ModifierType));
		Item->DisplayName = DisplayName;
		return Item;
	};

	TArray<TSharedPtr<FPlaceableItem>> DisplacePlaceables =
	{
		CreatePlaceableItem(MeshPartition::UMeshProjectModifier::StaticClass(), LOCTEXT("MeshProjectModifier", "Mesh")),
		CreatePlaceableItem(MeshPartition::UTexturePatchModifier::StaticClass(), LOCTEXT("TexturePatchModifier", "Texture")),
		CreatePlaceableItem(MeshPartition::USplineModifier::StaticClass(), LOCTEXT("SplineModifier", "Spline")),
		CreatePlaceableItem(MeshPartition::UProjectMeshLayersModifier::StaticClass(), LOCTEXT("SculptLayersModifier", "Brush")),
	};
	if (bShowExperimentalTools)
	{
		DisplacePlaceables.Add(CreatePlaceableItem(MeshPartition::ULatticeModifier::StaticClass(), LOCTEXT("LatticeModifier", "Lattice")));
	}
	DisplacePlaceables.Add(CreatePlaceableItem(MeshPartition::UNoiseModifier::StaticClass(), LOCTEXT("NoiseModifier", "Noise")));
	if (bShowExperimentalTools)
	{
		DisplacePlaceables.Add(CreatePlaceableItem(MeshPartition::UPatchModifier::StaticClass(), LOCTEXT("PatchModifier", "Patch")));
	}

	ModifierPlaceables.Add(
		{
			LOCTEXT("DisplaceSectionLabel", "Displace"),
			MoveTemp(DisplacePlaceables)
		});
	ModifierPlaceables.Add(
		{
			LOCTEXT("OtherSectionLabel", "Other"),
			{
				CreatePlaceableItem(MeshPartition::UBooleanModifier::StaticClass(), LOCTEXT("BooleanModifier", "Boolean")),
				CreatePlaceableItem(MeshPartition::URemeshModifier::StaticClass(), LOCTEXT("RemeshModifier", "Remesh")),
				CreatePlaceableItem(MeshPartition::USplineRemeshModifier::StaticClass(), LOCTEXT("SplineRemeshModifier", "Spline Remesh")),
			}
		});
}

FName FModifiersSubmode::GetName() const
{
	return GetStaticName();
}

void FModifiersSubmode::Activate()
{
	if (FMeshTerrainModeToolkit* ModeToolkit = static_cast<FMeshTerrainModeToolkit*>(Toolkit.Pin().Get()))
	{
		ModeToolkit->ShowDetailsOverlayWidget(false);
		ModeToolkit->ShowQuickSettingsOverlayWidget(false);
		ModeToolkit->ShowNumericalUIOverlayWidget(false);
	}
}

void FModifiersSubmode::Deactivate()
{
}

FName FModifiersSubmode::GetStaticName()
{
	return GetStaticEnterSubmodeAction()->GetCommandName();
}

TSharedPtr<FUICommandInfo> FModifiersSubmode::GetStaticEnterSubmodeAction()
{
	const FMeshTerrainModeManagerCommands& Commands = FMeshTerrainModeManagerCommands::Get();
	return Commands.EnterModifiersSubmode;
}

TSharedPtr<SWidget> FModifiersSubmode::GetToolPaletteHeader()
{
	auto CreatePlaceableItemWidget = [](const TSharedPtr<FPlaceableItem>& InItem)
	{
		TSharedRef<SPlaceableItemEntry> Entry =  SNew(SPlaceableItemEntry, InItem.ToSharedRef())
			.IconSize(FVector2D(20.0, 20.0))
			.Font(FMeshTerrainModeStyle::Get()->GetFontStyle("ToolPanel.Font"))
			.Clipping( EWidgetClipping::ClipToBounds );
		return Entry;
	};

	int GroupCount = 0;
	SVerticalBox::FArguments HeaderArgs;
	for (const FPlaceableGroup& Group : ModifierPlaceables)
	{
		if (GroupCount > 0)
		{
			HeaderArgs
			+ SVerticalBox::Slot()
			.Padding(2.0f, 4.0f)
			.AutoHeight()
			[
				SNew(SSeparator)
				.SeparatorImage(FMeshTerrainModeStyle::Get()->GetBrush("ToolPanel.SeparatorBrush"))
				.Thickness(1.0f)
			];
		}
		for (const TSharedPtr<FPlaceableItem>& Item : Group.Placeables)
		{
			HeaderArgs
			+ SVerticalBox::Slot()
			.Padding(4.0f, 2.0f)
			.AutoHeight()
			[
				CreatePlaceableItemWidget(Item)
			];
		}
		++GroupCount;
	}
	return SArgumentNew(HeaderArgs, SVerticalBox);
}



#undef LOCTEXT_NAMESPACE