// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Submode.h"

struct FPlaceableItem;

namespace UE::MeshTerrain
{

class FModifiersSubmode : public FSubmode
{
public:
	FModifiersSubmode(const TSharedPtr<FModeToolkit>& InToolkit);
	virtual ~FModifiersSubmode() override = default;

	// FSubmode 
	virtual FName GetName() const override;
	virtual void Activate() override;
	virtual void Deactivate() override;

	static FName GetStaticName();
	static TSharedPtr<FUICommandInfo> GetStaticEnterSubmodeAction();
	
	virtual TSharedPtr<SWidget> GetToolPaletteHeader() override;

private:
	struct FPlaceableGroup
	{
		FText Label;
		TArray<TSharedPtr<FPlaceableItem>> Placeables;
	};
	TArray<FPlaceableGroup> ModifierPlaceables;
};

}
