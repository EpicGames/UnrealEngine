// Copyright Epic Games, Inc. All Rights Reserved.

#include "PaintSubmode.h"
#include "MeshTerrainModeToolkit.h"
#include "MeshPartitionAttributePaintTool.h"

#include "MeshTerrainModeManagerActions.h"

#define LOCTEXT_NAMESPACE "FPaintSubmode"

using namespace UE::MeshTerrain;

FPaintSubmode::FPaintSubmode(const TSharedPtr<FModeToolkit>& InToolkit)
	: FSubmode(InToolkit)
{
	const FMeshTerrainModeManagerCommands& Commands = FMeshTerrainModeManagerCommands::Get();
	const TArray<TSharedPtr<FUICommandInfo>> PaintCommands =
	{
		Commands.BeginMeshAttributePaintTool,
		Commands.BeginMeshVertexPaintTool,
	};
	const TArray<TSharedPtr<FUICommandInfo>> AttrCommands =
	{
		Commands.BeginAttributeEditorTool,
	};
	ToolPalettes.Emplace(LOCTEXT("PaintPaletteLabel", "Paint"), PaintCommands);
	ToolPalettes.Emplace(LOCTEXT("AttrPaletteLabel", "Attributes"), AttrCommands);
	EnterSubmodeAction = GetStaticEnterSubmodeAction();
}

FName FPaintSubmode::GetName() const
{
	return GetStaticName();
}

void FPaintSubmode::Activate()
{
	if (FMeshTerrainModeToolkit* ModeToolkit = static_cast<FMeshTerrainModeToolkit*>(Toolkit.Pin().Get()))
	{
		ModeToolkit->ShowDetailsOverlayWidget(true);
		ModeToolkit->ShowQuickSettingsOverlayWidget(false);
		ModeToolkit->ShowNumericalUIOverlayWidget(false);
	}
}

void FPaintSubmode::Deactivate()
{
}

void UE::MeshTerrain::FPaintSubmode::OnToolStarted(UInteractiveTool* Tool)
{
	FSubmode::OnToolStarted(Tool);

	if (FMeshTerrainModeToolkit* ModeToolkit = static_cast<FMeshTerrainModeToolkit*>(Toolkit.Pin().Get()))
	{
		// For paint tool, use the quick settings. For everything else, don't.
		bool bIsPaintTool = Cast<UE::MeshPartition::UAttributePaintTool>(Tool) != nullptr;
		ModeToolkit->ShowDetailsOverlayWidget(!bIsPaintTool);
		ModeToolkit->ShowQuickSettingsOverlayWidget(bIsPaintTool);
	}
}

FName FPaintSubmode::GetStaticName()
{
	return GetStaticEnterSubmodeAction()->GetCommandName();
}

TSharedPtr<FUICommandInfo> FPaintSubmode::GetStaticEnterSubmodeAction()
{
	const FMeshTerrainModeManagerCommands& Commands = FMeshTerrainModeManagerCommands::Get();
	return Commands.EnterPaintSubmode;
}


#undef LOCTEXT_NAMESPACE