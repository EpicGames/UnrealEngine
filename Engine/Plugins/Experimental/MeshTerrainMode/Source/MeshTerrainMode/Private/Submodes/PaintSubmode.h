// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Submode.h"

namespace UE::MeshTerrain
{

class FPaintSubmode : public FSubmode
{
public:
	FPaintSubmode(const TSharedPtr<FModeToolkit>& InToolkit);
	virtual ~FPaintSubmode() override = default;

	// FSubmode
	virtual FName GetName() const override;
	virtual void Activate() override;
	virtual void Deactivate() override;
	virtual void OnToolStarted(UInteractiveTool* Tool) override;

	static FName GetStaticName();
	static TSharedPtr<FUICommandInfo> GetStaticEnterSubmodeAction();
};

}