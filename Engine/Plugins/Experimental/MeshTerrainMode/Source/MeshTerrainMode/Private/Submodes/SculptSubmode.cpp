// Copyright Epic Games, Inc. All Rights Reserved.

#include "SculptSubmode.h"
#include "MeshTerrainModeToolkit.h"

#include "MeshTerrainModeManagerActions.h"

#define LOCTEXT_NAMESPACE "FSculptSubmode"

using namespace UE::MeshTerrain;

FSculptSubmode::FSculptSubmode(const TSharedPtr<FModeToolkit>& InToolkit)
	: FSubmode(InToolkit)
{
	const FMeshTerrainModeManagerCommands& Commands = FMeshTerrainModeManagerCommands::Get();
	const TArray<TSharedPtr<FUICommandInfo>> SculptCommands =
	{
		Commands.BeginSculptMeshOffsetBrushTool,
		Commands.BeginSculptMeshMoveBrushTool,
		//Commands.BeginSculptMeshPullKelvinBrushTool,
		//Commands.BeginSculptMeshPullSharpKelvinBrushTool,
		Commands.BeginSculptMeshSmoothBrushTool,
		//Commands.BeginSculptMeshSmoothFillBrushTool,
		//Commands.BeginSculptMeshSculptViewBrushTool,
		//Commands.BeginSculptMeshSculptMaxBrushTool,
		//Commands.BeginSculptMeshInflateBrushTool,
		//Commands.BeginSculptMeshScaleKelvinBrushTool,
		Commands.BeginSculptMeshPinchBrushTool,
		//Commands.BeginSculptMeshTwistKelvinBrushTool,
		Commands.BeginSculptMeshFlattenBrushTool,
		//Commands.BeginSculptMeshPlaneBrushTool,
		//Commands.BeginSculptMeshPlaneViewAlignedBrushTool,
		//Commands.BeginSculptMeshFixedPlaneBrushTool
		Commands.BeginSculptMeshEraseLayerTool,
	};
	const TArray<TSharedPtr<FUICommandInfo>> HeightSculptCommands =
	{
		Commands.BeginHeightSculptBrushTool,
		Commands.BeginHeightSmoothBrushTool,
		Commands.BeginHeightFlattenBrushTool,
		Commands.BeginSlopeErodeBrushTool,
	};
	ToolPalettes.Emplace(LOCTEXT("SculptPaletteLabel", "Sculpt"), SculptCommands);
	ToolPalettes.Emplace(LOCTEXT("HeightSculptPaletteLabel", "Height Sculpt"), HeightSculptCommands);
	EnterSubmodeAction = GetStaticEnterSubmodeAction();
}

FName FSculptSubmode::GetName() const
{
	return GetStaticName();
}

void FSculptSubmode::Activate()
{
	if (FMeshTerrainModeToolkit* ModeToolkit = static_cast<FMeshTerrainModeToolkit*>(Toolkit.Pin().Get()))
	{
		ModeToolkit->ShowDetailsOverlayWidget(false);
		ModeToolkit->ShowQuickSettingsOverlayWidget(true);
		ModeToolkit->ShowNumericalUIOverlayWidget(false);
	}
}

void FSculptSubmode::Deactivate()
{
}

FName FSculptSubmode::GetStaticName()
{
	return GetStaticEnterSubmodeAction()->GetCommandName();
}

TSharedPtr<FUICommandInfo> FSculptSubmode::GetStaticEnterSubmodeAction()
{
	const FMeshTerrainModeManagerCommands& Commands = FMeshTerrainModeManagerCommands::Get();
	return Commands.EnterSculptSubmode;
}


#undef LOCTEXT_NAMESPACE