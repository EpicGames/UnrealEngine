// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Submode.h"

namespace UE::MeshTerrain
{

class FSculptSubmode : public FSubmode
{
public:
	FSculptSubmode(const TSharedPtr<FModeToolkit>& InToolkit);
	virtual ~FSculptSubmode() override = default;

	// FSubmode
	virtual FName GetName() const override;
	virtual void Activate() override;
	virtual void Deactivate() override;

	static FName GetStaticName();
	static TSharedPtr<FUICommandInfo> GetStaticEnterSubmodeAction();
};
	
}	
	