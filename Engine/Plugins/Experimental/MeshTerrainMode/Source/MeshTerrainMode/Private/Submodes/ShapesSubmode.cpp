// Copyright Epic Games, Inc. All Rights Reserved.

#include "ShapesSubmode.h"
#include "MeshTerrainModeToolkit.h"

#include "MeshTerrainModeManagerActions.h"

#define LOCTEXT_NAMESPACE "FShapesSubmode"

using namespace UE::MeshTerrain;

FShapesSubmode::FShapesSubmode(const TSharedPtr<FModeToolkit>& InToolkit)
	: FSubmode(InToolkit)
{
	FMeshTerrainModeToolkit* ModeToolkit = static_cast<FMeshTerrainModeToolkit*>(Toolkit.Pin().Get());
	bool bShowExperimentalTools = ModeToolkit ? ModeToolkit->ExperimentalToolsEnabled() : false;
	
	const FMeshTerrainModeManagerCommands& Commands = FMeshTerrainModeManagerCommands::Get();
	TArray<TSharedPtr<FUICommandInfo>> ShapesCommands =
	{
		Commands.BeginAddBoxPrimitiveTool,
		Commands.BeginAddSpherePrimitiveTool,
		Commands.BeginAddCylinderPrimitiveTool,
		Commands.BeginAddCapsulePrimitiveTool,
		Commands.BeginAddConePrimitiveTool,
		Commands.BeginAddTorusPrimitiveTool,
		Commands.BeginAddRectanglePrimitiveTool,
		Commands.BeginAddDiscPrimitiveTool,
	};

	if (bShowExperimentalTools)
	{
		ShapesCommands.Add(Commands.BeginAddArrowPrimitiveTool);
		ShapesCommands.Add(Commands.BeginAddStairsPrimitiveTool);
	}

	ToolPalettes.Emplace(LOCTEXT("AddModifierPaletteLabel", "Add Modifiers"), ShapesCommands);
	EnterSubmodeAction = GetStaticEnterSubmodeAction();
}

FName FShapesSubmode::GetName() const
{
	return GetStaticName();
}

void FShapesSubmode::Activate()
{
	if (FMeshTerrainModeToolkit* ModeToolkit = static_cast<FMeshTerrainModeToolkit*>(Toolkit.Pin().Get()))
    {
    	ModeToolkit->ShowDetailsOverlayWidget(true);
    	ModeToolkit->ShowQuickSettingsOverlayWidget(false);
    	ModeToolkit->ShowNumericalUIOverlayWidget(false);
    }
}

void FShapesSubmode::Deactivate()
{
}

FName FShapesSubmode::GetStaticName()
{
	return GetStaticEnterSubmodeAction()->GetCommandName();
}

TSharedPtr<FUICommandInfo> FShapesSubmode::GetStaticEnterSubmodeAction()
{
	const FMeshTerrainModeManagerCommands& Commands = FMeshTerrainModeManagerCommands::Get();
	return Commands.EnterShapesSubmode;
}


#undef LOCTEXT_NAMESPACE