// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Submode.h"

namespace UE::MeshTerrain
{

class FShapesSubmode : public FSubmode
{
public:
	FShapesSubmode(const TSharedPtr<FModeToolkit>& InToolkit);
	virtual ~FShapesSubmode() override = default;

	// FSubmode 
	virtual FName GetName() const override;
	virtual void Activate() override;
	virtual void Deactivate() override;

	static FName GetStaticName();
	static TSharedPtr<FUICommandInfo> GetStaticEnterSubmodeAction();
};

}
