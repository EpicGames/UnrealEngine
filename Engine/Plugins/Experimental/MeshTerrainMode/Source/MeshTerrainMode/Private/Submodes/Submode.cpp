// Copyright Epic Games, Inc. All Rights Reserved.

#include "Submodes/Submode.h"
#include "Toolkits/BaseToolkit.h"

using namespace UE::MeshTerrain;

// FSubmodeToolPalette

FSubmodeToolPalette::FSubmodeToolPalette(
	const FText& InPaletteLabel,
	const TArray<TSharedPtr<FUICommandInfo>>& InToolCommands)
: PaletteLabel(InPaletteLabel)
, ToolCommands(InToolCommands)
{
	
}

FSubmodeToolPalette::FSubmodeToolPalette(const FSubmodeToolPalette& Other)
: PaletteLabel(Other.PaletteLabel)
, ToolCommands(Other.ToolCommands)
{
	
}


// FSubmode

FSubmode::FSubmode(const TSharedPtr<FModeToolkit>& InToolkit)
	: Toolkit(InToolkit)
{
	
}


TSharedPtr<FUICommandInfo> FSubmode::GetEnterSubmodeAction() const
{
	return EnterSubmodeAction;
}

const TArray<FSubmodeToolPalette>& FSubmode::GetToolPalettes() const
{
	return ToolPalettes;
}

void FSubmode::AddToolPalette(const FSubmodeToolPalette& Palette)
{
	ToolPalettes.Add(Palette);
}
