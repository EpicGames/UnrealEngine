// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Templates/SharedPointer.h"
#include "Internationalization/Text.h"

struct FClickContext;
class FModeToolkit;
class UInteractiveTool;
class FUICommandInfo;
class IDetailsView;
class SWidget;

namespace UE::MeshTerrain
{

/** A struct that provides the data for a single tool Palette*/
struct FSubmodeToolPalette
{
	FSubmodeToolPalette(const FText& InPaletteLabel, const TArray<TSharedPtr<FUICommandInfo>>& InToolCommands);
	FSubmodeToolPalette(const FSubmodeToolPalette& Other);

	FText PaletteLabel;
	TArray<TSharedPtr<FUICommandInfo>> ToolCommands;
};

class FSubmode
{
public:
	FSubmode(const TSharedPtr<FModeToolkit>& InToolkit);
	virtual ~FSubmode() = default;

	virtual FName GetName() const = 0;

	virtual void Activate() { }
	virtual void Deactivate() { }

	// Called when one of a submode's tools has begun or ended
	virtual void OnToolStarted(UInteractiveTool* Tool) { }
	virtual void OnToolEnded(UInteractiveTool* Tool) { }

	virtual bool HandleClick(FClickContext& InOutContext) const { return false; }

	TSharedPtr<FUICommandInfo> GetEnterSubmodeAction() const;
	virtual TSharedPtr<SWidget> GetToolPaletteHeader() { return nullptr; }
	const TArray<FSubmodeToolPalette>& GetToolPalettes() const;
	void AddToolPalette(const FSubmodeToolPalette& Palette);

protected:
	TWeakPtr<FModeToolkit> Toolkit = nullptr;
	TSharedPtr<FUICommandInfo> EnterSubmodeAction = nullptr;
	TArray<FSubmodeToolPalette> ToolPalettes;
};

}