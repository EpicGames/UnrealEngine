// Copyright Epic Games, Inc. All Rights Reserved.

#include "MeshTerrainModeUtil.h"

#include "MeshVertexSculptTool.h"
#include "MeshTerrainModeStyle.h"
#include "InteractiveTool.h"
#include "Brushes/SlateRoundedBoxBrush.h"
#include "PropertyEditorUtils.h"
#include "Styling/SlateStyleRegistry.h"
#include "Widgets/Input/SEditableTextBox.h"
#include "Widgets/Input/SSearchableComboBox.h"
#include "Widgets/Input/SSpinBox.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "Widgets/Input/SCheckBox.h"
#include "Widgets/Input/SComboButton.h"
#include "Widgets/Layout/SBox.h"
#include "IDetailsView.h"
#include "MeshPartitionAttributePaintTool.h"
#include "MeshPartitionHeightSculptTool.h"
#include "MeshTerrainModeSettings.h"
#include "ModelingWidgets/ModelingCustomizationUtil.h"
#include "ModelingWidgets/SDynamicNumericEntry.h"
#include "ModelingWidgets/SToolInputAssetComboPanel.h"
#include "Widgets/Input/STextComboBox.h"
#include "Widgets/Layout/SSeparator.h"

static TArray<TSharedPtr<FString>> EnumOptions;

#define LOCTEXT_NAMESPACE "MeshTerrainModeUtil"

UE::MeshTerrain::FPropertyWidget::FPropertyWidget(const FText& InPropertyName, const TSharedRef<SWidget>& InWidgetRepresentation)
: PropertyName(InPropertyName), WidgetRepresentation(InWidgetRepresentation){}

void UE::MeshTerrain::CollectTaggedWidgetsRecursive(
	FProperty* Property,
	UObject* PropListOwner,
	const TCHAR* Tag,
	UInteractiveTool* ActiveTool,
	TMap<int, TArray<FPropertyWidget>>& PriorityToWidgets,
	const TFunction<void(FProperty* Prop, UObject* PropListOwner, TArray<FPropertyWidget>& Widgets)>& HandlePropsFunc,
	TMap<UClass*, FGetQuickPropertyCustomization> Customizations,
	const EQuickPropertyDisplay CustomizationDisplayType,
	TMap<UScriptStruct*, FGetQuickPropertyCustomization> StructTypeCustomizations)
{
	
	// if property is valid, editable, and has the given tag
	if (Property && Property->HasAnyPropertyFlags(CPF_Edit) && Property->HasMetaData(Tag))
	{
		// retrieve the priority associated with each property. If none, use INT_MAX
		const FString& ModelingPriorityString = Property->GetMetaData(Tag);
		int Priority = INT_MAX;

		if (!ModelingPriorityString.IsEmpty())
		{
			TTypeFromString<int>::FromString(Priority, *ModelingPriorityString);
		}
		
		// If this is a struct, need to determine if the child properties have the provided tag
		FStructProperty* StructProperty = CastField<FStructProperty>(Property);
		if(StructProperty && StructProperty->Struct)
		{
			// exception in the case where there is a registered customization on the struct
			// - detect if that is the case and handle that customization here immediately by calling its delegate
			// - no need to further search through struct
			bool bStructHandled = false;
			if (Customizations.Contains(ActiveTool->GetClass()))
			{
				// add any widgets found for struct customizations to the main Priority:WidgetArr map
				FGetQuickPropertyCustomization CustomizationDelegate = *Customizations.Find(ActiveTool->GetClass());
				if (CustomizationDelegate.IsBound())
				{
					bStructHandled = CustomizationDelegate.Execute(StructProperty, PropListOwner, PriorityToWidgets.FindOrAdd(Priority), ActiveTool, CustomizationDisplayType);
				}
			}

			// if no tool-level customization handled the struct, check struct-type customizations
			if (!bStructHandled && StructTypeCustomizations.Contains(StructProperty->Struct))
			{
				FGetQuickPropertyCustomization StructTypeDelegate = *StructTypeCustomizations.Find(StructProperty->Struct);
				if (StructTypeDelegate.IsBound())
				{
					bStructHandled = StructTypeDelegate.Execute(StructProperty, PropListOwner, PriorityToWidgets.FindOrAdd(Priority), ActiveTool, CustomizationDisplayType);
				}
			}

			// if no customization for the struct is found
			if (!bStructHandled)
			{
				// iterate through the struct properties
				for (TFieldIterator<FProperty> StructPropIt(StructProperty->Struct); StructPropIt; ++StructPropIt)
				{
					FProperty* InsideStructProperty = *StructPropIt;
					CollectTaggedWidgetsRecursive(InsideStructProperty, PropListOwner, Tag, ActiveTool,
						PriorityToWidgets, HandlePropsFunc, Customizations, CustomizationDisplayType, StructTypeCustomizations);
				}
			}
		}
		else // if not a FStructProperty
		{
			// are there are any customization overrides registered for the current tool?
			if (Customizations.Contains(ActiveTool->GetClass()))
			{
				// if overrides are found for the current property, callback to their functions, otherwise handle the property as normal
				FGetQuickPropertyCustomization CustomizationDelegate = *Customizations.Find(ActiveTool->GetClass());
				if (CustomizationDelegate.IsBound() && 
					!CustomizationDelegate.Execute(Property, PropListOwner, PriorityToWidgets.FindOrAdd(Priority), ActiveTool, CustomizationDisplayType))
				{
					HandlePropsFunc(Property, PropListOwner, PriorityToWidgets.FindOrAdd(Priority));
				}
			}
			// no customizations registered, handle props as usual
			else
			{
				HandlePropsFunc(Property, PropListOwner, PriorityToWidgets.FindOrAdd(Priority));
			}
		}
	}
}

void UE::MeshTerrain::AddSliderWidget(FFloatProperty* FloatProperty, UObject* PropListOwner, TArray<FPropertyWidget>& WidgetsToAdd, const EQuickPropertyDisplay DisplayType)
{
	const FString PropName = FloatProperty->GetName();
	const FText ToolTipText = FloatProperty->GetToolTipText();

	// begin calculation of slider min and maxes
	const FString& MetaUIMinString = FloatProperty->GetMetaData(TEXT("UIMin"));
	const FString& MetaUIMaxString = FloatProperty->GetMetaData(TEXT("UIMax"));
	const FString& ClampMinString = FloatProperty->GetMetaData(TEXT("ClampMin"));
	const FString& ClampMaxString = FloatProperty->GetMetaData(TEXT("ClampMax"));

	const FString& UIMinString = MetaUIMinString.Len() ? MetaUIMinString : ClampMinString;
	const FString& UIMaxString = MetaUIMaxString.Len() ? MetaUIMaxString : ClampMaxString;

	float ClampMin = TNumericLimits<float>::Lowest();
	float ClampMax = TNumericLimits<float>::Max();

	if (!ClampMinString.IsEmpty())
	{
		TTypeFromString<float>::FromString(ClampMin, *ClampMinString);
	}

	if (!ClampMaxString.IsEmpty())
	{
		TTypeFromString<float>::FromString(ClampMax, *ClampMaxString);
	}

	float UIMin = TNumericLimits<float>::Lowest();
	float UIMax = TNumericLimits<float>::Max();
	TTypeFromString<float>::FromString(UIMin, *UIMinString);
	TTypeFromString<float>::FromString(UIMax, *UIMaxString);

	const float ActualUIMin = FMath::Max(UIMin, ClampMin);
	const float ActualUIMax = FMath::Min(UIMax, ClampMax);

	
	TFunction<float()> SliderValueFunc = [FloatProperty, PropListOwner]()
	{
		const float Value = FloatProperty->GetPropertyValue_InContainer(PropListOwner);
		return Value;
	};
	TFunction<void(float)> SetSliderValueFunc = [FloatProperty, PropListOwner](const float NewValue)
	{
		CastFieldChecked<const FFloatProperty>(FloatProperty)->SetPropertyValue_InContainer(PropListOwner, NewValue);
		FPropertyChangedEvent PropertyChangedEvent(FloatProperty, EPropertyChangeType::ValueSet);
		PropListOwner->PostEditChangeProperty(PropertyChangedEvent);
	};
	// end calculation of slider min and maxes

	// construct the widget representation of the float property
	SHorizontalBox::FArguments FloatPropArgs;

	// if display should include the label, add LabelWidget to FloatPropsArgs
	if (DisplayType == EQuickPropertyDisplay::WidgetAndLabel)
	{
		FloatPropArgs
		+ SHorizontalBox::Slot()
		.VAlign(VAlign_Center)
		.AutoWidth()
		.Padding(4.0f, 0.0f, 4.0f, 0.0f)
		[
			SNew(STextBlock)
			.ToolTipText(ToolTipText)
			.Font(FAppStyle::GetFontStyle( TEXT( "MenuItem.Font" ) ))
			.Justification(ETextJustify::Center)
			.Text(FloatProperty->GetDisplayNameText())
		];
	}

	// if display should include the SSpinBox widget, add it to FloatPropsArgs
	if (DisplayType == EQuickPropertyDisplay::WidgetAndLabel || DisplayType == EQuickPropertyDisplay::WidgetOnly)
	{
		FloatPropArgs
		+ SHorizontalBox::Slot()
		.AutoWidth()
		.Padding(4.0f, 0.0f, 4.0f, 0.0f)
		[
			SNew(SBox)
			.WidthOverride(50.f)
			[
				SNew(SSpinBox<float>)
				.ToolTipText(ToolTipText)
				.Font(FAppStyle::GetFontStyle( TEXT("PropertyWindow.NormalFont")) )
				.MaxFractionalDigits(2)
				.IsEnabled_Lambda([](){ return true; })
				.Delta(0.03125f) // 1/32
				.MinValue(ClampMinString.Len() ? ClampMin : TOptional<float>())
				.MinSliderValue((UIMinString.Len()) ? ActualUIMin : TOptional<float>())
				.MaxValue(ClampMaxString.Len() ? ClampMax : TOptional<float>())
				.MaxSliderValue((UIMaxString.Len()) ? ActualUIMax : TOptional<float>())
				.Value_Lambda(MoveTemp(SliderValueFunc))
				.OnValueChanged_Lambda(MoveTemp(SetSliderValueFunc))
			]
		];
	}
	
	WidgetsToAdd.Add(FPropertyWidget(FloatProperty->GetDisplayNameText(), SArgumentNew(FloatPropArgs, SHorizontalBox)));
}

void UE::MeshTerrain::AddEnumWidget(FEnumProperty* EnumProperty, UObject* PropListOwner,
	TArray<FPropertyWidget>& WidgetsToAdd, const EQuickPropertyDisplay DisplayType)
{
	// retrieve the UEnum from the property
	const UEnum* EnumFromProp = EnumProperty->GetEnum();
	const FText ToolTipText = EnumProperty->GetToolTipText();

	// retrieve all the enum options for the drop-down
	EnumOptions.Empty();
	for (int64 EnumIndex = 0; EnumIndex < EnumFromProp->GetMaxEnumValue(); ++EnumIndex)
	{
		EnumOptions.Add(MakeShareable(new FString(EnumFromProp->GetNameStringByIndex(EnumIndex))));
	}
		
	// retrieve the current value index of the enum
	uint32 CurrentEnumIndex = 0;
	EnumProperty->GetValue_InContainer(PropListOwner, &CurrentEnumIndex);

	TSharedRef<STextComboBox> EnumDropDown =
		// TODO : support bitmask enum props
		SNew(STextComboBox)
		.ToolTipText(ToolTipText)
		.OptionsSource(&EnumOptions) // drop down options for the enum
		.InitiallySelectedItem(EnumOptions[CurrentEnumIndex])
		.ContentPadding(FMargin(5,0))
		.OnSelectionChanged_Lambda([EnumProperty, EnumFromProp, PropListOwner](TSharedPtr<FString> String, ESelectInfo::Type)
		{
			const int32 NewEnumIndex = EnumFromProp->GetIndexByName(FName(*String));
			CastFieldChecked<const FEnumProperty>(EnumProperty)->SetValue_InContainer(PropListOwner, &NewEnumIndex);
			FPropertyChangedEvent PropertyChangedEvent(EnumProperty, EPropertyChangeType::ValueSet);
			PropListOwner->PostEditChangeProperty(PropertyChangedEvent);
		});

	if (UInteractiveToolPropertySet* IntToolsPropSet = Cast<UInteractiveToolPropertySet>(PropListOwner))
	{
		// ensure that when the setting is modified outside the current widget, the change is reflected here
		IntToolsPropSet->GetOnModified().AddSPLambda(&EnumDropDown.Get(), [PropListOwner, EnumProperty, WeakEnumDropDown = EnumDropDown.ToWeakPtr()](const UObject* ModifiedPropSet, FProperty* ModifiedProp)
		{
			if (const FEnumProperty* ModifiedEnumProp = CastField<FEnumProperty>(ModifiedProp))
			{
				// if the modified property is the one we are creating a widget for
				if (ModifiedPropSet == PropListOwner && ModifiedEnumProp == EnumProperty)
				{
					if (TSharedPtr<STextComboBox> EnumDropDown = WeakEnumDropDown.Pin())
					{
						uint32 EnumIndex = 0;
						// retrieves the value that has been modified
						ModifiedEnumProp->GetValue_InContainer(PropListOwner, &EnumIndex);
						// sets the current Widget's value to the modified one
						EnumDropDown->SetSelectedItem(EnumOptions[EnumIndex]);
					}
				}
			}
		});
	}
	
	const FText PropDisplayName = EnumProperty->GetDisplayNameText();
	
	// construct the widget representation of the enum property
	SHorizontalBox::FArguments EnumPropArgs;

	// if display should include the label, add LabelWidget to EnumPropArgs
	if (DisplayType == EQuickPropertyDisplay::WidgetAndLabel)
	{
		EnumPropArgs
		+ SHorizontalBox::Slot()
		.HAlign(HAlign_Center)
		.VAlign(VAlign_Center)
		.AutoWidth()
		.Padding(4.0f, 0.0f, 4.0f, 0.0f)
		[
			SNew(STextBlock)
			.ToolTipText(ToolTipText)
			.Font(FAppStyle::GetFontStyle( TEXT( "MenuItem.Font" ) ))
			.Justification(ETextJustify::Center)
			.Text(PropDisplayName) // enum label
		];
	}

	// if display should include the drop-down widget, add it to EnumPropArgs
	if (DisplayType == EQuickPropertyDisplay::WidgetAndLabel || DisplayType == EQuickPropertyDisplay::WidgetOnly)
	{
		EnumPropArgs
		+SHorizontalBox::Slot()
		.HAlign(HAlign_Fill)
		.AutoWidth()
		.Padding(4.0f, 0.0f, 4.0f, 0.0f)
		[
			EnumDropDown
		];
	}
	
	WidgetsToAdd.Add(FPropertyWidget(EnumProperty->GetDisplayNameText(), SNew(SHorizontalBox)
		+ SHorizontalBox::Slot()
		.HAlign(HAlign_Fill)
		[
			SArgumentNew(EnumPropArgs, SHorizontalBox)
		]));
}

void UE::MeshTerrain::AddBoolWidget(FBoolProperty* BoolProperty, UObject* PropListOwner, TArray<FPropertyWidget>& WidgetsToAdd,
	TMap<FName, TAttribute<const FSlateBrush*>> BoolCustomizations, const EQuickPropertyDisplay DisplayType, const EBoolPropertyDisplay BoolDisplayType)
{
	const FText PropToolTip = BoolProperty->GetToolTipText();
	const FText PropDisplayName = BoolProperty->GetDisplayNameText();
	
	TFunction<ECheckBoxState()> BoolValueFunc = [BoolProperty, PropListOwner]()
	{
		const bool Value = BoolProperty->GetPropertyValue_InContainer(PropListOwner);
		const ECheckBoxState CheckBoxState = Value ? ECheckBoxState::Checked : ECheckBoxState::Unchecked;
		return CheckBoxState;
	};

	const TFunction<TSharedRef<SWidget>()> CheckBoxContentFunc = [&BoolProperty, &PropDisplayName, &BoolCustomizations]()
	{
		const FName BoolPropName = BoolProperty->GetFName();

		// if there is a registered override for the bool prop, use the provided brush as the checkbox content
		if (BoolCustomizations.Contains(BoolPropName))
		{
			return
				SNew(SHorizontalBox)
				+ SHorizontalBox::Slot()
				.VAlign(VAlign_Center)
				.Padding(4.0f, 0.0f, 4.0f, 0.0f)
				.AutoWidth()
				[
					SNew(SImage)
					.Image(BoolCustomizations.Find(BoolPropName)->Get())
				];
		}
		// otherwise, use the property name as text for the checkbox content
		return
			SNew(SHorizontalBox)
			+ SHorizontalBox::Slot()
			.VAlign(VAlign_Center)
			.Padding(4.0f, 0.0f, 4.0f, 0.0f)
			[
				SNew(STextBlock)
				.Justification(ETextJustify::Center)
				.TextStyle(FAppStyle::Get(), "DetailsView.CategoryTextStyle")
				.Text( PropDisplayName )
			];
	};

	// bool property label
	const TSharedRef<STextBlock> TextWidget =
		SNew(STextBlock)
		.Font(FAppStyle::GetFontStyle( TEXT( "MenuItem.Font" ) ))
		.Justification(ETextJustify::Center)
		.Text(PropDisplayName);


	const TSharedRef<SBox> ButtonWidget = BoolDisplayType ==
		EBoolPropertyDisplay::Button ?
		// button containing either the property label or (when customized) icon representation
		SNew(SBox)
		[
			SNew(SCheckBox)
			.Style(FAppStyle::Get(), "DetailsView.SectionButton")
			.Padding(FMargin(4, 2))
			.ToolTipText(PropToolTip)
			.HAlign(HAlign_Center)
			.OnCheckStateChanged_Lambda([BoolProperty, PropListOwner](const ECheckBoxState InState)
			{
				const bool BoolVal = (InState == ECheckBoxState::Checked);
				CastFieldChecked<const FBoolProperty>(BoolProperty)->SetPropertyValue_InContainer(PropListOwner, BoolVal);
				FPropertyChangedEvent PropertyChangedEvent(BoolProperty, EPropertyChangeType::ValueSet);
				PropListOwner->PostEditChangeProperty(PropertyChangedEvent);
			})
			.IsChecked_Lambda(MoveTemp(BoolValueFunc))
			[
				CheckBoxContentFunc()
			]
		] :
		// checkbox only
		SNew(SBox)
		[
			SNew(SCheckBox)
			.ToolTipText(PropToolTip)
			.HAlign(HAlign_Center)
			.OnCheckStateChanged_Lambda([BoolProperty, PropListOwner](const ECheckBoxState InState)
			{
				const bool BoolVal = (InState == ECheckBoxState::Checked);
				CastFieldChecked<const FBoolProperty>(BoolProperty)->SetPropertyValue_InContainer(PropListOwner, BoolVal);
				FPropertyChangedEvent PropertyChangedEvent(BoolProperty, EPropertyChangeType::ValueSet);
				PropListOwner->PostEditChangeProperty(PropertyChangedEvent);
			})
			.IsChecked_Lambda(MoveTemp(BoolValueFunc))
		];
	
	// construct the widget representation of the bool property
	SHorizontalBox::FArguments BoolPropArgs;
	
	// if display should include the label, add LabelWidget to BoolPropArgs
	if (DisplayType == EQuickPropertyDisplay::WidgetAndLabel)
	{
		BoolPropArgs
		+ SHorizontalBox::Slot()
		.Padding(4.0f, 0.0f, 4.0f, 0.0f)
		.AutoWidth()
		.VAlign(VAlign_Center)
		[
			TextWidget
		];
	}
	
	// if display should include the button/checkbox widget, add it to BoolPropArgs
	if (DisplayType == EQuickPropertyDisplay::WidgetAndLabel || DisplayType == EQuickPropertyDisplay::WidgetOnly)
	{
		BoolPropArgs
		+ SHorizontalBox::Slot()
		.AutoWidth()
		.Padding(4.0f, 0.0f, 4.0f, 0.0f)
		[
			ButtonWidget
		];
	}

	WidgetsToAdd.Add(FPropertyWidget(BoolProperty->GetDisplayNameText(), SArgumentNew(BoolPropArgs, SHorizontalBox)));
}

TSharedRef<SWidget> UE::MeshTerrain::CreateQuickEditPropertyRow(const FPropertyWidget& PropertyWidget)
{
	return
		SNew(SHorizontalBox)
		+SHorizontalBox::Slot()
		.AutoWidth()
		[
			SNew(SBox)
			.Padding(5.f, 0.f, 0.f, 0.f)
			.HAlign(HAlign_Fill)
			.WidthOverride(100.f)
			[
				SNew(STextBlock)
				.Font(FAppStyle::GetFontStyle( TEXT( "MenuItem.Font" ) ))
				.Justification(ETextJustify::Left)
				.OverflowPolicy(ETextOverflowPolicy::Ellipsis)
				.Text(PropertyWidget.PropertyName)
			]
		]
		+SHorizontalBox::Slot()
		.AutoWidth()
		[
			SNew(SSeparator)
			.Orientation(Orient_Vertical)
			.Thickness(1.f)
		]
		+SHorizontalBox::Slot()
		.AutoWidth()
		[
			SNew(SBox)
			.Padding(5.f, 0.f, 5.f, 0.f)
			.HAlign(HAlign_Fill)
			[
				PropertyWidget.WidgetRepresentation
			]
		];
}

bool UE::MeshTerrain::CreateFalloffWidget(FProperty* FalloffProperty, UObject* PropListOwner, TArray<FPropertyWidget>& WidgetsToAdd,
	UInteractiveTool* Tool, const EQuickPropertyDisplay DisplayType)
{
	UMeshVertexSculptTool* VSculptTool = Cast<UMeshVertexSculptTool>(Tool);
	if (!VSculptTool)
	{
		return false;
	}
	const TArray<UMeshSculptToolBase::FFalloffTypeInfo>& FalloffTypeInfos = VSculptTool->GetRegisteredPrimaryFalloffTypes();

	const TSharedRef<SWidget> ComboButton = SNew(SComboButton)
		.ToolTipText(LOCTEXT("FalloffToolTipText", "Primary Brush Falloff Type, multiplied by Alpha Mask where applicable"))
		.MenuPlacement(EMenuPlacement::MenuPlacement_CenteredBelowAnchor)
		.ComboButtonStyle(FMeshTerrainModeStyle::Get().Get(), "QuickProperties.SmallMarginComboButton")
		.OnGetMenuContent_Lambda([&FalloffTypeInfos, VSculptTool, FalloffProperty, PropListOwner]()
		{
			// build menu of all falloff types to choose from
			FMenuBuilder MenuBuilder(true, nullptr);

			// for each falloff type
			for ( UMeshSculptToolBase::FFalloffTypeInfo Falloff : FalloffTypeInfos)
			{
				const FString SourceBrushName = Falloff.Name.ToString();
				MenuBuilder.AddMenuEntry(
					Falloff.Name,
					Falloff.Name,
					FSlateIcon(FMeshTerrainModeStyle::Get()->GetStyleSetName(), FName(TEXT("FalloffQuickSettings.") + (SourceBrushName) )),
					FUIAction(
						FExecuteAction::CreateLambda([VSculptTool, Falloff, FalloffProperty, PropListOwner]
						{
							VSculptTool->SetActiveFalloffType(Falloff.Identifier);
							// ensure normal details panel is updated
							VSculptTool->OnDetailsPanelRequestRebuild.Broadcast();
						})
					),
					NAME_None,
					EUserInterfaceActionType::CollapsedButton
				); 
			}
			return MenuBuilder.MakeWidget();
		})
		.ButtonContent()
		[
			SNew(SHorizontalBox)
			+ SHorizontalBox::Slot()
			.AutoWidth()
			[
				SNew(SBox)
				.WidthOverride(16.f)
				.HeightOverride(16.f)
				[
					SNew(SImage)
					.Image_Lambda([&FalloffTypeInfos, VSculptTool]()
					{
						// combo button should display icon of current falloff type
						const EMeshSculptFalloffType CurrentFalloffType = VSculptTool->SculptProperties->PrimaryFalloffType;

						const FSlateBrush* CurFalloffBrush = nullptr;
						for ( UMeshSculptToolBase::FFalloffTypeInfo Falloff : FalloffTypeInfos)
						{
							if (static_cast<EMeshSculptFalloffType>(Falloff.Identifier) == CurrentFalloffType)
							{
								const FString SourceBrushName = Falloff.Name.ToString();
								CurFalloffBrush = FMeshTerrainModeStyle::Get()->GetBrush( FName(TEXT("FalloffQuickSettings.") + (SourceBrushName) ));
							}
						}
						return CurFalloffBrush;
					})
				]
			]
		];

	// construct the widget representation of the falloff property
	SHorizontalBox::FArguments FalloffPropArgs;
	
	// if display should include the falloff widget, add it to FalloffPropArgs
	if (DisplayType == EQuickPropertyDisplay::WidgetAndLabel || DisplayType == EQuickPropertyDisplay::WidgetOnly)
	{
		FalloffPropArgs
		// drop drown
		+ SHorizontalBox::Slot()
		.Padding(4.0f, 0.0f, 4.0f, 0.0f)
		.AutoWidth()
		[
			SNew(SBox)
			[
				ComboButton
			]
		];
	}

	WidgetsToAdd.Add(FPropertyWidget(FalloffProperty->GetDisplayNameText(), SArgumentNew(FalloffPropArgs, SHorizontalBox)));
	return true;
}

bool UE::MeshTerrain::CreateAlphaWidget(FProperty* AlphaProperty, UObject* PropListOwner, TArray<FPropertyWidget>& WidgetsToAdd,
	UInteractiveTool* Tool, const EQuickPropertyDisplay DisplayType)
{
	UMeshVertexSculptTool* VSculptTool = Cast<UMeshVertexSculptTool>(Tool);
	if (!VSculptTool)
	{
		return false;
	}
	
	// retrieve all alphas
	UMeshTerrainModeCustomizationSettings* UISettings = GetMutableDefault<UMeshTerrainModeCustomizationSettings>();
	TArray<SToolInputAssetComboPanel::FNamedCollectionList> BrushAlphasLists;
	for (const FMeshTerrainModeAssetCollectionSet& AlphasCollectionSet : UISettings->BrushAlphaSets)
	{
		SToolInputAssetComboPanel::FNamedCollectionList CollectionSet;
		CollectionSet.Name = AlphasCollectionSet.Name;
		for (FCollectionReference CollectionRef : AlphasCollectionSet.Collections)
		{
			CollectionSet.Collections.Emplace(
				FCollectionManagerModule::GetModule().Get().GetProjectCollectionContainer(),
				CollectionRef.CollectionName,
				ECollectionShareType::CST_Local);
		}
		BrushAlphasLists.Add(CollectionSet);
	}
		
	TSharedPtr<SToolInputAssetComboPanel> AlphaAssetPicker =
		SNew(SToolInputAssetComboPanel)
		.AssetClassType(UTexture2D::StaticClass())
		.OnSelectionChanged_Lambda([VSculptTool](const FAssetData& InAssetData)
		{
			// set new Alpha
			UObject* TextureObj = InAssetData.GetAsset();
			UTexture2D* Texture = Cast<UTexture2D>(TextureObj);
			VSculptTool->AlphaProperties->Alpha = Texture;
			VSculptTool->UpdateBrushAlpha(Texture);

			// ensure normal details panel is updated
			VSculptTool->OnDetailsPanelRequestRebuild.Broadcast();
		})
		.ComboButtonTileSize(FVector2D(16, 100))
		.FlyoutTileSize(FVector2D(80, 80))
		.FlyoutSize(FVector2D(1000, 600))
		.CollectionSets(BrushAlphasLists)
		.ThumbnailDisplayMode(EThumbnailDisplayMode::AssetName); // display text only in Quick Settings widget

	// ensure that when details panel is updated, the new alpha is also reflected here in the Quick settings Widget
	VSculptTool->AlphaProperties->GetOnModified().AddSPLambda(AlphaAssetPicker.Get(), [WeakAlphaAssetPicker = AlphaAssetPicker.ToWeakPtr()](const UObject* ModifiedPropSet, const FProperty* ModifiedProp)
	{
		if (ModifiedProp->GetFName() == FName("Alpha"))
		{
			if (TSharedPtr<SToolInputAssetComboPanel> AlphaAssetPicker = WeakAlphaAssetPicker.Pin())
			{
				// retrieve new alpha
				UObject* Texture;
				ModifiedProp->GetValue_InContainer(ModifiedPropSet, &Texture);
				// refresh combo panel button in QS Widget
				FAssetData Data = FAssetData(Texture);
				AlphaAssetPicker->RefreshThumbnail(Data);
			}
		}
	});

	// construct the widget representation of the alpha property
	SHorizontalBox::FArguments AlphaPropArgs;

	// if display should include the label, add LabelWidget to AlphaPropArgs
	if (DisplayType == EQuickPropertyDisplay::WidgetAndLabel)
	{
		AlphaPropArgs
		+ SHorizontalBox::Slot()
		.VAlign(VAlign_Center)
		.AutoWidth()
		.Padding(4.0f, 0.0f, 4.0f, 0.0f)
		[
			// add alpha label
			SNew(STextBlock)
			.Font(FAppStyle::GetFontStyle( TEXT( "MenuItem.Font" ) ))
			.Justification(ETextJustify::Center)
			.Text(LOCTEXT("AlphaLabel", "Alpha"))
		];
	}

	// if display should include the Alpha widget, add it to AlphaPropArgs
	if (DisplayType == EQuickPropertyDisplay::WidgetAndLabel || DisplayType == EQuickPropertyDisplay::WidgetOnly)
	{
		AlphaPropArgs
		+ SHorizontalBox::Slot()
		.Padding(4.0f, 0.0f, 4.0f, 0.0f)
		.AutoWidth()
		[
			SNew(SBox)
			[
				AlphaAssetPicker->AsShared()
			]
		];
	}

	WidgetsToAdd.Add(FPropertyWidget(AlphaProperty->GetDisplayNameText(), SArgumentNew(AlphaPropArgs, SHorizontalBox)));
	return true;
}

bool UE::MeshTerrain::CreateBrushSizeToggleWidget(FProperty* BrushSizeProperty, UObject* PropListOwner,
	TArray<FPropertyWidget>& WidgetsToAdd, UInteractiveTool* Tool, const EQuickPropertyDisplay DisplayType,
	FIsActionButtonVisible PressureButtonVisible)
{
	// retrieves the current BrushSize property (represented as FBrushToolRadius
	TFunction<FBrushToolRadius()> GetCurBrushSizeProp = [BrushSizeProperty, PropListOwner]()
	{
		FBrushToolRadius PropValue;
		BrushSizeProperty->GetValue_InContainer(PropListOwner, &PropValue);
		return PropValue;
	};

	// retrieves the current size type (adaptive or world) of the BrushSize property
	TFunction<EBrushToolSizeType()> GetCurrentSizeType = [GetCurBrushSizeProp]()
	{
		const FBrushToolRadius PropValue = GetCurBrushSizeProp();
		return (PropValue.SizeType == EBrushToolSizeType::World) ? EBrushToolSizeType::World : EBrushToolSizeType::Adaptive;
	};

	// toggles the size type of the BrushSize property between Adaptive and World
	TFunction<void(bool)> ToggleCurrentSizeType = [BrushSizeProperty, PropListOwner, GetCurBrushSizeProp] (bool bIsChecked)
	{
		const EBrushToolSizeType NewType = bIsChecked ? EBrushToolSizeType::World : EBrushToolSizeType::Adaptive;
		FBrushToolRadius PropValue = GetCurBrushSizeProp();
		PropValue.SizeType = NewType;
		BrushSizeProperty->SetValue_InContainer(PropListOwner, &PropValue);
		FPropertyChangedEvent PropertyChangedEvent(BrushSizeProperty, EPropertyChangeType::ValueSet);
		PropListOwner->PostEditChangeProperty(PropertyChangedEvent);
	};

	// is pressure sensitivity currently enabled?
	TFunction<bool()> GetEnablePressureSensitivity = [GetCurBrushSizeProp]()
	{
		const FBrushToolRadius PropValue = GetCurBrushSizeProp();
		return PropValue.bEnablePressureSensitivity;
	};
	
	// toggles the pressure sensitivity of the BrushSize property
	TFunction<void(bool)> TogglePressureSensitivity = [BrushSizeProperty, PropListOwner, GetCurBrushSizeProp] (bool bIsChecked)
	{
		FBrushToolRadius PropValue = GetCurBrushSizeProp();
		PropValue.bEnablePressureSensitivity = bIsChecked;
		BrushSizeProperty->SetValue_InContainer(PropListOwner, &PropValue);
		FPropertyChangedEvent PropertyChangedEvent(BrushSizeProperty, EPropertyChangeType::ValueSet);
		PropListOwner->PostEditChangeProperty(PropertyChangedEvent);
	};

	// retrieves the current size value of the BrushSize property
	const TFunction<float()> SliderValueFunc = [GetCurrentSizeType, GetCurBrushSizeProp]()
	{
		const FBrushToolRadius PropValue = GetCurBrushSizeProp();
		float Value = -1;
		if (GetCurrentSizeType() == EBrushToolSizeType::Adaptive)
		{
			Value = PropValue.AdaptiveSize;
		}
		else if (GetCurrentSizeType() == EBrushToolSizeType::World)
		{
			Value = PropValue.WorldRadius;
		}
		return Value;
	};

	// sets a new size value for the Brush Size property
	const TFunction<void(float)> SetSliderValueFunc =
		[GetCurrentSizeType, BrushSizeProperty, PropListOwner, GetCurBrushSizeProp](const float NewValue)
	{
		FBrushToolRadius PropValue = GetCurBrushSizeProp();
		
		if (GetCurrentSizeType() == EBrushToolSizeType::Adaptive)
		{
			PropValue.AdaptiveSize = NewValue;
		}
		else if (GetCurrentSizeType() == EBrushToolSizeType::World)
		{
			PropValue.WorldRadius = NewValue;
		}
		BrushSizeProperty->SetValue_InContainer(PropListOwner, &PropValue);
		FPropertyChangedEvent PropertyChangedEvent(BrushSizeProperty, EPropertyChangeType::ValueSet);
		PropListOwner->PostEditChangeProperty(PropertyChangedEvent);
	};

	// functions to support the slider - delegate to GetSculptBrushSizeRange so the override values
	// stay in one place and the Details-panel customization sees the same ranges.
	TFunction<TInterval<float>()> GetUIRange = [GetCurrentSizeType, Tool]()
	{
		TInterval<float> UI, Value;
		GetSculptBrushSizeRange(Tool, GetCurrentSizeType(), UI, Value);
		return UI;
	};
	TFunction<TInterval<float>()> GetValueRange = [GetCurrentSizeType, Tool]()
	{
		TInterval<float> UI, Value;
		GetSculptBrushSizeRange(Tool, GetCurrentSizeType(), UI, Value);
		return Value;
	};
	TFunction<float()> GetActualMin = [GetValueRange]() { return GetValueRange().Min; };
	TFunction<float()> GetActualMax = [GetValueRange]() { return GetValueRange().Max; };
	TFunction<float()> GetUIMin     = [GetUIRange]()    { return GetUIRange().Min; };
	TFunction<float()> GetUIMax     = [GetUIRange]()    { return GetUIRange().Max; };

	// construct the widget representation of the brush size property
	SHorizontalBox::FArguments BrushSizePropArgs;

	// if display should include the label, add LabelWidget to BrushSizePropArgs
	if (DisplayType == EQuickPropertyDisplay::WidgetAndLabel)
	{
		BrushSizePropArgs
		+ SHorizontalBox::Slot()
		.VAlign(VAlign_Center)
		.AutoWidth()
		.Padding(8.0f, 0.0f, 0.0f, 0.0f)
		[
			// add size label
			SNew(STextBlock)
			.Font(FAppStyle::GetFontStyle( TEXT( "MenuItem.Font" ) ))
			.Justification(ETextJustify::Center)
			.Text(LOCTEXT("SizeLabel", "Size"))
		];
	}

	// if display should include the SSpinBox widget, add it to BrushSizePropArgs
	if (DisplayType == EQuickPropertyDisplay::WidgetAndLabel || DisplayType == EQuickPropertyDisplay::WidgetOnly)
	{
		// slider
		BrushSizePropArgs
		+ SHorizontalBox::Slot()
		.AutoWidth()
		.Padding(4.0f, 0.0f, 0.0f, 0.0f)
		[
			SNew(SBox)
			.WidthOverride(50.f)
			[
				SNew(SSpinBox<float>)
				.Font(FAppStyle::GetFontStyle( TEXT("PropertyWindow.NormalFont")) )
				.MaxFractionalDigits(2)
				.IsEnabled_Lambda([](){ return true; })
				.Delta(0.03125f) // 1/32
				.MinValue_Lambda([GetActualMin]() { return GetActualMin(); })
				.MinSliderValue_Lambda([GetUIMin]() { return GetUIMin(); })
				.MaxValue_Lambda([GetActualMax]() { return GetActualMax(); })
				.MaxSliderValue_Lambda([GetUIMax]() { return GetUIMax(); })
				.Value_Lambda([SliderValueFunc](){ return SliderValueFunc();})
				.OnValueChanged_Lambda([SetSliderValueFunc](float NewValue){ SetSliderValueFunc(NewValue);})
			]
		]

		// adaptive/world toggle
		+ SHorizontalBox::Slot()
		.AutoWidth()
		.HAlign(HAlign_Left)
		.Padding(8.0f, 0.0f, 8.0f, 0.0f)
		[
			SNew(SBox)
			[
				SNew(SCheckBox)
				.Style(FMeshTerrainModeStyle::Get().Get(), "QuickSettingsToggleButton")
				.HAlign(HAlign_Center)
				.Padding(0.f)
				.ToolTipText( LOCTEXT("WorldToggleTooltip", "Specify Brush Size in World Units") )
				.OnCheckStateChanged_Lambda([ToggleCurrentSizeType](ECheckBoxState State) {
					ToggleCurrentSizeType( State == ECheckBoxState::Checked );
				})
				.IsChecked_Lambda([GetCurrentSizeType]() {
					return (GetCurrentSizeType() == (EBrushToolSizeType::World)) ? ECheckBoxState::Checked : ECheckBoxState::Unchecked;
				})
				[
					SNew(SHorizontalBox)
					+ SHorizontalBox::Slot()
					.VAlign(VAlign_Center)
					.Padding(4.0f)
					.AutoWidth()
					[
						SNew(SImage)
						.Image(FMeshTerrainModeStyle::Get()->GetBrush("QuickSettings.RelativeCoordinateSystem_World"))
						.ColorAndOpacity(FSlateColor::UseForeground())
					]
				]
			]
		];
		if (GetCurBrushSizeProp().bToolSupportsPressureSensitivity)
		{
			BrushSizePropArgs
			+ SHorizontalBox::Slot()
			.AutoWidth()
			.HAlign(HAlign_Left)
			.Padding(0.0f, 0.0f, 8.0f, 0.0f)
			[
				SNew(SBox)
				[
					SNew(SCheckBox)
					.Style(FMeshTerrainModeStyle::Get().Get(), "QuickSettingsToggleButton")
					.HAlign(HAlign_Center)
					.Padding(0.f)
					.ToolTipText( LOCTEXT("PressureSensitivityToggleTooltip", "Toggle Pressure Sensitivity") )
					.Visibility_Lambda([PressureButtonVisible]()
						{
							if (PressureButtonVisible.IsBound())
							{
								return PressureButtonVisible.Execute() ? EVisibility::Visible : EVisibility::Collapsed;
							}
							return EVisibility::Collapsed; 
						})
					.OnCheckStateChanged_Lambda([TogglePressureSensitivity](ECheckBoxState State) {
						TogglePressureSensitivity( State == ECheckBoxState::Checked );
					})
					.IsChecked_Lambda([GetEnablePressureSensitivity]() {
						return GetEnablePressureSensitivity() ? ECheckBoxState::Checked : ECheckBoxState::Unchecked;
					})
					[
						SNew(SHorizontalBox)
						+ SHorizontalBox::Slot()
						.VAlign(VAlign_Center)
						.Padding(4.0f)
						.AutoWidth()
						[
							SNew(SImage)
							.Image(FMeshTerrainModeStyle::Get()->GetBrush("BrushIcons.PressureSensitivity"))
							.ColorAndOpacity(FSlateColor::UseForeground())
						]
					]
				]
			];
		}
		
		BrushSizePropArgs
		+ SHorizontalBox::Slot()
		.AutoWidth()
		.Padding(4.f, 0.f, 4.f, 0.f)
		[
			SNew(SSeparator)
			.Orientation(Orient_Vertical)
			.Thickness(1.f)
			.ColorAndOpacity(FStyleColors::Hover)
			.SeparatorImage(FAppStyle::GetBrush( "ThinLine.Horizontal" ))
		];
	}
	
	WidgetsToAdd.Add(FPropertyWidget(BrushSizeProperty->GetDisplayNameText(), SArgumentNew(BrushSizePropArgs, SHorizontalBox)));
	
	return true;
}

bool UE::MeshTerrain::CreateBrushSizeWidget(FProperty* BrushSizeProperty, UObject* PropListOwner,
	TArray<FPropertyWidget>& WidgetsToAdd, UInteractiveTool* Tool, const EQuickPropertyDisplay DisplayType,
	FIsActionButtonVisible PressureButtonVisible)
{
	UBaseBrushTool* BrushTool = Cast<UBaseBrushTool>(Tool);
	if (!BrushTool || !BrushTool->BrushProperties->bToolSupportsPressureSensitivity)
	{
		return false;
	}

	// retrieves the current BrushSize
	TFunction<float()> SliderValueFunc = [BrushSizeProperty, PropListOwner]()
	{
		float PropValue;
		BrushSizeProperty->GetValue_InContainer(PropListOwner, &PropValue);
		return PropValue;
	};

	// sets a new size value for the BrushSize property
	const TFunction<void(float)> SetSliderValueFunc = [BrushSizeProperty, PropListOwner](const float NewValue)
	{
		BrushSizeProperty->SetValue_InContainer(PropListOwner, &NewValue);
		FPropertyChangedEvent PropertyChangedEvent(BrushSizeProperty, EPropertyChangeType::ValueSet);
		PropListOwner->PostEditChangeProperty(PropertyChangedEvent);
	};

	// is pressure sensitivity currently enabled?
	TFunction<bool()> GetEnablePressureSensitivity = [BrushTool]()
	{
		return BrushTool->BrushProperties->bEnablePressureSensitivity;
	};
	
	// toggles the pressure sensitivity of the BrushSize property
	TFunction<void(bool)> TogglePressureSensitivity = [BrushTool, BrushSizeProperty, PropListOwner] (bool bIsChecked)
	{
		BrushTool->BrushProperties->bEnablePressureSensitivity = bIsChecked;
		FPropertyChangedEvent PropertyChangedEvent(BrushSizeProperty, EPropertyChangeType::ValueSet);
		PropListOwner->PostEditChangeProperty(PropertyChangedEvent);
	};
		
	// begin calculation of slider min and maxes
	const FString& MetaUIMinString = BrushSizeProperty->GetMetaData(TEXT("UIMin"));
	const FString& MetaUIMaxString = BrushSizeProperty->GetMetaData(TEXT("UIMax"));
	const FString& ClampMinString = BrushSizeProperty->GetMetaData(TEXT("ClampMin"));
	const FString& ClampMaxString = BrushSizeProperty->GetMetaData(TEXT("ClampMax"));

	const FString& UIMinString = MetaUIMinString.Len() ? MetaUIMinString : ClampMinString;
	const FString& UIMaxString = MetaUIMaxString.Len() ? MetaUIMaxString : ClampMaxString;

	float ClampMin = TNumericLimits<float>::Lowest();
	float ClampMax = TNumericLimits<float>::Max();

	if (!ClampMinString.IsEmpty())
	{
		TTypeFromString<float>::FromString(ClampMin, *ClampMinString);
	}

	if (!ClampMaxString.IsEmpty())
	{
		TTypeFromString<float>::FromString(ClampMax, *ClampMaxString);
	}

	float UIMin = TNumericLimits<float>::Lowest();
	float UIMax = TNumericLimits<float>::Max();
	TTypeFromString<float>::FromString(UIMin, *UIMinString);
	TTypeFromString<float>::FromString(UIMax, *UIMaxString);

	const float ActualUIMin = FMath::Max(UIMin, ClampMin);
	const float ActualUIMax = FMath::Min(UIMax, ClampMax);
	
	const FText ToolTipText = BrushSizeProperty->GetToolTipText();

	// construct the widget representation of the BrushSize property
	SHorizontalBox::FArguments BrushSizePropArgs;

	// if display should include the label, add LabelWidget to BrushSizePropArgs
	if (DisplayType == EQuickPropertyDisplay::WidgetAndLabel)
	{
		BrushSizePropArgs
		+ SHorizontalBox::Slot()
		.VAlign(VAlign_Center)
		.AutoWidth()
		.Padding(8.0f, 0.0f, 0.0f, 0.0f)
		[
			// add BrushSize label
			SNew(STextBlock)
			.ToolTipText(ToolTipText)
			.Font(FAppStyle::GetFontStyle( TEXT( "MenuItem.Font" ) ))
			.Justification(ETextJustify::Center)
			.Text(LOCTEXT("SizeLabel", "Size"))
		];
	}
		
	// if display should include the SSpinBox widget, add it to BrushSizePropArgs
	if (DisplayType == EQuickPropertyDisplay::WidgetAndLabel || DisplayType == EQuickPropertyDisplay::WidgetOnly)
	{
		// slider
		BrushSizePropArgs
		+ SHorizontalBox::Slot()
		.AutoWidth()
		.Padding(4.0f, 0.0f, 8.0f, 0.0f)
		[
			SNew(SBox)
			.WidthOverride(50.f)
			[
				SNew(SSpinBox<float>)
				.ToolTipText(ToolTipText)
				.Font(FAppStyle::GetFontStyle( TEXT("PropertyWindow.NormalFont")) )
				.MaxFractionalDigits(2)
				.IsEnabled_Lambda([](){ return true; })
				.Delta(0.03125f) // 1/32
				.MinValue(ClampMinString.Len() ? ClampMin : TOptional<float>())
				.MinSliderValue((UIMinString.Len()) ? ActualUIMin : TOptional<float>())
				.MaxValue(ClampMaxString.Len() ? ClampMax : TOptional<float>())
				.MaxSliderValue((UIMaxString.Len()) ? ActualUIMax : TOptional<float>())
				.Value_Lambda([SliderValueFunc](){ return SliderValueFunc();})
				.OnValueChanged_Lambda([SetSliderValueFunc](float NewValue){ SetSliderValueFunc(NewValue);})
			]
		];
			
		BrushSizePropArgs
		+ SHorizontalBox::Slot()
		.AutoWidth()
		.HAlign(HAlign_Left)
		.Padding(0.0f, 0.0f, 8.0f, 0.0f)
		[
			SNew(SBox)
			[
				SNew(SCheckBox)
				.Style(FMeshTerrainModeStyle::Get().Get(), "QuickSettingsToggleButton")
				.HAlign(HAlign_Center)
				.Padding(0.f)
				.ToolTipText( LOCTEXT("PressureSensitivityToggleTooltip", "Toggle Pressure Sensitivity") )
				.Visibility_Lambda([PressureButtonVisible]()
					{
						if (PressureButtonVisible.IsBound())
						{
							return PressureButtonVisible.Execute() ? EVisibility::Visible : EVisibility::Collapsed;
						}
						return EVisibility::Collapsed; 
					})
				.OnCheckStateChanged_Lambda([TogglePressureSensitivity](ECheckBoxState State) {
					TogglePressureSensitivity( State == ECheckBoxState::Checked );
				})
				.IsChecked_Lambda([GetEnablePressureSensitivity]() {
					return GetEnablePressureSensitivity() ? ECheckBoxState::Checked : ECheckBoxState::Unchecked;
				})
				[
					SNew(SHorizontalBox)
					+ SHorizontalBox::Slot()
					.VAlign(VAlign_Center)
					.Padding(4.0f)
					.AutoWidth()
					[
						SNew(SImage)
						.Image(FMeshTerrainModeStyle::Get()->GetBrush("BrushIcons.PressureSensitivity"))
						.ColorAndOpacity(FSlateColor::UseForeground())
					]
				]
			]
		]
		+ SHorizontalBox::Slot()
		.AutoWidth()
		.Padding(4.f, 0.f, 8.f, 0.f)
		[
			SNew(SSeparator)
			.Orientation(Orient_Vertical)
			.Thickness(1.f)
			.ColorAndOpacity(FStyleColors::Hover)
			.SeparatorImage(FAppStyle::GetBrush( "ThinLine.Horizontal" ))
		];
	}
	
	TSharedRef<SWidget> WholeWidget =
		SNew(SBorder)
		.BorderImage(FAppStyle::GetBrush("NoBorder"))
		.Padding(0.0f)
		.Visibility_Lambda([BrushTool]()
		{
			// if we are not using BrushSize, don't display any widget
			return !BrushTool->BrushProperties->bSpecifyRadius ? EVisibility::Visible : EVisibility::Collapsed;
		})
		[
			SArgumentNew(BrushSizePropArgs, SHorizontalBox)
		];
	
	WidgetsToAdd.Add(FPropertyWidget(BrushSizeProperty->GetDisplayNameText(), WholeWidget));
	return true;
}


bool UE::MeshTerrain::GetBrushRadiusRangeOverride(
	const UInteractiveTool* Tool,
	FString& OutUIMin, FString& OutUIMax,
	FString& OutClampMin, FString& OutClampMax)
{
	// MeshPartition channel painting deals with brushes that can be much larger than the default
	// 1.0..1000.0 slider range allows. Widen the slider here. Leave clamps untouched so the
	// typed-input bounds inherited from UBrushBaseProperties::BrushRadius (ClampMin=0.1,
	// ClampMax=50000.0) stand.
	if (Cast<const UE::MeshPartition::UAttributePaintTool>(Tool))
	{
		OutUIMin = TEXT("1.0");
		OutUIMax = TEXT("10000.0");
		return true;
	}
	return false;
}

void UE::MeshTerrain::GetSculptBrushSizeRange(
	const UInteractiveTool* Tool,
	EBrushToolSizeType SizeType,
	TInterval<float>& OutUIRange,
	TInterval<float>& OutValueRange)
{
	// Defaults match the values previously hardcoded in the two customizations
	// (FModelingToolsBrushSizeCustomization::CustomizeHeader and CreateBrushSizeToggleWidget).
	if (SizeType == EBrushToolSizeType::Adaptive)
	{
		OutUIRange    = TInterval<float>(0.0f, 1.0f);
		OutValueRange = TInterval<float>(0.0f, 10.0f);
	}
	else // World
	{
		OutUIRange    = TInterval<float>(1.0f, 1000.0f);
		OutValueRange = TInterval<float>(0.01f, 50000.0f);
	}

	// MeshPartition height-sculpt brushes paint terrain at scales much larger than the default
	// 1000 cm slider ceiling. Widen the World UI range so users can dial up to 10000 cm without
	// having to type. The value range stays at 50000 (already permissive enough for typed input).
	if (SizeType == EBrushToolSizeType::World &&
		Cast<const UE::MeshPartition::UHeightSculptTool>(Tool) != nullptr)
	{
		OutUIRange = TInterval<float>(1.0f, 10000.0f);
	}
}

bool UE::MeshTerrain::CreateBrushRadiusWidget(FProperty* BrushRadiusProperty, UObject* PropListOwner,
	TArray<FPropertyWidget>& WidgetsToAdd, UInteractiveTool* Tool, const EQuickPropertyDisplay DisplayType)
{
	// this function is almost identical to CreateBrushSize (no pressure sensitivity, and they represent diff properties)
	// TODO: factor out common elements to reduce code duplication
	UBaseBrushTool* BrushTool = Cast<UBaseBrushTool>(Tool);
	if (!BrushTool)
	{
		return false;
	}

	// retrieves the current BrushRadius
	TFunction<float()> SliderValueFunc = [BrushRadiusProperty, PropListOwner]()
	{
		float PropValue;
		BrushRadiusProperty->GetValue_InContainer(PropListOwner, &PropValue);
		return PropValue;
	};

	// sets a new size value for the BrushRadius property
	const TFunction<void(float)> SetSliderValueFunc = [BrushRadiusProperty, PropListOwner](const float NewValue)
	{
		BrushRadiusProperty->SetValue_InContainer(PropListOwner, &NewValue);
		FPropertyChangedEvent PropertyChangedEvent(BrushRadiusProperty, EPropertyChangeType::ValueSet);
		PropListOwner->PostEditChangeProperty(PropertyChangedEvent);
	};
		
	// begin calculation of slider min and maxes
	FString MetaUIMinString = BrushRadiusProperty->GetMetaData(TEXT("UIMin"));
	FString MetaUIMaxString = BrushRadiusProperty->GetMetaData(TEXT("UIMax"));
	FString ClampMinString = BrushRadiusProperty->GetMetaData(TEXT("ClampMin"));
	FString ClampMaxString = BrushRadiusProperty->GetMetaData(TEXT("ClampMax"));

	GetBrushRadiusRangeOverride(Tool, MetaUIMinString, MetaUIMaxString, ClampMinString, ClampMaxString);

	const FString& UIMinString = MetaUIMinString.Len() ? MetaUIMinString : ClampMinString;
	const FString& UIMaxString = MetaUIMaxString.Len() ? MetaUIMaxString : ClampMaxString;

	float ClampMin = TNumericLimits<float>::Lowest();
	float ClampMax = TNumericLimits<float>::Max();

	if (!ClampMinString.IsEmpty())
	{
		TTypeFromString<float>::FromString(ClampMin, *ClampMinString);
	}

	if (!ClampMaxString.IsEmpty())
	{
		TTypeFromString<float>::FromString(ClampMax, *ClampMaxString);
	}

	float UIMin = TNumericLimits<float>::Lowest();
	float UIMax = TNumericLimits<float>::Max();
	TTypeFromString<float>::FromString(UIMin, *UIMinString);
	TTypeFromString<float>::FromString(UIMax, *UIMaxString);

	const float ActualUIMin = FMath::Max(UIMin, ClampMin);
	const float ActualUIMax = FMath::Min(UIMax, ClampMax);
	
	const FText ToolTipText = BrushRadiusProperty->GetToolTipText();

	// construct the widget representation of the BrushRadius property
	SHorizontalBox::FArguments BrushRadiusPropArgs;

	// if display should include the label, add LabelWidget to BrushRadiusPropArgs
	if (DisplayType == EQuickPropertyDisplay::WidgetAndLabel)
	{
		BrushRadiusPropArgs
		+ SHorizontalBox::Slot()
		.VAlign(VAlign_Center)
		.AutoWidth()
		.Padding(8.0f, 0.0f, 0.0f, 0.0f)
		[
			// add BrushRadius label
			SNew(STextBlock)
			.ToolTipText(ToolTipText)
			.Font(FAppStyle::GetFontStyle( TEXT( "MenuItem.Font" ) ))
			.Justification(ETextJustify::Center)
			.Text(LOCTEXT("RadiusLabel", "Radius"))
		];
	}
		
	// if display should include the SSpinBox widget, add it to BrushRadiusPropArgs
	if (DisplayType == EQuickPropertyDisplay::WidgetAndLabel || DisplayType == EQuickPropertyDisplay::WidgetOnly)
	{
		// slider
		BrushRadiusPropArgs
		+ SHorizontalBox::Slot()
		.AutoWidth()
		.Padding(4.0f, 0.0f, 8.0f, 0.0f)
		[
			SNew(SBox)
			.WidthOverride(50.f)
			[
				SNew(SSpinBox<float>)
				.ToolTipText(ToolTipText)
				.Font(FAppStyle::GetFontStyle( TEXT("PropertyWindow.NormalFont")) )
				.MaxFractionalDigits(2)
				.IsEnabled_Lambda([](){ return true; })
				.Delta(0.03125f) // 1/32
				.MinValue(ClampMinString.Len() ? ClampMin : TOptional<float>())
				.MinSliderValue((UIMinString.Len()) ? ActualUIMin : TOptional<float>())
				.MaxValue(ClampMaxString.Len() ? ClampMax : TOptional<float>())
				.MaxSliderValue((UIMaxString.Len()) ? ActualUIMax : TOptional<float>())
				.Value_Lambda([SliderValueFunc](){ return SliderValueFunc();})
				.OnValueChanged_Lambda([SetSliderValueFunc](float NewValue){ SetSliderValueFunc(NewValue);})
			]
		];
			
		BrushRadiusPropArgs
		+ SHorizontalBox::Slot()
		.AutoWidth()
		.Padding(4.f, 0.f, 8.f, 0.f)
		[
			SNew(SSeparator)
			.Orientation(Orient_Vertical)
			.Thickness(1.f)
			.ColorAndOpacity(FStyleColors::Hover)
			.SeparatorImage(FAppStyle::GetBrush( "ThinLine.Horizontal" ))
		];
	}
	
	TSharedRef<SWidget> WholeWidget =
		SNew(SBorder)
		.BorderImage(FAppStyle::GetBrush("NoBorder"))
		.Padding(0.0f)
		.Visibility_Lambda([BrushTool]()
			{
				// if we are not using BrushRadius, don't show widget
				return BrushTool->BrushProperties->bSpecifyRadius ? EVisibility::Visible : EVisibility::Collapsed;
			})
		[
			SArgumentNew(BrushRadiusPropArgs, SHorizontalBox)
		];

	
	WidgetsToAdd.Add(FPropertyWidget(BrushRadiusProperty->GetDisplayNameText(), WholeWidget));
	return true;
}

bool UE::MeshTerrain::CreateStrengthWidget(FProperty* StrengthProperty, UObject* PropListOwner,
	TArray<FPropertyWidget>& WidgetsToAdd, UInteractiveTool* Tool, const EQuickPropertyDisplay DisplayType,
	FIsActionButtonVisible PressureButtonVisible)
{
	UMeshVertexSculptTool* VSculptTool = Cast<UMeshVertexSculptTool>(Tool);
	if (!VSculptTool)
	{
		return false;
	}

	// retrieves the current Strength
	TFunction<float()> SliderValueFunc = [StrengthProperty, PropListOwner]()
	{
		float PropValue;
		StrengthProperty->GetValue_InContainer(PropListOwner, &PropValue);
		return PropValue;
	};

	// sets a new size value for the Strength property
	const TFunction<void(float)> SetSliderValueFunc = [StrengthProperty, PropListOwner](const float NewValue)
	{
		StrengthProperty->SetValue_InContainer(PropListOwner, &NewValue);
		FPropertyChangedEvent PropertyChangedEvent(StrengthProperty, EPropertyChangeType::ValueSet);
		PropListOwner->PostEditChangeProperty(PropertyChangedEvent);
	};

	// is pressure sensitivity currently enabled?
	TFunction<bool()> GetEnablePressureSensitivity = [VSculptTool]()
	{
		return VSculptTool->GetBrushStrengthPressureEnabled();
	};
	
	// toggles the pressure sensitivity of the Strength property
	TFunction<void(bool)> TogglePressureSensitivity = [VSculptTool, StrengthProperty, PropListOwner] (bool bIsChecked)
	{
		VSculptTool->SetBrushStrengthPressureEnabled(bIsChecked);
		FPropertyChangedEvent PropertyChangedEvent(StrengthProperty, EPropertyChangeType::ValueSet);
		PropListOwner->PostEditChangeProperty(PropertyChangedEvent);
	};
		
	// begin calculation of slider min and maxes
	const FString& MetaUIMinString = StrengthProperty->GetMetaData(TEXT("UIMin"));
	const FString& MetaUIMaxString = StrengthProperty->GetMetaData(TEXT("UIMax"));
	const FString& ClampMinString = StrengthProperty->GetMetaData(TEXT("ClampMin"));
	const FString& ClampMaxString = StrengthProperty->GetMetaData(TEXT("ClampMax"));

	const FString& UIMinString = MetaUIMinString.Len() ? MetaUIMinString : ClampMinString;
	const FString& UIMaxString = MetaUIMaxString.Len() ? MetaUIMaxString : ClampMaxString;

	float ClampMin = TNumericLimits<float>::Lowest();
	float ClampMax = TNumericLimits<float>::Max();

	if (!ClampMinString.IsEmpty())
	{
		TTypeFromString<float>::FromString(ClampMin, *ClampMinString);
	}

	if (!ClampMaxString.IsEmpty())
	{
		TTypeFromString<float>::FromString(ClampMax, *ClampMaxString);
	}

	float UIMin = TNumericLimits<float>::Lowest();
	float UIMax = TNumericLimits<float>::Max();
	TTypeFromString<float>::FromString(UIMin, *UIMinString);
	TTypeFromString<float>::FromString(UIMax, *UIMaxString);

	const float ActualUIMin = FMath::Max(UIMin, ClampMin);
	const float ActualUIMax = FMath::Min(UIMax, ClampMax);
	
	const FText ToolTipText = StrengthProperty->GetToolTipText();

	// construct the widget representation of the strength property
	SHorizontalBox::FArguments StrengthPropArgs;

	// if display should include the label, add LabelWidget to StrengthPropArgs
	if (DisplayType == EQuickPropertyDisplay::WidgetAndLabel)
	{
		StrengthPropArgs
		+ SHorizontalBox::Slot()
		.VAlign(VAlign_Center)
		.AutoWidth()
		.Padding(8.0f, 0.0f, 0.0f, 0.0f)
		[
			// add strength label
			SNew(STextBlock)
			.Font(FAppStyle::GetFontStyle( TEXT( "MenuItem.Font" ) ))
			.Justification(ETextJustify::Center)
			.Text(LOCTEXT("StrengthLabel", "Strength"))
			.ToolTipText(ToolTipText)
		];
	}
		
	// if display should include the SSpinBox widget, add it to StrengthPropArgs
	if (DisplayType == EQuickPropertyDisplay::WidgetAndLabel || DisplayType == EQuickPropertyDisplay::WidgetOnly)
	{
		// slider
		StrengthPropArgs
		+ SHorizontalBox::Slot()
		.AutoWidth()
		.Padding(4.0f, 0.0f, 8.0f, 0.0f)
		[
			SNew(SBox)
			.WidthOverride(50.f)
			[
				SNew(SSpinBox<float>)
				.ToolTipText(ToolTipText)
				.Font(FAppStyle::GetFontStyle( TEXT("PropertyWindow.NormalFont")) )
				.MaxFractionalDigits(2)
				.IsEnabled_Lambda([](){ return true; })
				.Delta(0.03125f) // 1/32
				.MinValue(ClampMinString.Len() ? ClampMin : TOptional<float>())
				.MinSliderValue((UIMinString.Len()) ? ActualUIMin : TOptional<float>())
				.MaxValue(ClampMaxString.Len() ? ClampMax : TOptional<float>())
				.MaxSliderValue((UIMaxString.Len()) ? ActualUIMax : TOptional<float>())
				.Value_Lambda([SliderValueFunc](){ return SliderValueFunc();})
				.OnValueChanged_Lambda([SetSliderValueFunc](float NewValue){ SetSliderValueFunc(NewValue);})
			]
		];
			
		StrengthPropArgs
		+ SHorizontalBox::Slot()
		.AutoWidth()
		.HAlign(HAlign_Left)
		.Padding(0.0f, 0.0f, 8.0f, 0.0f)
		[
			SNew(SBox)
			[
				SNew(SCheckBox)
				.Style(FMeshTerrainModeStyle::Get().Get(), "QuickSettingsToggleButton")
				.HAlign(HAlign_Center)
				.Padding(0.f)
				.ToolTipText( LOCTEXT("PressureSensitivityToggleTooltip", "Toggle Pressure Sensitivity") )
				.Visibility_Lambda([PressureButtonVisible]()
					{
						if (PressureButtonVisible.IsBound())
						{
							return PressureButtonVisible.Execute() ? EVisibility::Visible : EVisibility::Collapsed;
						}
						return EVisibility::Collapsed; 
					})
				.OnCheckStateChanged_Lambda([TogglePressureSensitivity](ECheckBoxState State) {
					TogglePressureSensitivity( State == ECheckBoxState::Checked );
				})
				.IsChecked_Lambda([GetEnablePressureSensitivity]() {
					return GetEnablePressureSensitivity() ? ECheckBoxState::Checked : ECheckBoxState::Unchecked;
				})
				[
					SNew(SHorizontalBox)
					+ SHorizontalBox::Slot()
					.VAlign(VAlign_Center)
					.Padding(4.0f)
					.AutoWidth()
					[
						SNew(SImage)
						.Image(FMeshTerrainModeStyle::Get()->GetBrush("BrushIcons.PressureSensitivity"))
						.ColorAndOpacity(FSlateColor::UseForeground())
					]
				]
			]
		]
		+ SHorizontalBox::Slot()
		.AutoWidth()
		.Padding(4.f, 0.f, 8.f, 0.f)
		[
			SNew(SSeparator)
			.Orientation(Orient_Vertical)
			.Thickness(1.f)
			.ColorAndOpacity(FStyleColors::Hover)
			.SeparatorImage(FAppStyle::GetBrush( "ThinLine.Horizontal" ))
		];
	}
	WidgetsToAdd.Add(FPropertyWidget(StrengthProperty->GetDisplayNameText(), SArgumentNew(StrengthPropArgs, SHorizontalBox)));
	return true;
}

bool UE::MeshTerrain::CreateEditorGizmoToggleWidget(FProperty* Prop, UObject* PropListOwner,
	TArray<FPropertyWidget>& WidgetsToAdd, UInteractiveTool* Tool, const EQuickPropertyDisplay DisplayType)
{
	FBoolProperty* const BoolProp = CastField<FBoolProperty>(Prop);
	if (!BoolProp)
	{
		return false;
	}

	WidgetsToAdd.Add(FPropertyWidget(
		Prop->GetDisplayNameText(),
		SNew(SBox)
		[
			SNew(SCheckBox)
			.Style(FMeshTerrainModeStyle::Get().Get(), "QuickSettingsToggleButton")
			.HAlign(HAlign_Center)
			.Padding(0.f)
			.ToolTipText(Prop->GetToolTipText())
			.OnCheckStateChanged_Lambda([BoolProp, PropListOwner](const ECheckBoxState State) {
				const bool bValue = (State == ECheckBoxState::Checked);
				BoolProp->SetPropertyValue_InContainer(PropListOwner, bValue);
				FPropertyChangedEvent PropertyChangedEvent(BoolProp, EPropertyChangeType::ValueSet);
				PropListOwner->PostEditChangeProperty(PropertyChangedEvent);
			})
			.IsChecked_Lambda([BoolProp, PropListOwner]() {
				return BoolProp->GetPropertyValue_InContainer(PropListOwner) ? ECheckBoxState::Checked : ECheckBoxState::Unchecked;
			})
			[
				SNew(SHorizontalBox)
				+ SHorizontalBox::Slot()
				.VAlign(VAlign_Center)
				.Padding(4.0f)
				.AutoWidth()
				[
					SNew(SImage)
					.Image(FAppStyle::GetBrush("LevelEditor.SelectMode"))
					.ColorAndOpacity(FSlateColor::UseForeground())
				]
			]
		]
	));
	return true;
}

bool UE::MeshTerrain::CreateShowGizmoToggleWidget(FProperty* Prop, UObject* PropListOwner,
	TArray<FPropertyWidget>& WidgetsToAdd, UInteractiveTool* Tool, const EQuickPropertyDisplay DisplayType)
{
	FBoolProperty* const BoolProp = CastField<FBoolProperty>(Prop);
	if (!BoolProp)
	{
		return false;
	}

	WidgetsToAdd.Add(FPropertyWidget(
		Prop->GetDisplayNameText(),
		SNew(SBox)
		[
			SNew(SCheckBox)
			.Style(FMeshTerrainModeStyle::Get().Get(), "QuickSettingsToggleButton")
			.HAlign(HAlign_Center)
			.Padding(0.f)
			.ToolTipText(Prop->GetToolTipText())
			.OnCheckStateChanged_Lambda([BoolProp, PropListOwner](const ECheckBoxState State) {
				const bool bValue = (State == ECheckBoxState::Checked);
				BoolProp->SetPropertyValue_InContainer(PropListOwner, bValue);
				FPropertyChangedEvent PropertyChangedEvent(BoolProp, EPropertyChangeType::ValueSet);
				PropListOwner->PostEditChangeProperty(PropertyChangedEvent);
			})
			.IsChecked_Lambda([BoolProp, PropListOwner]() {
				return BoolProp->GetPropertyValue_InContainer(PropListOwner) ? ECheckBoxState::Checked : ECheckBoxState::Unchecked;
			})
			[
				SNew(SHorizontalBox)
				+ SHorizontalBox::Slot()
				.VAlign(VAlign_Center)
				.Padding(4.0f)
				.AutoWidth()
				[
					SNew(SImage)
					.Image(FAppStyle::GetBrush("EditorViewport.TranslateMode"))
					.ColorAndOpacity(FSlateColor::UseForeground())
				]
			]
		]
	));
	return true;
}

bool UE::MeshTerrain::CreateChannelNameWidget(FProperty* Property, UObject* PropListOwner,
	TArray<FPropertyWidget>& WidgetsToAdd, UInteractiveTool* Tool,
	const EQuickPropertyDisplay DisplayType)
{
	FStructProperty* StructProperty = CastField<FStructProperty>(Property);
	if (!StructProperty)
	{
		return false;
	}

	FNameProperty* NameProp = CastField<FNameProperty>(StructProperty->Struct->FindPropertyByName(TEXT("Name")));
	if (!NameProp)
	{
		return false;
	}
	
	const FText ToolTipText = Property->GetToolTipText();

	TSharedPtr<TArray<TSharedPtr<FString>>> ChannelOptions = MakeShared<TArray<TSharedPtr<FString>>>();
	TSharedPtr<bool> bConstrainToDefinition = MakeShared<bool>(false);
	TSharedPtr<TSharedPtr<SSearchableComboBox>> ComboBoxHolder = MakeShared<TSharedPtr<SSearchableComboBox>>();

	FString GetOptionsFunctionName = StructProperty->GetMetaData(TEXT("GetOptions"));

	auto GetCurrentName = [StructProperty, NameProp, PropListOwner]() -> FName
	{
		const void* StructData = StructProperty->ContainerPtrToValuePtr<void>(PropListOwner);
		return NameProp->GetPropertyValue(NameProp->ContainerPtrToValuePtr<FName>(StructData));
	};

	auto SetCurrentName = [StructProperty, NameProp, PropListOwner](const FName& NewName)
	{
		void* StructData = StructProperty->ContainerPtrToValuePtr<void>(PropListOwner);
		NameProp->SetPropertyValue(NameProp->ContainerPtrToValuePtr<FName>(StructData), NewName);
		FPropertyChangedEvent PropertyChangedEvent(StructProperty, EPropertyChangeType::ValueSet);
		PropListOwner->PostEditChangeProperty(PropertyChangedEvent);
	};

	// Repopulates ChannelOptions from the GetOptions function and refreshes the combo box selection.
	// Returns true if the current name is a recognized option (or NAME_None).
	auto RefreshComboOptions = [PropListOwner, ChannelOptions, GetCurrentName, ComboBoxHolder](FString GetOptionsFunctionName) -> bool
	{
		ChannelOptions->Reset();
		if (!GetOptionsFunctionName.IsEmpty())
		{
			TArray<UObject*> OuterObjects = {PropListOwner};
			TArray<FString> OptionStrings;
			PropertyEditorUtils::GetPropertyOptions(OuterObjects, GetOptionsFunctionName, OptionStrings, nullptr);
			for (const FString& Str : OptionStrings)
			{
				ChannelOptions->Add(MakeShared<FString>(Str));
			}
		}

		const FName CurrentName = GetCurrentName();
		bool bCurrentIsValid = CurrentName.IsNone();
		const FString CurrentNameString = CurrentName.ToString();

		TSharedPtr<SSearchableComboBox>& ComboBox = *ComboBoxHolder;
		if (ComboBox.IsValid())
		{
			ComboBox->RefreshOptions();
			const TSharedPtr<FString>* ExistingOption = ChannelOptions->FindByPredicate(
				[&CurrentNameString](const TSharedPtr<FString>& Str) { return Str && *Str == CurrentNameString; });
			if (ExistingOption)
			{
				bCurrentIsValid = true;
				ComboBox->SetSelectedItem(*ExistingOption);
			}
			else
			{
				ComboBox->SetSelectedItem(MakeShared<FString>(CurrentNameString));
			}
		}
		else
		{
			bCurrentIsValid |= ChannelOptions->ContainsByPredicate(
				[&CurrentNameString](const TSharedPtr<FString>& Str) { return Str && *Str == CurrentNameString; });
		}

		return bCurrentIsValid;
	};

	// Do an initial options pass so ChannelOptions is populated before the combo box is constructed.
	*bConstrainToDefinition = RefreshComboOptions(GetOptionsFunctionName);

	TSharedRef<SSearchableComboBox> ComboBoxRef =
		SNew(SSearchableComboBox)
		.ToolTipText(ToolTipText)
		.OptionsSource(ChannelOptions.Get())
		.OnComboBoxOpening_Lambda([RefreshComboOptions, GetOptionsFunctionName]()
		{
			RefreshComboOptions(GetOptionsFunctionName);
		})
		.OnGenerateWidget_Lambda([](TSharedPtr<FString> InOption) -> TSharedRef<SWidget>
		{
			return SNew(STextBlock)
				.Text(InOption.IsValid() ? FText::FromString(*InOption) : FText::GetEmpty());
		})
		.OnSelectionChanged_Lambda([SetCurrentName](TSharedPtr<FString> InSelectedItem, ESelectInfo::Type)
		{
			if (InSelectedItem.IsValid())
			{
				SetCurrentName(FName(**InSelectedItem));
			}
		})
		.SearchVisibility(EVisibility::Visible)
		.Visibility_Lambda([bConstrainToDefinition]()
		{
			return *bConstrainToDefinition ? EVisibility::Visible : EVisibility::Collapsed;
		})
		[
			SNew(STextBlock)
			.Text_Lambda([GetCurrentName]() { return FText::FromName(GetCurrentName()); })
			.Font(FAppStyle::GetFontStyle(TEXT("PropertyWindow.NormalFont")))
		];

	// Now that the combo box is built, set the holder and refresh to establish the correct selection.
	*ComboBoxHolder = ComboBoxRef;
	*bConstrainToDefinition = RefreshComboOptions(GetOptionsFunctionName);

	TSharedRef<SEditableTextBox> TextBoxRef =
		SNew(SEditableTextBox)
		.ToolTipText(ToolTipText)
		.Font(FAppStyle::GetFontStyle(TEXT("PropertyWindow.NormalFont")))
		.Text_Lambda([GetCurrentName]() { return FText::FromName(GetCurrentName()); })
		.OnTextCommitted_Lambda([SetCurrentName](const FText& NewText, ETextCommit::Type)
		{
			SetCurrentName(FName(*NewText.ToString()));
		})
		.Visibility_Lambda([bConstrainToDefinition]()
		{
			return *bConstrainToDefinition ? EVisibility::Collapsed : EVisibility::Visible;
		});

	// Retrieve the MeshPartitionDefinition brush from the registered MegaMesh style.
	// This icon matches the one used by FToggleableConstraintNameCustomization.
	static const FName MegaMeshStyleName(TEXT("MegaMeshEditorUIStyle"));
	static const FName DefinitionBrushName(TEXT("MeshPartitionDefinition"));
	const FSlateBrush* DefinitionBrush = FAppStyle::GetBrush("NoBrush");
	if (const ISlateStyle* MegaMeshStyle = FSlateStyleRegistry::FindSlateStyle(MegaMeshStyleName))
	{
		DefinitionBrush = MegaMeshStyle->GetBrush(DefinitionBrushName);
	}

	TSharedRef<SCheckBox> ToggleRef =
		SNew(SCheckBox)
		.Style(FAppStyle::Get(), "DetailsView.SectionButton")
		.Padding(FMargin(4, 2))
		.ToolTipText(LOCTEXT("ChannelNameConstrainToDefinitionTooltip", "Constrain options to ones defined in the Mesh Partition Definition."))
		.HAlign(HAlign_Center)
		.OnCheckStateChanged_Lambda([bConstrainToDefinition](const ECheckBoxState NewState)
		{
			*bConstrainToDefinition = NewState == ECheckBoxState::Checked;
		})
		.IsChecked_Lambda([bConstrainToDefinition]() -> ECheckBoxState
		{
			return *bConstrainToDefinition ? ECheckBoxState::Checked : ECheckBoxState::Unchecked;
		})
		[
			SNew(SHorizontalBox)
			+ SHorizontalBox::Slot()
			.VAlign(VAlign_Center)
			.Padding(FMargin(0))
			.AutoWidth()
			[
				SNew(SImage)
				.Image(DefinitionBrush)
			]
		];

	SHorizontalBox::FArguments ChannelNameArgs;

	if (DisplayType == EQuickPropertyDisplay::WidgetAndLabel)
	{
		ChannelNameArgs
		+ SHorizontalBox::Slot()
		.VAlign(VAlign_Center)
		.AutoWidth()
		.Padding(4.0f, 0.0f, 4.0f, 0.0f)
		[
			SNew(STextBlock)
			.ToolTipText(ToolTipText)
			.Font(FAppStyle::GetFontStyle(TEXT("MenuItem.Font")))
			.Justification(ETextJustify::Center)
			.Text(StructProperty->GetDisplayNameText())
		];
	}

	if (DisplayType == EQuickPropertyDisplay::WidgetAndLabel || DisplayType == EQuickPropertyDisplay::WidgetOnly)
	{
		ChannelNameArgs
		+ SHorizontalBox::Slot()
		.AutoWidth()
		.VAlign(VAlign_Center)
		.Padding(4.0f, 0.0f, 0.0f, 0.0f)
		[
			SNew(SBox)
			.WidthOverride(100.f)
			[
				TextBoxRef
			]
		]
		+ SHorizontalBox::Slot()
		.AutoWidth()
		.VAlign(VAlign_Center)
		.Padding(4.0f, 0.0f, 0.0f, 0.0f)
		[
			SNew(SBox)
			.WidthOverride(100.f)
			[
				ComboBoxRef
			]
		]
		+ SHorizontalBox::Slot()
		.AutoWidth()
		.VAlign(VAlign_Center)
		.Padding(2.0f, 0.0f, 4.0f, 0.0f)
		[
			ToggleRef
		];
	}

	WidgetsToAdd.Add(FPropertyWidget(StructProperty->GetDisplayNameText(), SArgumentNew(ChannelNameArgs, SHorizontalBox)));
	return true;
}

#undef LOCTEXT_NAMESPACE
