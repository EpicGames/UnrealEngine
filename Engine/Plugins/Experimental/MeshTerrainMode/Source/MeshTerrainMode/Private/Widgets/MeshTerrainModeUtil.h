// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Framework/Commands/UIAction.h"
#include "Widgets/SCompoundWidget.h"

class UInteractiveTool;
class UScriptStruct;
enum class EBrushToolSizeType : uint8;

namespace UE::MeshTerrain
{
	/** Types of Property displays */
	enum class EQuickPropertyDisplay : uint8
	{
		/** A Quick Property's display should only include the widget itself (SSpinbox, SButton, etc.) */
		WidgetOnly = 0,
		/** A Quick Property's display should include both the widget itself (SSpinbox, SButton, etc.) and the property's label */
		WidgetAndLabel = 1,
	};

	/** Types of boolean displays */
	enum class EBoolPropertyDisplay : uint8
	{
		/** A button containing either the property label, or an icon (when a customization is registered) */
		Button = 0,
		/** Represented as a standard checkbox */
		CheckBox = 1
	};

	/** A struct to track properties and the widgets which represent them */
	struct FPropertyWidget
	{
		FPropertyWidget(const FText& InPropertyName, const TSharedRef<SWidget>& InWidgetRepresentation);;

		FText PropertyName;
		TSharedRef<SWidget> WidgetRepresentation;
	};
	
	DECLARE_DELEGATE_RetVal_FiveParams(bool, FGetQuickPropertyCustomization, FProperty*, UObject*, TArray<FPropertyWidget>&, UInteractiveTool*, EQuickPropertyDisplay);

	/** FModelingQuickPropertyCustomizations */
	class FModelingQuickPropertyCustomizations
	{
		public:
			FModelingQuickPropertyCustomizations() { };

			// return a map containing all the Modeling Quick Property registered customizations
			TMap<UClass*, FGetQuickPropertyCustomization> GetCustomizations() { return Customizations; }

			// return a map containing all the Modeling Quick Property registered boolean customizations
			TMap<FName, TAttribute<const FSlateBrush*>> GetBoolCustomizations() { return BoolCustomizations; }

			// register a customization function to a tool
			void RegisterQuickPropertyCustomization(UClass* ToolClass, FGetQuickPropertyCustomization CustomizationDelegate)
			{
				// ensure the provided customization func is not null before registering
				if (CustomizationDelegate.IsBound())
				{
					Customizations.Add(ToolClass, CustomizationDelegate);
				}
			}

			// register a slate brush override to a boolean property
			void RegisterQuickPropertyBooleanCustomization(const FName BoolPropName, const TAttribute<const FSlateBrush*>& IconOverride)
			{
				BoolCustomizations.Add(BoolPropName, IconOverride);
			}

			// register a customization function to a struct type, applied to any tool that exposes a tagged property of that struct type
			void RegisterQuickPropertyStructTypeCustomization(UScriptStruct* StructType, FGetQuickPropertyCustomization CustomizationDelegate)
			{
				if (CustomizationDelegate.IsBound())
				{
					StructTypeCustomizations.Add(StructType, CustomizationDelegate);
				}
			}

			// return a map containing all registered struct-type customizations
			TMap<UScriptStruct*, FGetQuickPropertyCustomization> GetStructTypeCustomizations() { return StructTypeCustomizations; }

		private:
			// maps tool class to its customization function
			TMap<UClass*, FGetQuickPropertyCustomization> Customizations;

			// maps boolean property names to their intended slate brush overrides
			TMap<FName, TAttribute<const FSlateBrush*>> BoolCustomizations;

			// maps struct types to their customization functions, applied regardless of tool class
			TMap<UScriptStruct*, FGetQuickPropertyCustomization> StructTypeCustomizations;
	};
	
	/* Widget Collection */
	
	void CollectTaggedWidgetsRecursive(
		FProperty* Property,
		UObject* PropListOwner,
		const TCHAR* Tag,
		UInteractiveTool* ActiveTool,
		TMap<int, TArray<FPropertyWidget>>& PriorityToWidgets,
		const TFunction<void(FProperty* Prop, UObject* PropListOwner, TArray<FPropertyWidget>& Widgets)>& HandlePropsFunc,
		TMap<UClass*, FGetQuickPropertyCustomization> Customizations,
		const EQuickPropertyDisplay CustomizationDisplayType = EQuickPropertyDisplay::WidgetAndLabel,
		TMap<UScriptStruct*, FGetQuickPropertyCustomization> StructTypeCustomizations = {});

	/* Standard Widget Creation */
	
	void AddSliderWidget(FFloatProperty* FloatProperty, UObject* PropListOwner, TArray<FPropertyWidget>& WidgetsToAdd,
		const EQuickPropertyDisplay DisplayType = EQuickPropertyDisplay::WidgetAndLabel);
	void AddEnumWidget(FEnumProperty* EnumProperty, UObject* PropListOwner, TArray<FPropertyWidget>& WidgetsToAdd,
		const EQuickPropertyDisplay DisplayType = EQuickPropertyDisplay::WidgetAndLabel);
	void AddBoolWidget(FBoolProperty* BoolProperty, UObject* PropListOwner, TArray<FPropertyWidget>& WidgetsToAdd, TMap<FName, TAttribute<const FSlateBrush*>> BoolCustomizations,
		const EQuickPropertyDisplay DisplayType = EQuickPropertyDisplay::WidgetOnly, const EBoolPropertyDisplay BoolDisplayType = EBoolPropertyDisplay::Button);

	TSharedRef<SWidget> CreateQuickEditPropertyRow(const FPropertyWidget& PropertyWidget);
	
	/* Widget Customizations */
	
	// appends a Widget representing the Falloff property to the provided Array
	bool CreateFalloffWidget(FProperty* FalloffProperty, UObject* PropListOwner, TArray<FPropertyWidget>& WidgetsToAdd,
		UInteractiveTool* Tool, const EQuickPropertyDisplay DisplayType = EQuickPropertyDisplay::WidgetOnly);

	// appends a Widget representing the BrushSize property to the provided Array - including world/adaptive size toggle
	bool CreateBrushSizeToggleWidget(FProperty* Prop, UObject* PropListOwner, TArray<FPropertyWidget>& WidgetsToAdd,
		UInteractiveTool* Tool, const EQuickPropertyDisplay DisplayType = EQuickPropertyDisplay::WidgetAndLabel,
		FIsActionButtonVisible PressureButtonVisible = nullptr);

	// appends a Widget representing the Alpha property to the provided Array
	bool CreateAlphaWidget(FProperty* AlphaProperty, UObject* PropListOwner, TArray<FPropertyWidget>& WidgetsToAdd,
		UInteractiveTool* Tool, EQuickPropertyDisplay DisplayType = EQuickPropertyDisplay::WidgetAndLabel);

	// appends a Widget representing the Strength property to the provided Array
	bool CreateStrengthWidget(FProperty* StrengthProperty, UObject* PropListOwner, TArray<FPropertyWidget>& WidgetsToAdd,
		UInteractiveTool* Tool, EQuickPropertyDisplay DisplayType = EQuickPropertyDisplay::WidgetAndLabel,
		FIsActionButtonVisible PressureButtonVisible = nullptr);
	
	// appends a Widget representing the BrushSize property to the provided Array - only including slider and pressure sensitivity toggle
	bool CreateBrushSizeWidget(FProperty* Prop, UObject* PropListOwner, TArray<FPropertyWidget>& WidgetsToAdd,
		UInteractiveTool* Tool, const EQuickPropertyDisplay DisplayType = EQuickPropertyDisplay::WidgetAndLabel,
		FIsActionButtonVisible PressureButtonVisible = nullptr);

	// appends a Widget representing the BrushRadius property to the provided Array - only including slider
	bool CreateBrushRadiusWidget(FProperty* Prop, UObject* PropListOwner, TArray<FPropertyWidget>& WidgetsToAdd,
		UInteractiveTool* Tool, const EQuickPropertyDisplay DisplayType = EQuickPropertyDisplay::WidgetAndLabel);

	// Resolves the effective slider range for the BrushRadius property when it is owned by a
	// specific tool. Returns true if any of the output strings have been replaced, false otherwise.
	// An output left empty by the helper means "fall back to the FProperty's declared metadata".
	bool GetBrushRadiusRangeOverride(
		const UInteractiveTool* Tool,
		FString& OutUIMin,
		FString& OutUIMax,
		FString& OutClampMin,
		FString& OutClampMax);

	// Resolves the effective slider range for FBrushToolRadius given the active tool and which
	// mode (Adaptive vs World) is currently selected. Returns the defaults from the metadata when no
	// tool-specific override applies; otherwise returns a widened range tailored to the tool.
	void GetSculptBrushSizeRange(
		const UInteractiveTool* Tool,
		EBrushToolSizeType SizeType,
		TInterval<float>& OutUIRange,
		TInterval<float>& OutValueRange);

	// appends a toggle Widget for the bAllowEditorGizmo property to the provided Array
	bool CreateEditorGizmoToggleWidget(FProperty* Prop, UObject* PropListOwner, TArray<FPropertyWidget>& WidgetsToAdd,
		UInteractiveTool* Tool, const EQuickPropertyDisplay DisplayType = EQuickPropertyDisplay::WidgetAndLabel);

	// appends a toggle Widget for the bShowGizmo property to the provided Array
	bool CreateShowGizmoToggleWidget(FProperty* Prop, UObject* PropListOwner, TArray<FPropertyWidget>& WidgetsToAdd,
		UInteractiveTool* Tool, const EQuickPropertyDisplay DisplayType = EQuickPropertyDisplay::WidgetOnly);

	// appends a Widget representing a MeshPartition::FChannelName property to the provided Array.
	// Renders a searchable combo box constrained to GetOptions values, with a toggle to switch to
	// free-text entry, matching the look of FToggleableConstraintNameCustomization.
	bool CreateChannelNameWidget(FProperty* ChannelNameProperty, UObject* PropListOwner,
		TArray<FPropertyWidget>& WidgetsToAdd, UInteractiveTool* Tool,
		const EQuickPropertyDisplay DisplayType = EQuickPropertyDisplay::WidgetAndLabel);

}
