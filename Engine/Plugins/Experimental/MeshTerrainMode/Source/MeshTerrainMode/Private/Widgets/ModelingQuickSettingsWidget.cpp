// Copyright Epic Games, Inc. All Rights Reserved.

#include "Widgets/ModelingQuickSettingsWidget.h"

#include "InteractiveTool.h"
#include "Brushes/SlateRoundedBoxBrush.h"
#include "Widgets/Images/SImage.h"
#include "Widgets/Input/SComboButton.h"
#include "Widgets/Layout/SBox.h"
#include "Widgets/SOverlay.h"
#include "Widgets/Input/SCheckBox.h"
#include "Widgets/Layout/SSeparator.h"
#include "Styling/StyleColors.h"
#include "IDetailsView.h"
#include "MeshTerrainModeStyle.h"
#include "MeshTerrainModeUtil.h"

SModelingQuickSettingsWidget::SModelingQuickSettingsWidget()
{
	QuickSettingsCustomizations = MakeShared<UE::MeshTerrain::FModelingQuickPropertyCustomizations>(UE::MeshTerrain::FModelingQuickPropertyCustomizations());
}

void SModelingQuickSettingsWidget::Construct(const FArguments& InArgs)
{
	this->ToolShutdownButtons = InArgs._ToolShutdownButtons;
	
	ChildSlot
	.HAlign(HAlign_Fill)
	.VAlign(VAlign_Center)
	[
		SNew(SOverlay)
		+ SOverlay::Slot()
		.HAlign(HAlign_Fill)
		.VAlign(VAlign_Center)
		[
			// This outer box is intended to constrain this overlay slot's width to:
			// SlotWidth = TotalWidth - ShutdownContainerSlotWidth.
			// This ensures that this overlay slot will clip before the ShutdownContainer.
			SNew(SBox)
			.Padding(this, &SModelingQuickSettingsWidget::ComputeContentPadding)
			[
				SNew(SBox)
				.Clipping(EWidgetClipping::ClipToBounds)
				[
					SAssignNew(Container, SHorizontalBox)
				]
			]
		]
		+ SOverlay::Slot()
		.HAlign(HAlign_Right)
		.VAlign(VAlign_Center)
		[
			SAssignNew(ShutdownContainer, SHorizontalBox)
		]
	];
}

void SModelingQuickSettingsWidget::SetActiveTool(UInteractiveTool* Tool)
{
	Container->ClearChildren();
	ShutdownContainer->ClearChildren();

	// toggle button to show/hide details view
	const TSharedRef<SWidget> PropertiesButton =
		SNew(SCheckBox)
		.HAlign(HAlign_Center)
		.Style(FMeshTerrainModeStyle::Get().Get(), "QuickSettingsLargeButton")
		.IsChecked_Lambda([this]()
		{
			return bCurrentlyShowingDetailsView ? ECheckBoxState::Checked : ECheckBoxState::Unchecked;
		})
		.OnCheckStateChanged_Lambda([this](const ECheckBoxState CheckState)
		{
			bCurrentlyShowingDetailsView = CheckState == ECheckBoxState::Checked;
			OnPropertiesButtonPressed.ExecuteIfBound(bCurrentlyShowingDetailsView);
		})
		[
			SNew(SHorizontalBox)
			+SHorizontalBox::Slot()
			.VAlign(VAlign_Center)
			.AutoWidth()
			[
				SNew(SImage)
				.DesiredSizeOverride(FVector2D(20,20))
				.Image(FAppStyle::GetBrush("EditorViewportToolBar.OptionsDropdown"))
			]
		];

	if (!ToolShutdownButtons.IsValid())
	{
		ToolShutdownButtons = SNullWidget::NullWidget;
	}

	Container->AddSlot()
		.HAlign(HAlign_Center)
		.VAlign(VAlign_Fill)
		.AutoWidth()
		.Padding(6.f, 2.f, 8.f, 2.f)
		[
			PropertiesButton
		];

	// retrieves the tool's properties and continues to build the widget based on them
	const TArray<UObject*> CurToolProperties = Tool->GetToolProperties(true);
	ActiveTool = Tool;
	AddToolPropertiesToWidget(CurToolProperties);

	// add the tool shutdown (accept/cancel/complete) widget(s) to the persistent right layer
	ShutdownContainer->AddSlot()
	.Padding(8.f, 2.f, 8.f, 2.f)
	.HAlign(HAlign_Center)
	.VAlign(VAlign_Fill)
	.AutoWidth()
	[
		SNew(SSeparator)
		.Orientation(Orient_Vertical)
		.Thickness(1.f)
		.ColorAndOpacity(FStyleColors::Hover)
		.SeparatorImage(FAppStyle::GetBrush("ThinLine.Horizontal"))
	];
	ShutdownContainer->AddSlot()
	.Padding(4.f, 0.f, 2.f, 0.f)
	.HAlign(HAlign_Center)
	.VAlign(VAlign_Center)
	.AutoWidth()
	[
		ToolShutdownButtons.ToSharedRef()
	];
}


TSharedRef<SWidget> SModelingQuickSettingsWidget::CreateDetailPanel() const
{
	return DetailsView.ToSharedRef();
}

FMargin SModelingQuickSettingsWidget::ComputeContentPadding() const
{
	if (!ShutdownContainer.IsValid())
	{
		return FMargin(0.f);
	}
	return FMargin(0.f, 0.f, ShutdownContainer->GetDesiredSize().X, 0.f);
}

void SModelingQuickSettingsWidget::AddToolPropertiesToWidget(const TArray<UObject*>& AllToolPropertySets) const
{
	using namespace UE::MeshTerrain;
	
	// there may be multiple QS properties (and therefore widgets) assigned to each priority num.
	//	we will collect these widgets in an array and map this array to the priority num
	TMap<int, TArray<FPropertyWidget>> PriorityToWidgets;

	// default handling of properties tagged with ModelingQuickSettings (no customizations found)
	auto DefaultHandleQuickSettingsProp =
		[this](FProperty* Prop, UObject* PropListOwner, TArray<FPropertyWidget>& Widgets)
		{
			// collect widgets for float props (represented by slider/SSpinBox)
			if (FFloatProperty* FloatProperty = CastField<FFloatProperty>(Prop))
			{
				AddSliderWidget(FloatProperty, PropListOwner, Widgets);
			}
			// collect widgets for bool props (represented by checkbox)
			else if (FBoolProperty* BoolProperty = CastField<FBoolProperty>(Prop))
			{
				AddBoolWidget(BoolProperty, PropListOwner, Widgets, QuickSettingsCustomizations->GetBoolCustomizations());
			}
			// collect widgets for enum props (represented by drop down menu)
			else if (FEnumProperty* EnumProperty = CastField<FEnumProperty>(Prop))
			{
				AddEnumWidget(EnumProperty, PropListOwner, Widgets);
			}
		};

	// iterate through all the tool's property sets
	for (UObject* PropertySet : AllToolPropertySets)
	{
		// iterate through all properties within a set
		for (TFieldIterator<FProperty> PropIt(PropertySet->GetClass()); PropIt; ++PropIt)
		{
			FProperty* Property = *PropIt;

			// determine which properties are valid, editable, and tagged via its metadata with 'ModelingQuickSettings'
			// and collect these property widgets to be added to this main QuickSettingsWidget
			CollectTaggedWidgetsRecursive(Property, PropertySet, TEXT("ModelingQuickSettings"), ActiveTool,
				PriorityToWidgets, DefaultHandleQuickSettingsProp, QuickSettingsCustomizations->GetCustomizations(),
				EQuickPropertyDisplay::WidgetAndLabel, QuickSettingsCustomizations->GetStructTypeCustomizations());
		}
	}

	// sort widget arrays by priority (low->high)
	PriorityToWidgets.KeySort(TLess<int>());

	TArray<TArray<FPropertyWidget>> SortedWidgetArrays;
	PriorityToWidgets.GenerateValueArray(SortedWidgetArrays);
	
	SHorizontalBox::FArguments PropsArgs;

	// beginning with widgets in smallest-priority arrays, add each widget in each array to PropsArgs
	for (TArray<FPropertyWidget> WidgetArray : SortedWidgetArrays)
	{
		for (const FPropertyWidget& Widget : WidgetArray)
		{
			PropsArgs
			+ SHorizontalBox::Slot()
			.AutoWidth()
			[
				Widget.WidgetRepresentation->AsShared()
			];
		}
	}

	Container->AddSlot()
	.HAlign(HAlign_Fill)
	.VAlign(VAlign_Center)
	.FillWidth(1.0f)
	[
		// To ensure that the tool properties clip first as the widget is compressed. 
		SNew(SBox)
		.HAlign(HAlign_Center)
		.Clipping(EWidgetClipping::ClipToBounds)
		[
			SArgumentNew(PropsArgs, SHorizontalBox)
		]
	];
}