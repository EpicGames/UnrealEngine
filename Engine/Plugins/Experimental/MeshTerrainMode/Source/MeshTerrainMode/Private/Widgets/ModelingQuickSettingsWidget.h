// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Widgets/SCompoundWidget.h"
#include "Widgets/SBoxPanel.h"

class UInteractiveTool;
class IDetailsView;
namespace UE::MeshTerrain { class FModelingQuickPropertyCustomizations; }
struct FArguments;

DECLARE_DELEGATE_OneParam(FOnPropertiesButtonPressed, bool);

// SModelingQuickSettingsWidget
class SModelingQuickSettingsWidget : public SCompoundWidget
{
public:
	SLATE_BEGIN_ARGS( SModelingQuickSettingsWidget ){}
	SLATE_ARGUMENT(TSharedPtr<SWidget>, ToolShutdownButtons)
	SLATE_END_ARGS();

	SModelingQuickSettingsWidget();

	void Construct(const FArguments& InArgs);

	// Builds the ModelingQuickSettings widget based on the active tool & its properties
	void SetActiveTool(UInteractiveTool* Tool);

	// Sets the IDetailsView for the tool (active tool's IDetailsView)
	void SetDetailsView(const TSharedPtr<IDetailsView>& InDetailsView) { DetailsView = InDetailsView; }

	// Handle the button which displays/hides the details view widget
	FOnPropertiesButtonPressed OnPropertiesButtonPressed;
	bool ShowingDetailsView() const { return bCurrentlyShowingDetailsView; }
	void SetShowingDetailsView(const bool bShowDetailsView) { bCurrentlyShowingDetailsView = bShowDetailsView; }

	// Gets the Modeling Quick Settings customizations for the widget
	TSharedPtr<UE::MeshTerrain::FModelingQuickPropertyCustomizations> GetCustomizations() { return QuickSettingsCustomizations;}
private:
	TSharedPtr<SHorizontalBox> Container;
	TSharedPtr<SHorizontalBox> ShutdownContainer;

	FMargin ComputeContentPadding() const;

	// add widgets to the Quick Settings Overlay for given tool properties
	void AddToolPropertiesToWidget(const TArray<UObject*>&) const;

	// creates details panel from IDetailsView
	TSharedRef<SWidget> CreateDetailPanel() const;
	TSharedPtr<IDetailsView> DetailsView = nullptr;
	
	// Widget which contains tool shutdown buttons (accept, cancel, complete)
	TSharedPtr<SWidget> ToolShutdownButtons;
	bool bCurrentlyShowingDetailsView = false;

	// represents the widget's Modeling Quick Settings Customizations
	TSharedPtr<UE::MeshTerrain::FModelingQuickPropertyCustomizations> QuickSettingsCustomizations;

	// the currently active tool
	UInteractiveTool* ActiveTool = nullptr;
};