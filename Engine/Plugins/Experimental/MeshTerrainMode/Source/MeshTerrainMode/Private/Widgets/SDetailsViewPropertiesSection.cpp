// Copyright Epic Games, Inc. All Rights Reserved.

#include "Widgets/SDetailsViewPropertiesSection.h"

#include "ISinglePropertyView.h"
#include "MeshTerrainDetailCustomizations.h"
#include "MeshTerrainModeStyle.h"
#include "SSectionedDetailsViewWidget.h"
#include "Modules/ModuleManager.h"
#include "Widgets/Layout/SBox.h"
#include "Widgets/Text/STextBlock.h"
#include "Widgets/Layout/SSeparator.h"
#include "Styling/StyleColors.h"

void UE::MeshTerrain::SDetailsViewPropertiesSection::Construct(const FArguments& InArgs)
{
	OnSectionedDetailsPanelRequestRebuild = InArgs._OnSectionedDetailsPanelRequestRebuild;

	ActiveProperties = MakeShared<TSet<FProperty*>>();
	
	ChildSlot
	.HAlign(HAlign_Fill)
	.VAlign(VAlign_Center)
	[
		SAssignNew(Container, SVerticalBox)
	];
}

void UE::MeshTerrain::SDetailsViewPropertiesSection::UpdateActiveSection(const TArray<FDetailsCategory>& NewActiveSection,
	const TSharedPtr<IDetailsView>& DetailsView)
{
	ActiveSection = NewActiveSection;
	DetailsView.IsValid() ? Rebuild(DetailsView) : Rebuild();
}

void UE::MeshTerrain::SDetailsViewPropertiesSection::Rebuild()
{
	Container->ClearChildren();

	if (ActiveSection.Num() > 0)
	{
		Container->AddSlot()
		.MaxHeight(1.0f)
		.Padding(0.f)
		[
			SNew(SSeparator)
			.Orientation(Orient_Horizontal)
			.Thickness(1.f)
			.ColorAndOpacity(FStyleColors::Background)
			.SeparatorImage(FAppStyle::GetBrush( "ThinLine.Horizontal" ))
		];
	}
	
	// for each category in the section
	for (FDetailsCategory Category : ActiveSection)
	{
		// TODO : allow drop down functionality
		Container->AddSlot()
		.HAlign(HAlign_Fill)
		.VAlign(VAlign_Center)
		.AutoHeight()
		[
			MakeCategoryHeader(Category.CategoryName).ToSharedRef()
		];

		// add each individual property within each category
		for (FPropertyDetails PropDetails : Category.Properties)
		{
			// retrieve the property's customization information, if it exists
			const FMeshTerrainCustomizationData* CustomizationData = FMeshTerrainDetailCustomizations::Get()->GetCustomizationData(PropDetails.PropOwner, PropDetails.Property);

			TSharedRef<SWidget> Widget = SNullWidget::NullWidget;

			// if customization data exists, use it to find the property's custom widget
			if (CustomizationData && CustomizationData->Type != EMeshTerrainCustomizationType::None && CustomizationData->DetailCustomization.IsBound())
			{
				Widget = CustomizationData->DetailCustomization.Execute(PropDetails.Property, PropDetails.PropOwner);
			}
			// otherwise, use the SinglePropertyView interface
			else
			{
				FPropertyEditorModule& PropertyEditor = FModuleManager::Get().LoadModuleChecked<FPropertyEditorModule>("PropertyEditor");

				FSinglePropertyParams SinglePropertyParams;
				SinglePropertyParams.bHideResetToDefault = true;
				SinglePropertyParams.NamePlacement = EPropertyNamePlacement::Hidden;

				TSharedRef<ISinglePropertyView> SinglePropertyView = PropertyEditor.CreateSingleProperty(PropDetails.PropOwner, PropDetails.Property->GetFName(), SinglePropertyParams).ToSharedRef();
				Widget = SinglePropertyView;
			}

			SHorizontalBox::FArguments PropertyRowArgs;
			// standard layout: label on left and affecting widget (either custom or default) on right
			if (!CustomizationData || CustomizationData->Type == EMeshTerrainCustomizationType::WidgetOnly
			|| CustomizationData->Type == EMeshTerrainCustomizationType::None)
			{
				PropertyRowArgs
				+ SHorizontalBox::Slot()
				.AutoWidth()
				.HAlign(HAlign_Center)
				[
					SNew(SHorizontalBox)
					+SHorizontalBox::Slot()
					.AutoWidth()
					[
						SNew(SBox)
						.Padding(5.f, 4.f, 0.f, 4.f)
						.HAlign(HAlign_Fill)
						.VAlign(VAlign_Center)
						.WidthOverride(100.f)
						[
							SNew(STextBlock)
							.Font(FAppStyle::GetFontStyle( TEXT("DetailsView.CategoryFontStyle")))
							.Justification(ETextJustify::Left)
							.OverflowPolicy(ETextOverflowPolicy::Ellipsis)
							.Text( (PropDetails.Property->GetDisplayNameText()))
						]
					]
					+SHorizontalBox::Slot()
					.AutoWidth()
					[
						SNew(SSeparator)
						.Orientation(Orient_Vertical)
						.Thickness(1.f)
					]
					+SHorizontalBox::Slot()
					.AutoWidth()
					[
						SNew(SBox)
						.Padding(5.f, 0.f, 5.f, 0.f)
						.HAlign(HAlign_Fill)
						[
							Widget
						]
					]
				];
			}
			// case in which a customization takes over the whole row, does not need a label
			else if (CustomizationData->Type == EMeshTerrainCustomizationType::WholeRow)
			{
				PropertyRowArgs
				+ SHorizontalBox::Slot()
				.AutoWidth()
				[
					SNew(SHorizontalBox)
					+SHorizontalBox::Slot()
					.AutoWidth()
					[
						SNew(SBox)
						.Padding(5.f, 0.f, 5.f, 0.f)
						.HAlign(HAlign_Fill)
						[
							Widget
						]
					]
				];
			}
				
			Container->AddSlot()
			.HAlign(HAlign_Fill)
			.VAlign(VAlign_Center)
			.AutoHeight()
			[
				SNew(SBorder)
				.BorderImage( FAppStyle::Get().GetBrush("ToolPanel.DarkGroupBorder"))
				.BorderBackgroundColor(FSlateColor(FLinearColor::White))
				.Padding(0.f)
				.Visibility_Lambda([this, PropDetails]()
				{
					const FMeshTerrainEditConditionData* EditConditionData = FMeshTerrainDetailCustomizations::Get()->GetEditCondition(PropDetails.PropOwner, PropDetails.Property);

					// if there is no edit condition, show the property as normal
					if (!EditConditionData)
					{
						return EVisibility::Visible;
					}

					const bool EditConditionResult = EditConditionData->EditCondition.Execute(PropDetails.Property, PropDetails.PropOwner);

					// if the EditCondition is true and EditConditionHides, hide the entire property
					if (!EditConditionResult && EditConditionData->bEditConditionHides)
					{
						return EVisibility::Collapsed;
					}
					// if the EditCondition is true and EditCondition does NOT hide, grey out the property
					if (!EditConditionResult && !EditConditionData->bEditConditionHides)
					{
						return EVisibility::HitTestInvisible;
					}

					// otherwise, there is an EditCondition and it is true (property can be edited)
					return EVisibility::Visible;
						
				})
				[
					SArgumentNew(PropertyRowArgs, SHorizontalBox)
				]
			];
		}
	}
}

void UE::MeshTerrain::SDetailsViewPropertiesSection::Rebuild(TSharedPtr<IDetailsView> DetailsView)
{
	if (!ActiveProperties)
	{
		return;
	}

	Container->ClearChildren();
	ActiveProperties->Empty();

	if (ActiveSection.Num() > 0)
	{
		Container->AddSlot()
		.MaxHeight(1.0f)
		.Padding(0.f)
		[
			SNew(SSeparator)
			.Orientation(Orient_Horizontal)
			.Thickness(1.f)
			.ColorAndOpacity(FStyleColors::Background)
			.SeparatorImage(FAppStyle::GetBrush( "ThinLine.Horizontal" ))
		];
	}
	
	// for each category in the active section
	for (FDetailsCategory Category : ActiveSection)
	{
		// add each individual property within each category
		for (FPropertyDetails PropDetails : Category.Properties)
		{
			ActiveProperties->Add(PropDetails.Property);
		}
	}
	OnSectionedDetailsPanelRequestRebuild.ExecuteIfBound();

	if (DetailsView.IsValid() && ActiveSection.Num() > 0)
	{
		Container->AddSlot()
		.HAlign(HAlign_Fill)
		.VAlign(VAlign_Center)
		.AutoHeight()
		[
			DetailsView.ToSharedRef()
		];
	}
}

TSharedPtr<SWidget> UE::MeshTerrain::SDetailsViewPropertiesSection::MakeCategoryHeader(const FName CategoryName)
{
	return SNew(SVerticalBox)
		// TODO : replace these separators with outline in border brush ?
		+ SVerticalBox::Slot()
		.AutoHeight()
		[
			SNew(SSeparator)
			.Orientation(Orient_Horizontal)
			.Thickness(1.f)
			.ColorAndOpacity(FStyleColors::Background)
			.SeparatorImage(FAppStyle::GetBrush( "ThinLine.Horizontal" ))
		]
		+ SVerticalBox::Slot()
		.AutoHeight()
		[
			SNew(SBorder)
			.BorderImage( FAppStyle::Get().GetBrush("DetailsView.CategoryTop"))
			.BorderBackgroundColor(FSlateColor(FLinearColor::White))
			.Padding(0.0f)
			[
				SNew(SBox)
				.MinDesiredHeight(26.0f)
				[
					SNew(SHorizontalBox)
					// TODO add dropdown arrow and functionality
					/*+ SHorizontalBox::Slot()
					.HAlign(HAlign_Left)
					.VAlign(VAlign_Center)
					.Padding(2.0f, 0.0f, 0.0f, 0.0f)
					.AutoWidth()
					[
						SNew(SImage)
						.Visibility(EVisibility::HitTestInvisible)
						.Image( FAppStyle::GetBrush(TEXT("AnimEditor.RefreshButton")) ) // random icon
					]*/
					+ SHorizontalBox::Slot()
					.VAlign(VAlign_Center)
					.Padding(4.0f, 0.0f, 0.0f, 0.0f)
					.FillWidth(1)
					[
						SNew(STextBlock)
						.Text(FText::FromName(CategoryName))
						.Font(FAppStyle::GetFontStyle(TEXT("MenuItem.Font")))
					]
				]
			]
		]
		+ SVerticalBox::Slot()
		.AutoHeight()
		[
			SNew(SSeparator)
			.Orientation(Orient_Horizontal)
			.Thickness(1.f)
			.ColorAndOpacity(FStyleColors::Background)
			.SeparatorImage(FAppStyle::GetBrush( "ThinLine.Horizontal" ))
		];
}
