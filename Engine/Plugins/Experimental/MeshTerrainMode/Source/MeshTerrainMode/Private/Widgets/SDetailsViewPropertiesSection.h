// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Widgets/SCompoundWidget.h"
#include "Widgets/SBoxPanel.h"
#include "Containers/Array.h"
#include "Widgets/SSectionedDetailsViewWidget.h"

namespace UE::MeshTerrain { struct FDetailsCategory; }

namespace UE::MeshTerrain
{
	class SDetailsViewPropertiesSection : public SCompoundWidget
	{
	public:
		SLATE_BEGIN_ARGS(SDetailsViewPropertiesSection) {}
			SLATE_EVENT(FSimpleDelegate, OnSectionedDetailsPanelRequestRebuild)
		SLATE_END_ARGS()

		SDetailsViewPropertiesSection(){ };

		void Construct(const FArguments& InArgs);

		/** Update the currently active section and refresh the widget to reflect it */
		void UpdateActiveSection(const TArray<FDetailsCategory>& NewActiveSection, const TSharedPtr<IDetailsView>& DetailsView);

		/** Return a set of properties that should be currently displayed */
		TSharedPtr<TSet<FProperty*>> GetActiveProperties() const { return ActiveProperties; }
	
	private:
		TSharedPtr<SVerticalBox> Container;
		
		FSimpleDelegate OnSectionedDetailsPanelRequestRebuild;
		
		TSharedPtr<TSet<FProperty*>> ActiveProperties;
		
		// creates a widget representation of the Category Header
		TSharedPtr<SWidget> MakeCategoryHeader(const FName CategoryName);

		// refresh the widget
		void Rebuild();
		void Rebuild(TSharedPtr<IDetailsView> DetailsView);

		// the active section, represented as an array of its categories. Empty if no section is active
		TArray<FDetailsCategory> ActiveSection;
	};
}