// Copyright Epic Games, Inc. All Rights Reserved.

#include "SPlaceableItemEntry.h"
#include "Widgets/Layout/SSpacer.h"
#include "Widgets/Images/SImage.h"
#include "Widgets/Text/STextBlock.h"
#include "Widgets/Layout/SScrollBar.h"
#include "Widgets/Views/SListView.h"
#include "Widgets/Input/SCheckBox.h"
#include "Styling/AppStyle.h"
#include "EditorModeManager.h"
#include "EditorModes.h"
#include "Framework/Application/SlateApplication.h"
#include "AssetThumbnail.h"
#include "LevelEditor.h"
#include "LevelEditorActions.h"
#include "LevelEditorViewport.h"
#include "ContentBrowserDataDragDropOp.h"
#include "EditorClassUtils.h"
#include "Widgets/Input/SSearchBox.h"
#include "ClassIconFinder.h"
#include "Widgets/Docking/SDockTab.h"
#include "ThumbnailRendering/ThumbnailManager.h"
#include "AssetSelection.h"
#include "SAssetDropTarget.h"
#include "ActorFactories/ActorFactory.h"
#include "ScopedTransaction.h"
#include "Layout/CategoryDrivenContentBuilder.h"
#include "ToolkitBuilder.h"
#include "Styles/SlateBrushTemplates.h"
#include "Layout/WidgetPath.h"
#include "Styling/CoreStyle.h"
#include "IDocumentation.h"

#define LOCTEXT_NAMESPACE "SPlaceableItemEntry"

namespace UE::MeshTerrain
{
	
/**
 * These are the asset thumbnails.
 */
class SPlaceableItemThumbnail : public SCompoundWidget
{
public:
	SLATE_BEGIN_ARGS( SPlaceableItemThumbnail )
		: _Width( 32 )
		, _Height( 32 )
		, _AlwaysUseGenericThumbnail( false )
		, _AssetTypeColorOverride()
		, _CustomIconBrush( nullptr )
	{}

	SLATE_ARGUMENT( uint32, Width )

	SLATE_ARGUMENT( uint32, Height )

	SLATE_ARGUMENT( FName, ClassThumbnailBrushOverride )

	SLATE_ARGUMENT( bool, AlwaysUseGenericThumbnail )

	SLATE_ARGUMENT( TOptional<FLinearColor>, AssetTypeColorOverride )

	SLATE_ARGUMENT( const FSlateBrush*, CustomIconBrush )

	SLATE_END_ARGS()

	void Construct( const FArguments& InArgs, const FAssetData& InAsset)
	{
		Asset = InAsset;

		TSharedPtr<FAssetThumbnailPool> ThumbnailPool = UThumbnailManager::Get().GetSharedThumbnailPool();

		Thumbnail = MakeShareable(new FAssetThumbnail(Asset, InArgs._Width, InArgs._Height, ThumbnailPool));
	
		TSharedPtr<SImage> ThumbnailImage;

		// figure out the proper image to show based on whether the asset is a class type
		TWeakObjectPtr<UClass> ThumbnailClass = MakeWeakObjectPtr( const_cast<UClass*>( FClassIconFinder::GetIconClassForAssetData( Asset, &bIsClassType ) ) );
		const FName AssetClassName = Asset.AssetClassPath.GetAssetName();
		const FName DefaultThumbnail = bIsClassType ? NAME_None : FName( *FString::Printf( TEXT("ClassThumbnail.%s"), *AssetClassName.ToString() ) );
		const FSlateBrush* ThumbnailBrush = !InArgs._ClassThumbnailBrushOverride.IsNone() ?
			FClassIconFinder::FindThumbnailForClass( nullptr,  InArgs._ClassThumbnailBrushOverride ) :
			FClassIconFinder::FindThumbnailForClass( ThumbnailClass.Get(), DefaultThumbnail );

		if ( InArgs._CustomIconBrush )
		{
			ThumbnailBrush = InArgs._CustomIconBrush;
		}

		ChildSlot
		[
			SAssignNew( ThumbnailImage, SImage ).Image( ThumbnailBrush )
		];
	}

private:

	FAssetData Asset;
	TSharedPtr< FAssetThumbnail > Thumbnail;

	/** Indicates whether the Asset is a class type */
	bool bIsClassType = false;
};

void SPlaceableItemEntry::Construct(const FArguments& InArgs, const TSharedPtr<const FPlaceableItem>& InItem)
{
	IconSize = InArgs._IconSize.Get();
	Font = InArgs._Font;
	OnGetMenuContent = InArgs._OnGetMenuContent;
	bIsPressed = false;

	Item = InItem;

	TSharedPtr< SHorizontalBox > ActorType = SNew( SHorizontalBox );

	const bool bIsClass = Item->AssetData.GetClass() == UClass::StaticClass();
	const bool bIsActor = bIsClass ? CastChecked<UClass>(Item->AssetData.GetAsset())->IsChildOf(AActor::StaticClass()) : false;

	AActor* DefaultActor = nullptr;
	if (Item->Factory != nullptr)
	{
		DefaultActor = Item->Factory->GetDefaultActor(Item->AssetData);
	}
	else if (bIsActor)
	{
		DefaultActor = CastChecked<AActor>(CastChecked<UClass>(Item->AssetData.GetAsset())->GetDefaultObject(false));
	}

	TSharedPtr<IToolTip> AssetEntryToolTip;
	constexpr bool bItemInternalsInTooltip = false;
	if (bItemInternalsInTooltip)
	{
		AssetEntryToolTip = FSlateApplicationBase::Get().MakeToolTip(
			FText::Format(LOCTEXT("ItemInternalsTooltip", "Native Name: {0}\nAsset Path: {1}\nFactory Class: {2}"), 
			FText::FromString(Item->NativeName), 
			FText::FromString(Item->AssetData.GetObjectPathString()),
			FText::FromString(Item->Factory ? Item->Factory->GetClass()->GetName() : TEXT("None"))));
	}

	UClass* DocClass = nullptr;
	if(DefaultActor != nullptr)
	{
		DocClass = DefaultActor->GetClass();
		if (!AssetEntryToolTip)
		{
			AssetEntryToolTip = FEditorClassUtils::GetTooltip(DefaultActor->GetClass());
		}
	}

	if (!AssetEntryToolTip)
	{
		AssetEntryToolTip = IDocumentation::Get()->CreateToolTip(Item->DisplayName, nullptr, "Shared/Types/AssetEntries", Item->DisplayName.ToString());
	}

	const FButtonStyle& ButtonStyle = FAppStyle::GetWidgetStyle<FButtonStyle>( "PlacementBrowser.Asset" );

	NormalImage = &ButtonStyle.Normal;
	HoverImage = &ButtonStyle.Hovered;
	PressedImage = &ButtonStyle.Pressed;

	float TextFillWidth = 0.99f;
	const FMargin DragHandlePadding{ 0.f,0.f,8.f, 0.f };

	FMargin WholeAssetPadding{ 8.f, 2.f, 12.f, 2.f };
	const FSlateBrush* WholeAssetBackgroundBrush{ FAppStyle::Get().GetBrush( "PlacementBrowser.Asset.Background" ) };
	FMargin ThumbnailBoxPadding{ 8.f ,4.f,8.f, 4.f };
	FMargin AssetTextPadding{ 9, 0, 0, 1 };
	TSharedRef<SWidget> DraggableAssetEndWidget = SNullWidget::NullWidget;

	WholeAssetPadding = 0;
	WholeAssetBackgroundBrush = FAppStyle::Get().GetBrush("PlacementBrowser.Asset.ThumbnailBackground");
	ThumbnailBoxPadding = FMargin{ 4.f,4.f,0.f, 4.f };
	AssetTextPadding = FMargin{ 4, 0, 8, 1 };
	DraggableAssetEndWidget = SNew(SBox)
								.Padding( DragHandlePadding )
								[
									SNew(SImage).Image( FAppStyle::GetBrush("VerticalBoxDragIndicatorShort") )
								];

	const FSlateBrush* CustomIconBrush = nullptr;
	if ( Item->DragHandler.IsValid() && Item->DragHandler->IconBrush )
	{
		CustomIconBrush = Item->DragHandler->IconBrush;
	}

	ChildSlot
	.Padding( WholeAssetPadding )
	[
		SNew(SOverlay)
		+ SOverlay::Slot()
		[
			SNew(SBorder)
			.BorderImage( WholeAssetBackgroundBrush)
			.Cursor( EMouseCursor::GrabHand )
			.ToolTip( AssetEntryToolTip )
			.Padding(0)
			[

				SNew( SHorizontalBox )
				+ SHorizontalBox::Slot()
				.Padding( ThumbnailBoxPadding )
				.AutoWidth()
				.HAlign(HAlign_Center)
				.VAlign(VAlign_Center)
				[
					SNew( SBox )
					.WidthOverride( IconSize.X )
					.HeightOverride( IconSize.Y )
					[
						SNew( SPlaceableItemThumbnail, Item->AssetData )
						.ClassThumbnailBrushOverride( Item->ClassThumbnailBrushOverride )
						.AlwaysUseGenericThumbnail( Item->bAlwaysUseGenericThumbnail )
						.AssetTypeColorOverride( FLinearColor::Transparent )
						.CustomIconBrush( CustomIconBrush )
					]
				]
				+ SHorizontalBox::Slot()
				.VAlign(VAlign_Fill)
				.Padding(0)
				[
					SNew(SBorder)
					.BorderImage(FAppStyle::Get().GetBrush("PlacementBrowser.Asset.LabelBack"))
					[
						SNew( SHorizontalBox)
						+SHorizontalBox::Slot()	
						.FillContentWidth( TextFillWidth )
						.Padding( AssetTextPadding )
						.VAlign(VAlign_Center)
						[
							SNew( STextBlock )
							.TextStyle( FAppStyle::Get(), "PlacementBrowser.Asset.Name" )
							.Text( Item->DisplayName )
							.Font(Font)
							.OverflowPolicy( ETextOverflowPolicy::Ellipsis )
							.HighlightText(  InArgs._HighlightText )
						]
						+ SHorizontalBox::Slot()
							.VAlign( VAlign_Center )
							.AutoWidth()
							[
								DraggableAssetEndWidget
							]
					]
				]
			]
		]
		+ SOverlay::Slot()
		[
			SNew(SBorder)
			.BorderImage( this, &SPlaceableItemEntry::GetBorder )
			.Cursor( EMouseCursor::GrabHand )
			.ToolTip( AssetEntryToolTip )
		]
	];
}

FReply SPlaceableItemEntry::OnMouseButtonDown(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent)
{
	if ( MouseEvent.GetEffectingButton() == EKeys::LeftMouseButton )
	{
		bIsPressed = true;
		return FReply::Handled().DetectDrag( SharedThis( this ), MouseEvent.GetEffectingButton() );
	}

	// Create the context menu to be launched on right mouse click. 
	if ( MouseEvent.GetEffectingButton() == EKeys::RightMouseButton )
	{
		FWidgetPath WidgetPath = MouseEvent.GetEventPath() != nullptr ? *MouseEvent.GetEventPath() : FWidgetPath();
		FSlateApplication::Get().PushMenu(
			AsShared(),
			WidgetPath,
			OnGetMenuContent.IsBound() ? OnGetMenuContent.Execute() : SNullWidget::NullWidget,
			MouseEvent.GetScreenSpacePosition(),
			FPopupTransitionEffect( FPopupTransitionEffect::ContextMenu )
			);
		return FReply::Handled();
	}
	return FReply::Unhandled();
}

FReply SPlaceableItemEntry::OnMouseButtonUp(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent)
{
	if ( MouseEvent.GetEffectingButton() == EKeys::LeftMouseButton )
	{
		bIsPressed = false;
	}

	return FReply::Unhandled();
}

FReply SPlaceableItemEntry::OnDragDetected(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent)
{
	bIsPressed = false;

	if (FEditorDelegates::OnAssetDragStarted.IsBound())
	{
		TArray<FAssetData> DraggedAssetDatas;
		DraggedAssetDatas.Add( Item->AssetData );
		FEditorDelegates::OnAssetDragStarted.Broadcast( DraggedAssetDatas, Item->Factory );
		return FReply::Handled();
	}

	if ( MouseEvent.IsMouseButtonDown( EKeys::LeftMouseButton ) )
	{
		if ( Item->DragHandler.IsValid() && Item->DragHandler->GetContentToDrag.IsBound() )
		{
			return FReply::Handled().BeginDragDrop( Item->DragHandler->GetContentToDrag.Execute() );
		}
		return FReply::Handled().BeginDragDrop(FAssetDragDropOp::New(Item->AssetData, Item->AssetFactory));
	}
	return FReply::Handled();
}

bool SPlaceableItemEntry::IsPressed() const
{
	return bIsPressed;
}

const FSlateBrush* SPlaceableItemEntry::GetBorder() const
{
	return IsPressed() ? PressedImage : IsHovered() ? HoverImage : NormalImage;
}
	
} // namespace UE::MeshTerrain

#undef LOCTEXT_NAMESPACE
