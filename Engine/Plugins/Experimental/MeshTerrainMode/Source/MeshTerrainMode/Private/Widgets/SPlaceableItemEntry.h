// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreMinimal.h"
#include "Layout/Visibility.h"
#include "Input/Reply.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/SWidget.h"
#include "Widgets/SCompoundWidget.h"
//#include "ActorPlacementInfo.h"
#include "IPlacementModeModule.h"
#include "Widgets/Layout/SUniformWrapPanel.h"
#include "Widgets/Views/STableViewBase.h"
#include "Widgets/Views/STableRow.h"
#include "Misc/TextFilter.h"
#include "Fonts/SlateFontInfo.h"

class STextBlock;
class FCategoryDrivenContentBuilder;

// Derived from SPlacementAssetEntry in PlacementModeTools.h
// TODO: Consider exposing SPlacementAssetEntry from PlacementMode directly.

namespace UE::MeshTerrain
{
	
/**
 * A tile representation of the class or the asset.  These are embedded into the views inside
 * of each tab.
 */
class SPlaceableItemEntry : public SCompoundWidget
{
public:
	SLATE_BEGIN_ARGS(SPlaceableItemEntry){}
	SLATE_ATTRIBUTE(FVector2D, IconSize)
	/** Highlight this text in the text block */
	SLATE_ATTRIBUTE(FText, HighlightText)
	SLATE_ATTRIBUTE(FSlateFontInfo, Font)
	SLATE_EVENT( FOnGetContent, OnGetMenuContent )
	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs, const TSharedPtr<const FPlaceableItem>& InItem);

	virtual FReply OnMouseButtonDown(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent) override;
	virtual FReply OnMouseButtonUp(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent) override;
	virtual FReply OnDragDetected(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent) override;

	bool IsPressed() const;

	TSharedPtr<const FPlaceableItem> Item;

private:

	/** Delegate to execute to get the menu content of this button */
	FOnGetContent OnGetMenuContent;

	const FSlateBrush* GetBorder() const;

	bool bIsPressed = false;

	/** Brush resource that represents a button */
	const FSlateBrush* NormalImage = nullptr;
	/** Brush resource that represents a button when it is hovered */
	const FSlateBrush* HoverImage = nullptr;
	/** Brush resource that represents a button when it is pressed */
	const FSlateBrush* PressedImage = nullptr;

	FVector2D IconSize = FVector2D(20.0,20.0);

	TAttribute<FSlateFontInfo> Font;
};
	
}
	