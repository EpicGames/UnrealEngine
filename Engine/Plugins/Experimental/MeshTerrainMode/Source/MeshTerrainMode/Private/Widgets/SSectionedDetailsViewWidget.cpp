// Copyright Epic Games, Inc. All Rights Reserved.

#include "Widgets/SSectionedDetailsViewWidget.h"

#include "InteractiveTool.h"
#include "Widgets/Images/SImage.h"
#include "Widgets/Input/SCheckBox.h"
#include "Widgets/Layout/SBox.h"
#include "Widgets/Text/STextBlock.h"
#include "MeshTerrainDetailsSections.h"
#include "MeshTerrainModeStyle.h"
#include "Widgets/Layout/SSeparator.h"
#include "ObjectEditorUtils.h"
#include "Widgets/SDetailsViewPropertiesSection.h"
#include "Styling/StyleColors.h"

#define LOCTEXT_NAMESPACE "SectionedDetailsViewWidget"
DEFINE_LOG_CATEGORY(LogMeshTerrain);

void UE::MeshTerrain::SSectionedDetailsViewWidget::Construct(const FArguments& InArgs)
{
	this->ToolShutdownButtons = InArgs._ToolShutdownButtons;
	this->CloseDetailsViewButton = InArgs._CloseDetailsViewButton;
	
	ChildSlot
	.VAlign(VAlign_Fill)
	.HAlign(HAlign_Fill)
	[
		SAssignNew(Container, SVerticalBox)
	];
}

void UE::MeshTerrain::SSectionedDetailsViewWidget::SetActiveTool(UInteractiveTool* Tool, const FSlateBrush* ActiveToolIcon, FText ActiveToolName, bool bDisplayToolShutdownButtons)
{
	// reset widget
	Container->ClearChildren();
	AllSections.Empty();
	ActiveSection = NAME_None;
	
	if (!ToolShutdownButtons.IsValid())
	{
		ToolShutdownButtons = SNullWidget::NullWidget;
	}

	if (!CloseDetailsViewButton.IsValid())
	{
		CloseDetailsViewButton = SNullWidget::NullWidget;
	}

	SHorizontalBox::FArguments HeaderArgs;

	HeaderArgs
	+SHorizontalBox::Slot()
	.AutoWidth()
	.FillWidth(1.0f)
	.HAlign(HAlign_Left)
	[
		SNew(SHorizontalBox)
		+SHorizontalBox::Slot()
		.AutoWidth()
		.HAlign(HAlign_Left)
		.VAlign(VAlign_Center)
		[
			SNew(SImage)
			.Image(ActiveToolIcon)
			.DesiredSizeOverride(FVector2D(20, 20))
			.ColorAndOpacity(FSlateColor::UseForeground())
		]
		+SHorizontalBox::Slot()
		.HAlign(HAlign_Left)
		.VAlign(VAlign_Center)
		.Padding(4.f, 0.f, 4.f, 0.f)
		[
			SNew(STextBlock)
			.Text(ActiveToolName)
			.OverflowPolicy(ETextOverflowPolicy::Ellipsis)
			.Clipping(EWidgetClipping::ClipToBounds)
			.Font(FCoreStyle::GetDefaultFontStyle("Regular", 12))
		]
	];

	if (bDisplayToolShutdownButtons)
	{
		HeaderArgs
		+SHorizontalBox::Slot()
		.AutoWidth()
		.HAlign(HAlign_Right)
		.VAlign(VAlign_Center)
		[
			ToolShutdownButtons.ToSharedRef()
		];
	}
	else
	{
		HeaderArgs
		+SHorizontalBox::Slot()
		.AutoWidth()
		.HAlign(HAlign_Right)
		.VAlign(VAlign_Center)
		[
			CloseDetailsViewButton.ToSharedRef()
		];
	}
	
	Container->AddSlot()
	.HAlign(HAlign_Fill)
	.VAlign(VAlign_Fill)
	.AutoHeight()
	.Padding(8.f, 4.f, 2.f, 4.f)
	[
		SArgumentNew(HeaderArgs, SHorizontalBox)
	];
	Container->AddSlot()
	.MaxHeight(1.0f)
	.Padding(0.f)
	.HAlign(HAlign_Fill)
	.VAlign(VAlign_Fill)
	[
		SNew(SSeparator)
		.Orientation(Orient_Horizontal)
		.Thickness(1.f)
		.ColorAndOpacity(FStyleColors::Background)
		.SeparatorImage(FAppStyle::GetBrush( "ThinLine.Horizontal" ))
	];
	
	// retrieves the tool's properties and continues to build the widget based on them
	const TArray<UObject*> CurToolProperties = Tool->GetToolProperties(true);
	
	// iterate through all the tool's property sets
	for (UObject* PropertySet : CurToolProperties)
	{
		TMap<FName, TArray<FPropertyDetails>> CatToProps;
		
		// iterate through all properties within a set
		for (TFieldIterator<FProperty> PropIt(PropertySet->GetClass()); PropIt; ++PropIt)
		{
			FProperty* Property = *PropIt;
			FName CatName = FObjectEditorUtils::GetCategoryFName(Property);

			// group all properties based on their category
			if (CatToProps.Contains(CatName))
			{
				CatToProps.Find(CatName)->Add(FPropertyDetails(Property, PropertySet));
			}
			else
			{
				TArray<FPropertyDetails> CatProps;
				CatProps.Add(FPropertyDetails(Property, PropertySet));
				CatToProps.Add(CatName,CatProps);
			}
		}

		// map of Category Name to its Section (tab) Name
		FMeshTerrainDetailsSections* SectionsInfo = FMeshTerrainDetailsSections::Get();
		FName PropSetName = FName(PropertySet->GetFName().GetPlainNameString());
		TMap<FName,FName> SectionMap = SectionsInfo->GetSectionMappings(PropSetName);

		// map of all categories that are specified to be under a specific tab
		TArray<FName> DefinedCategoryNames;
		SectionMap.GenerateKeyArray(DefinedCategoryNames);

		// iterate through each category in the property set
		for (TMap<FName, TArray<FPropertyDetails>>::TConstIterator CategoryIterator(CatToProps); CategoryIterator; ++CategoryIterator)
		{
			FDetailsCategory NewCategory = FDetailsCategory(CategoryIterator.Key(), CategoryIterator.Value());

			// add the category to the AllCategories array, which will be used for the 'All Sections' tab
			AllCategories.Add(NewCategory);

			// checking that the category is one that has been registered
			if (DefinedCategoryNames.Contains(CategoryIterator.Key()))
			{
				FName CategoryName = NewCategory.CategoryName;
				FName TabName = *SectionMap.Find(CategoryName);

				// if the Tab/Section does not exist yet, create it and add this category under it
				if (!AllSections.Contains(TabName))
				{
					TArray<FDetailsCategory> CategoryArr;
					CategoryArr.Add(NewCategory);
					AllSections.Add(TabName, CategoryArr);
				}
				// if the Tab/Section already exists, add this category as one to be displayed under it
				else
				{
					AllSections.Find(TabName)->Add(NewCategory);
				}
			}
		}
		if (SectionMap.Num() == 0)
		{
			UE_LOGF(LogMeshTerrain, Warning, "Property Set '%ls' was not found in Section mappings.", *PropSetName.ToString());
		}
	}

	// add the 'All' tab at the end, using an array of all the categories for the tool
	// this will display every category, even if they have not been defined to be under a specific tab
	if (AllSections.Num() != 1)
	{
		AllSections.Add(FName("ALL_SECTIONS"), AllCategories);
	}

	// build the toolbar which contains buttons to activate each section, along with an 'All' button which displays all the tool's categories
	SHorizontalBox::FArguments SectionsToolbar;
	
	for (TMap<FName, TArray<FDetailsCategory>>::TConstIterator SectionIt(AllSections); SectionIt; ++SectionIt)
	{
		const FName SectionName = SectionIt.Key();
		TFunction<ECheckBoxState()> IsCheckedFunc = [this, SectionName]()
		{
			if (ActiveSection == SectionName)
			{
				return ECheckBoxState::Checked;
			}
			return ECheckBoxState::Unchecked;
		};
		TFunction<void(const ECheckBoxState)> OnCheckStateChangedFunc = [this, SectionName](const ECheckBoxState NewState)
		{
			if (ActiveSection == SectionName)
			{
				ActiveSection = NAME_None;
			}
			else
			{
				ActiveSection = SectionName;
			}
			RebuildPropertiesSection();
		};

		// build the tab button for the section
		SectionsToolbar
		+ SHorizontalBox::Slot()
		.AutoWidth()
		.Padding(4.0f, 0.0f, 4.0f, 0.0f)
		[
			SNew(SCheckBox)
			.Style(FMeshTerrainModeStyle::Get().Get(), "DetailsViewTabButton")
			.HAlign(HAlign_Center)
			.Padding(0.f)
			.Style(FMeshTerrainModeStyle::Get().Get(), "DetailsViewToggleButton")
			.IsChecked_Lambda(MoveTemp(IsCheckedFunc))
			.OnCheckStateChanged_Lambda(MoveTemp(OnCheckStateChangedFunc))
			[
				SNew(SVerticalBox)
				+ SVerticalBox::Slot()
				.VAlign(VAlign_Center)
				.HAlign(HAlign_Center)
				.Padding(4.0)
				.AutoHeight()
				[
					SNew(SImage)
					.Visibility(EVisibility::HitTestInvisible)
					.Image(FMeshTerrainModeStyle::Get()->GetBrush( FName(TEXT("SectionHeader.") + SectionIt.Key().ToString())))
					.DesiredSizeOverride(FVector2D(16, 16))
					.ColorAndOpacity(FSlateColor::UseForeground())
				]
			]
		];
	}

	Container->AddSlot()
	.AutoHeight()
	.HAlign(HAlign_Left)
	.VAlign(VAlign_Fill)
	.Padding(4.0f)
	[
		SArgumentNew(SectionsToolbar, SHorizontalBox)
	];
	
	Container->AddSlot()
	.AutoHeight()
	.HAlign(HAlign_Fill)
	.VAlign(VAlign_Fill)
	[
		SAssignNew(PropertiesSection, SDetailsViewPropertiesSection)
		.OnSectionedDetailsPanelRequestRebuild(OnSectionedDetailsPanelRequestRebuild)
	];
	RebuildPropertiesSection();
}

TSharedPtr<TSet<FProperty*>> UE::MeshTerrain::SSectionedDetailsViewWidget::GetActiveProperties() const
{
	if (!PropertiesSection)
	{
		return nullptr;
	}
	return PropertiesSection->GetActiveProperties();
}

void UE::MeshTerrain::SSectionedDetailsViewWidget::RebuildPropertiesSection()
{
	TArray<FDetailsCategory> ActiveCategories;
	if (ActiveSection == NAME_None)
	{
		ActiveCategories = TArray<FDetailsCategory>();
	}
	else
	{
		ActiveCategories = *AllSections.Find(ActiveSection);
	}
	
	PropertiesSection->UpdateActiveSection(ActiveCategories, DetailsView);
}

#undef LOCTEXT_NAMESPACE