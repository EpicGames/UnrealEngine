// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Widgets/SCompoundWidget.h"
#include "Widgets/SBoxPanel.h"

class UInteractiveTool;
class IDetailsView;
struct FArguments;
namespace UE::MeshTerrain { class SDetailsViewPropertiesSection; }

namespace UE::MeshTerrain
{
	// stores information about a property to be represented in the Widget
	struct FPropertyDetails
	{
		FProperty* Property;
		UObject* PropOwner;
	};

	// stores information about a Category and the Properties that fall under it
	struct FDetailsCategory
	{
		FName CategoryName;
		TArray<FPropertyDetails> Properties;
	};

	// stores information about a Section (tab) and the categories to be included within it
	struct FDetailsSection
	{
		FName SectionName;
		TArray<FDetailsCategory> Categories;
	};

	// SSectionedDetailsViewWidget
	class SSectionedDetailsViewWidget : public SCompoundWidget
	{
	public:
		SLATE_BEGIN_ARGS( SSectionedDetailsViewWidget ){}
		SLATE_ARGUMENT(TSharedPtr<SWidget>, ToolShutdownButtons)
		SLATE_ARGUMENT(TSharedPtr<SWidget>, CloseDetailsViewButton)
		SLATE_END_ARGS();

		SSectionedDetailsViewWidget(){ };

		void Construct(const FArguments& InArgs);

		/** Builds the SSectionedDetailsView widget based on the active tool & its properties */
		void SetActiveTool(UInteractiveTool* Tool, const FSlateBrush* ActiveToolIcon, FText ActiveToolName, bool bDisplayToolShutdownButtons = true);

		/** Sets the detail view to be used in the PropertiesSection, with properties filtered accordingly */
		void SetDetailsView(const TSharedPtr<IDetailsView>& InDetailsView) { DetailsView = InDetailsView; }

		/** Return a set of properties which are 'active' and should be displayed. A property is 'active' if it falls
		 * under a category that is currently 'active' (or all categories are active)*/
		TSharedPtr<TSet<FProperty*>> GetActiveProperties() const;

		FSimpleDelegate OnSectionedDetailsPanelRequestRebuild;
		TSharedPtr<SVerticalBox> Container;

	private:
		/** The section of this widget which contains the categories and their properties */
		TSharedPtr<SDetailsViewPropertiesSection> PropertiesSection;
		/** Force a rebuild of this Properties section, when tool or active category changes */
		void RebuildPropertiesSection();

		/** Widget which contains tool shutdown buttons (accept, cancel, complete) */
		TSharedPtr<SWidget> ToolShutdownButtons;

		/** Widget which contains details view 'close' button */
		TSharedPtr<SWidget> CloseDetailsViewButton;

		/** The details view to be displayed in the properties section, displaying only the active properties */
		TSharedPtr<IDetailsView> DetailsView = nullptr;

		/** The name of the active sections/tabs to be displayed. 'None' if no tabs are active, 'ALL_SECTIONS' if all */
		FName ActiveSection;

		/** Keep an array of all the categories, represented as FDetailsCategory, for the case of the 'all sections' tab */
		TArray<FDetailsCategory> AllCategories;
		
		/** contains information about all sections/tabs that have been registered for each of the current tool's property sets */
		TMap<FName, TArray<FDetailsCategory>> AllSections;
	};
}

DECLARE_LOG_CATEGORY_EXTERN(LogMeshTerrain, Warning, All);


