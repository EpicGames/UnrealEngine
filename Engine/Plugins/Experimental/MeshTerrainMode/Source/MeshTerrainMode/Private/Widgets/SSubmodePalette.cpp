// Copyright Epic Games, Inc. All Rights Reserved.

#include "Widgets/SSubmodePalette.h"

#include "Widgets/SBoxPanel.h"
#include "Widgets/Layout/SScrollBox.h"
#include "Widgets/Layout/SBox.h"
#include "Widgets/Layout/SSeparator.h"
#include "Widgets/Text/STextBlock.h"
#include "Widgets/Images/SImage.h"
#include "Widgets/Input/SCheckBox.h"
#include "Submodes/Submode.h"
#include "MeshTerrainModeStyle.h"
#include "Styling/StyleColors.h"
#include "Framework/Commands/UICommandList.h"

namespace UE::MeshTerrain
{
	// where should the Submode Label be displayed?
	static TAutoConsoleVariable<int32> CVarSubmodePaletteLabelLocation(
	TEXT("MeshTerrainMode.SubmodePaletteLabelLocation"),
	0,
	TEXT("0 - right of icon ; 1 - below icon"));
}

using namespace UE::MeshTerrain;

// SSubmodePalette - represents both the submode and tool palettes

void SSubmodePalette::Construct(const FArguments& InArgs)
{
	if (InArgs._Width.IsSet())
	{
		this->Width = InArgs._Width;
	}
	this->ToolPanelFont = InArgs._ToolPanelFont;
	
	ChildSlot
	.HAlign(HAlign_Left)
	[
		SNew(SBox)
		.WidthOverride(Width)
		[
			SNew(SHorizontalBox)
			+ SHorizontalBox::Slot()
			.AutoWidth()
			.Padding(FMargin(0.0f, 0.0f, 4.0f, 0.0f))
			[
				SNew(SVerticalBox)
				+ SVerticalBox::Slot()
				.AutoHeight()
				[
					SNew(SBorder)
					.BorderImage(FAppStyle::Get().GetBrush("ProjectBrowser.ProjectTile.DropShadow"))
					.Padding(FMargin(0.0f, 0.0f, 0.0f, 4.0f))
					.HAlign(HAlign_Fill)
					[
						SNew(SVerticalBox)
						+ SVerticalBox::Slot()
						.AutoHeight()
						[
							SAssignNew(SubmodeContainer, SSubmodesPanel)
							.LabelVisibility(EVisibility::Collapsed)
						]
					]
				]
			]
			+SHorizontalBox::Slot()
			.HAlign(HAlign_Fill)
			.Padding(FMargin(4.0f, 0.0f, 0.0f, 0.0f))
			.FillContentWidth(1.0f)
			[
				SNew(SVerticalBox)
				+ SVerticalBox::Slot()
				.AutoHeight()
				[
					SNew(SBorder)
					.BorderImage(FAppStyle::Get().GetBrush("ProjectBrowser.ProjectTile.DropShadow"))
					.Padding(FMargin(0.0f, 0.0f, 0.0f, 4.0f))
					.HAlign(HAlign_Fill)
					.VAlign(VAlign_Top)
					[
						SNew(SHorizontalBox)
						+ SHorizontalBox::Slot()
						.HAlign(HAlign_Fill)
						.FillContentWidth(1.0f)
						[
							SAssignNew(ToolContainer, SSubmodeToolPanel)
							.Font(ToolPanelFont)
						]
					]
				]
			]
		]
	];

	ActivePaletteChangedHandle = SubmodeContainer->OnSubmodePaletteChanged.AddLambda([this]()
	{
		ToolContainer->SetActiveSubmode(SubmodeContainer->GetActiveSubmodePtr());
	});
}

// -----------------------------------------------------------------------------------------------------------

// represents palette of submodes only

void SSubmodesPanel::Construct(const FArguments& InArgs)
{
	this->LabelVisibility = InArgs._LabelVisibility;

	SubmodePaletteCommandList = MakeShareable(new FUICommandList);
	ChildSlot
	.HAlign(HAlign_Center)
	[
		SNew(SBorder)
		.BorderImage( FMeshTerrainModeStyle::Get()->GetBrush("MeshTerrainMode.SubmodePaletteLighterBrush"))
		[
			SNew(SVerticalBox)
			+ SVerticalBox::Slot()
			.AutoHeight()
			.Padding(4.f)
			[
				SAssignNew(SubmodeContainer, SVerticalBox)
			]
		]
	];
}

void SSubmodesPanel::CreateSubmodesPalette()
{
	SubmodeContainer->ClearChildren();

	// arguments that will encompass all the submode buttons
	SVerticalBox::FArguments SubmodesArgs;

	// add a button for each submode
	for (const TSharedPtr<FSubmode>& Submode : SubmodeList)
	{
		TSharedRef<SImage> SubmodeIconWidget =
			SNew(SImage)
			.Visibility(EVisibility::HitTestInvisible)
			.Image(Submode->GetEnterSubmodeAction()->GetIcon().GetIcon())
			.ColorAndOpacity(FSlateColor::UseForeground())
			.DesiredSizeOverride(FVector2D(20, 20));

		TSharedRef<STextBlock> SubmodeTextWidget =
			SNew(STextBlock)
			.OverflowPolicy(ETextOverflowPolicy::Ellipsis)
			.Text(Submode->GetEnterSubmodeAction()->GetLabel());

		SBox::FArguments SubmodesBoxArgs;
		
		// if submode label text should be to the RIGHT of its icon
		if (UE::MeshTerrain::CVarSubmodePaletteLabelLocation.GetValueOnGameThread() == 0)
		{
			SHorizontalBox::FArguments SubmodeHorizontalBoxArgs;
			SubmodeHorizontalBoxArgs
			+ SHorizontalBox::Slot()
			.VAlign(VAlign_Center)
			.HAlign(HAlign_Center)
			.AutoWidth()
			[
				SubmodeIconWidget
			];

			if (LabelVisibility.IsVisible())
			{
				SubmodeHorizontalBoxArgs
				+ SHorizontalBox::Slot()
				.VAlign(VAlign_Center)
				.HAlign(HAlign_Left)
				.Padding(4.0f, 0.0f, 0.0f, 0.0f)
				.FillWidth(1.0f)
				[
					SubmodeTextWidget
				];
			}

			SubmodesBoxArgs
			.Content()
			[
				SArgumentNew(SubmodeHorizontalBoxArgs, SHorizontalBox)
			];
		}
		// if submode label text should be BELOW its icon
		else if (UE::MeshTerrain::CVarSubmodePaletteLabelLocation.GetValueOnGameThread() == 1)
		{
			SVerticalBox::FArguments SubmodeVerticalBoxArgs;
			SubmodeVerticalBoxArgs
			+ SVerticalBox::Slot()
			.HAlign(HAlign_Center)
			.AutoHeight()
			[
				SubmodeIconWidget
			];

			if (LabelVisibility.IsVisible())
			{
				SubmodeVerticalBoxArgs
				+ SVerticalBox::Slot()
				.HAlign(HAlign_Center)
				.AutoHeight()
				[
					SubmodeTextWidget
				];
			}

			SubmodesBoxArgs
			.Content()
			[
				SArgumentNew(SubmodeVerticalBoxArgs, SVerticalBox)
			];
		}

		TFunction<ECheckBoxState()> IsCheckedFunc = [this, &Submode]()
		{
			if (ActiveSubmodePtr)
			{
				return SubmodePaletteCommandList->GetCheckState(Submode->GetEnterSubmodeAction().ToSharedRef());
			}
			return ECheckBoxState::Unchecked;
		};

		TFunction<bool()> IsEnabledFunc = [this, &Submode]()
		{
			TSharedPtr<FUICommandInfo> EnterSubmodeAction = Submode->GetEnterSubmodeAction();
			if (ActiveSubmodePtr)
			{
				bool bEnabled = true;
				if( SubmodePaletteCommandList.IsValid() && EnterSubmodeAction.IsValid() )
				{
					bEnabled = SubmodePaletteCommandList->CanExecuteAction( EnterSubmodeAction.ToSharedRef() );
				}
				return bEnabled;
			}
			return false;
		};

		TFunction<void(const ECheckBoxState)> OnCheckStateChangedFunc = [this, &Submode](const ECheckBoxState State)
		{
			TSharedPtr<FUICommandInfo> EnterSubmodeAction = Submode->GetEnterSubmodeAction();
			if( SubmodePaletteCommandList.IsValid() && EnterSubmodeAction.IsValid() )
			{
				SubmodePaletteCommandList->ExecuteAction( EnterSubmodeAction.ToSharedRef() );
			}
			OnSubmodePaletteChanged.Broadcast();
		};

		TSharedPtr<SCheckBox> CheckBox;

		if (LabelVisibility.IsVisible())
		{
			// when icon only, center the checkbox content
			CheckBox = SNew(SCheckBox)
				.Padding(4.f)
				.Style(FMeshTerrainModeStyle::Get().Get(), "SubmodePaletteToggleButton")
				.ToolTipText(Submode->GetEnterSubmodeAction()->GetDescription())
				.HAlign(HAlign_Left) 
				.IsChecked_Lambda(MoveTemp(IsCheckedFunc))
				.IsEnabled_Lambda(MoveTemp(IsEnabledFunc))
				.OnCheckStateChanged_Lambda(MoveTemp(OnCheckStateChangedFunc))
				[
					SArgumentNew(SubmodesBoxArgs, SBox)
				];
		}
		else
		{
			// when icon and text, align to left
			CheckBox = SNew(SCheckBox)
				.Padding(4.f)
				.Style(FMeshTerrainModeStyle::Get().Get(), "SubmodePaletteToggleButton")
				.ToolTipText(Submode->GetEnterSubmodeAction()->GetDescription())
				.HAlign(HAlign_Center) 
				.IsChecked_Lambda(MoveTemp(IsCheckedFunc))
				.IsEnabled_Lambda(MoveTemp(IsEnabledFunc))
				.OnCheckStateChanged_Lambda(MoveTemp(OnCheckStateChangedFunc))
				[
					SArgumentNew(SubmodesBoxArgs, SBox)
				];
		}

		SubmodesArgs
		+ SVerticalBox::Slot()
		.Padding(4.0f)
		.AutoHeight()
		[
			CheckBox.ToSharedRef()
		];
	}

	// adds the submode buttons to the SubmodeContainer
	SubmodeContainer->AddSlot()
	.Padding(0.f)
	.HAlign(HAlign_Fill)
	[
		SArgumentNew(SubmodesArgs, SVerticalBox)
	];
}

void SSubmodesPanel::AddSubmode(const TSharedPtr<FSubmode>& SubmodePtr)
{
	if (SubmodePtr)
	{
		SubmodeList.Add(SubmodePtr);
	}
}

const TSharedPtr<FUICommandList>& SSubmodesPanel::GetSubmodePaletteCommandList()
{
	return SubmodePaletteCommandList;
}

void SSubmodesPanel::EnterSubmode(const TSharedPtr<FSubmode>& SubmodePtr)
{
	// Ensure we only accept submodes registered to this widget
	if (SubmodeList.Find(SubmodePtr) != INDEX_NONE)
	{
		ActiveSubmodePtr = SubmodePtr;
		OnSubmodePaletteChanged.Broadcast();
	}
}

// ---------------------------------------------------------------------------------

// represents the palette of tools only

void SSubmodeToolPanel::Construct(const FArguments& InArgs)
{
	if (InArgs._Width.IsSet())
	{
		this->Width = InArgs._Width;
	}
	this->DesiredLabelVisibility = InArgs._LabelVisibility;
	this->Font = InArgs._Font;

	LabelVisibility = DesiredLabelVisibility;
	
	ChildSlot
	.HAlign(HAlign_Fill)
	.VAlign(VAlign_Top)
	[
		SNew(SBorder)
		.BorderImage( FMeshTerrainModeStyle::Get()->GetBrush("MeshTerrainMode.SubmodePaletteDarkerBrush"))
		[
			SNew(SBox)
			.WidthOverride(Width)
			[
				SNew(SScrollBox)
				.Orientation(Orient_Vertical)
				.ConsumeMouseWheel(EConsumeMouseWheel::Always)
				.ScrollBarThickness(FVector2D(3.0f, 3.0f))
				.ScrollBarPadding(0.5f)
				+ SScrollBox::Slot()
				[
					SNew(SVerticalBox)
					+ SVerticalBox::Slot()
					.AutoHeight()
					.Padding(4.f)
					[
						SAssignNew(ToolContainer, SVerticalBox)
					]
				]
			]
		]
	];
}

void SSubmodeToolPanel::SetToolCommandList(const TSharedRef<FUICommandList>& CommandList)
{
	ToolCommandList = CommandList;
}

EVisibility SSubmodeToolPanel::GetLabelVisibility() const
{
	return LabelVisibility;
}

void SSubmodeToolPanel::SetLabelVisibility(EVisibility Vis)
{
	LabelVisibility = Vis;
	UpdateToolsPalette();
}

void SSubmodeToolPanel::UpdateToolsPalette()
{
	ToolContainer->ClearChildren();

	if (ensure(ActiveSubmodePtr))
	{
		if (TSharedPtr<SWidget> ToolHeader = ActiveSubmodePtr->GetToolPaletteHeader())
		{
			ToolContainer->AddSlot()
			.HAlign(HAlign_Fill)
			.Padding(0.f)
			.AutoHeight()
			[
				SNew(SVerticalBox)
				+ SVerticalBox::Slot()
				.Padding(4.0f)
				.AutoHeight()
				[
					ToolHeader.ToSharedRef()
				]
			];
		}
		
		const TArray<FSubmodeToolPalette>& SubmodePalettes = ActiveSubmodePtr->GetToolPalettes();
		for (int SPaletteIndex = 0; SPaletteIndex < SubmodePalettes.Num(); SPaletteIndex++)
		{
			const FSubmodeToolPalette& Palette = SubmodePalettes[SPaletteIndex];
			SVerticalBox::FArguments ToolsArgs;

			// iterate through each tool in a given palette
			for (const TSharedPtr<FUICommandInfo>& Command : Palette.ToolCommands)
			{
				SHorizontalBox::FArguments ToolDisplayHBox;

				// add the icon to the horizontal box
				ToolDisplayHBox
				+ SHorizontalBox::Slot()
				.VAlign(VAlign_Center)
				.HAlign(HAlign_Left)
				.AutoWidth()
				[
					// tool icon
					SNew(SImage)
					.Visibility(EVisibility::HitTestInvisible)
					.Image(Command->GetIcon().GetIcon())
					.DesiredSizeOverride(FVector2D(20, 20))
					.ColorAndOpacity(FSlateColor::UseForeground())
				];

				// if we want to display the tool labels, add the text widget to the horizontal box
				if (IsLabelVisible())
				{
					ToolDisplayHBox
					+ SHorizontalBox::Slot()
					.VAlign(VAlign_Center)
					.HAlign(HAlign_Fill)
					.Padding(4.0f, 0.0f, 0.0f, 0.0f)
					.FillWidth(1.0f)
					[
						// tool text
						SNew(STextBlock)
						.OverflowPolicy(ETextOverflowPolicy::Clip)
						.Text(Command->GetLabel())
						.Font(Font)
					];
				}

				// button IsChecked func
				TFunction<ECheckBoxState()> IsCheckedFunc = [this, &Command]()
				{
					if (ToolCommandList.IsValid())
					{
						return ToolCommandList->GetCheckState(Command.ToSharedRef());
					}
					return ECheckBoxState::Unchecked;
				};

				// button IsEnabled func
				TFunction<bool()> IsEnabledFunc = [this, &Command]()
				{
					bool bEnabled = true;
					if( ToolCommandList.IsValid() && Command.IsValid() )
					{
						bEnabled = ToolCommandList->CanExecuteAction( Command.ToSharedRef() );
					}
					return bEnabled;
				};

				// button OnCheckStateChanged func
				TFunction<void(const ECheckBoxState)> OnCheckStateChangedFunc = [this, &Command](const ECheckBoxState NewState)
				{
					if(ToolCommandList.IsValid() && Command.IsValid())
					{
						ToolCommandList->ExecuteAction( Command.ToSharedRef() );
					}
				};

				TSharedPtr<SCheckBox> CheckBox;

				if (IsLabelVisible())
				{
					// when icon and text, align to the left
					CheckBox = SNew(SCheckBox)
						.Padding(4.f)
						.ToolTipText(Command->GetDescription())
						.Style(FMeshTerrainModeStyle::Get().Get(), "ToolPaletteToggleButton")
						.HAlign(HAlign_Left)
						.IsChecked_Lambda(MoveTemp(IsCheckedFunc))
						.IsEnabled_Lambda(MoveTemp(IsEnabledFunc))
						.OnCheckStateChanged_Lambda(MoveTemp(OnCheckStateChangedFunc))
						[
							SArgumentNew(ToolDisplayHBox, SHorizontalBox)
						];
				}
				else
				{
					// when icon only, center check box content
					CheckBox = SNew(SCheckBox)
						.Padding(4.f)
						.ToolTipText(Command->GetDescription())
						.Style(FMeshTerrainModeStyle::Get().Get(), "ToolPaletteToggleButton")
						.HAlign(HAlign_Center)
						.IsChecked_Lambda(MoveTemp(IsCheckedFunc))
						.IsEnabled_Lambda(MoveTemp(IsEnabledFunc))
						.OnCheckStateChanged_Lambda(MoveTemp(OnCheckStateChangedFunc))
						[
							SArgumentNew(ToolDisplayHBox, SHorizontalBox)
						];
				}

				ToolsArgs
				+ SVerticalBox::Slot()
				.Padding(4.0f)
				.AutoHeight()
				[
					CheckBox.ToSharedRef()
				];
			}

			ToolContainer->AddSlot()
			.HAlign(HAlign_Fill)
			.Padding(0.f)
			.AutoHeight()
			[
				SArgumentNew(ToolsArgs, SVerticalBox)
			];

			// add a space between the tools belonging to each palette
			if (SPaletteIndex < SubmodePalettes.Num() - 1)
			{
				ToolContainer->AddSlot()
				.MaxHeight(1.0f)
				.Padding(4.0f, 0.0f, 4.0f, 0.0f)
				[
					SNew(SSeparator)
					.SeparatorImage(FMeshTerrainModeStyle::Get()->GetBrush("ToolPanel.SeparatorBrush"))
					.Thickness(1.0f)
				];
			}
		}
	}
}

bool SSubmodeToolPanel::IsLabelVisible() const
{
	return DesiredLabelVisibility.IsVisible() && LabelVisibility.IsVisible();
}

void SSubmodeToolPanel::SetActiveSubmode(const TSharedPtr<FSubmode>& SubmodePtr)
{
	ActiveSubmodePtr = SubmodePtr;
	UpdateToolsPalette();
}

