// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Fonts/SlateFontInfo.h"
#include "Types/SlateStructs.h"
#include "Widgets/SCompoundWidget.h"

class FUICommandList;
struct FButtonArgs;
class SVerticalBox;
struct FToolMenuContext;
namespace UE::MeshTerrain
{

struct FSubmodeToolPalette;
class FSubmode;

class SSubmodeToolPanel : public SCompoundWidget
{
public:
	SLATE_BEGIN_ARGS( SSubmodeToolPanel ){}
	SLATE_ATTRIBUTE(FOptionalSize, Width)
	SLATE_ATTRIBUTE(FSlateFontInfo, Font)
	SLATE_ARGUMENT_DEFAULT(EVisibility, LabelVisibility) = EVisibility::Visible;
	SLATE_END_ARGS();

	SSubmodeToolPanel() {}

	void Construct(const FArguments& InArgs);

	/** Assign the Command List for the Tool Container. */
	void SetToolCommandList(const TSharedRef<FUICommandList>& CommandList);

	void SetActiveSubmode(const TSharedPtr<UE::MeshTerrain::FSubmode>& SubmodePtr);

	EVisibility GetLabelVisibility() const;
	void SetLabelVisibility(EVisibility Vis);

private:
	void UpdateToolsPalette();
	
	bool IsLabelVisible() const;

	TAttribute<FOptionalSize> Width = 140.0f;
	TAttribute<FSlateFontInfo> Font;

	EVisibility DesiredLabelVisibility = EVisibility::Visible;
	EVisibility LabelVisibility = EVisibility::Visible;
	
	TSharedPtr<FUICommandList> ToolCommandList;

	/** A pointer to the currently active palette. */
	TSharedPtr<UE::MeshTerrain::FSubmode> ActiveSubmodePtr = nullptr;
	
	TSharedPtr<SVerticalBox> ToolContainer;
};

// -----------------------------------------------------------------------------------------------------------------------------

class SSubmodesPanel : public SCompoundWidget
{
public:
	SLATE_BEGIN_ARGS( SSubmodesPanel ){}
		SLATE_ARGUMENT_DEFAULT(EVisibility, LabelVisibility) = EVisibility::Visible;
	SLATE_END_ARGS();

	SSubmodesPanel() {}

	void Construct(const FArguments& InArgs);

	/** Register a submode with the palette. */
	void AddSubmode(const TSharedPtr<UE::MeshTerrain::FSubmode>& SubmodePtr);

	/** Gets the pointer to the FUICommandList for the FUICommandInfos which load a tool palette. */
	const TSharedPtr<FUICommandList>& GetSubmodePaletteCommandList();

	/** Enter the given Submode. */
	void EnterSubmode(const TSharedPtr<UE::MeshTerrain::FSubmode>& SubmodePtr);

	/** Creates the palette containing all submodes. */
	void CreateSubmodesPalette();
	
	/** Retrieve the active submode pointer */
	TSharedPtr<UE::MeshTerrain::FSubmode> GetActiveSubmodePtr() { return ActiveSubmodePtr; };
	
	/** Delegate for when the active submode is changed */
	FSimpleMulticastDelegate OnSubmodePaletteChanged;

private:
	EVisibility LabelVisibility = EVisibility::Visible;
	
	TArray<TSharedPtr<UE::MeshTerrain::FSubmode>> SubmodeList;
	
	TSharedPtr<FUICommandList> SubmodePaletteCommandList;
	
	/** A pointer to the currently active palette. */
	TSharedPtr<UE::MeshTerrain::FSubmode> ActiveSubmodePtr = nullptr;

	/** Containers for the change submode widgets. */
	TSharedPtr<SVerticalBox> SubmodeContainer;
};

// -----------------------------------------------------------------------------------


/** A class to define a submode palette. */
class SSubmodePalette : public SCompoundWidget
{
public:
	SLATE_BEGIN_ARGS( SSubmodePalette ){}
	SLATE_ATTRIBUTE(FOptionalSize, Width)
	SLATE_ATTRIBUTE(FSlateFontInfo, ToolPanelFont)
	SLATE_END_ARGS();

	SSubmodePalette() {}

	void Construct(const FArguments& InArgs);

	/** Register a submode with the palette. */
	void AddSubmode(const TSharedPtr<UE::MeshTerrain::FSubmode>& SubmodePtr) { SubmodeContainer->AddSubmode(SubmodePtr); };

	/** Gets the pointer to the FUICommandList for the FUICommandInfos which load a tool palette. */
	const TSharedPtr<FUICommandList>& GetSubmodePaletteCommandList() { return SubmodeContainer->GetSubmodePaletteCommandList(); };

	/** Assign the Command List for the Tool Container. */
	void SetToolCommandList(const TSharedRef<FUICommandList>& CommandList) { ToolContainer->SetToolCommandList(CommandList); };

	/** Enter the given Submode. */
	void EnterSubmode(const TSharedPtr<UE::MeshTerrain::FSubmode>& SubmodePtr) { SubmodeContainer->EnterSubmode(SubmodePtr); };

	/** Creates the palette containing all submodes. */
	void CreateSubmodesPalette() { SubmodeContainer->CreateSubmodesPalette(); };

private:

	FDelegateHandle ActivePaletteChangedHandle;

	TAttribute<FOptionalSize> Width = 200.0f;
	TAttribute<FSlateFontInfo> ToolPanelFont;
	
	/** Containers for the change submode widgets and the submodes's tool widgets. */
	TSharedPtr<SSubmodesPanel> SubmodeContainer;
	TSharedPtr<SSubmodeToolPanel> ToolContainer;
};

}